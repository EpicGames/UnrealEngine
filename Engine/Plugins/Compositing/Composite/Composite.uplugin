{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.1",
	"FriendlyName": "Composure",
	"Description": "Modern system for real-time compositing. This plugin succeeds legacy Composure and extends CompositeCore.",
	"Category": "Compositing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "Composite",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "CompositeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CompositeCore",
			"Enabled": true
		},
		{
			"Name": "ComposureShared",
			"Enabled": true
		},
		{
			"Name": "OpenColorIO",
			"Enabled": true
		},
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "MediaFrameworkUtilities",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "MeshModelingToolset",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
		{
			"Name": "ObjectMixer",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
		{
			"Name": "ColorGrading",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "VPRoles",
			"Enabled": true
		}
	]
}