[CoreRedirects]

+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutComposite", NewName="/Script/Composite.CompositeActor")
+ClassRedirects=(OldName="/Script/Composite.Composite", NewName="/Script/Composite.CompositeActor")
+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositePassBase", NewName="/Script/Composite.CompositePassBase")
+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositePassMaterial", NewName="/Script/Composite.CompositePassMaterial")
+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositePassFXAA", NewName="/Script/Composite.CompositePassFXAA")
+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositeViewProjectionComponent", NewName="/Script/Composite.CompositeViewProjectionComponent")

+EnumRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositePassInput",NewName="/Script/Composite.CompositePassInput")

+StructRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositeProcessingPasses", NewName="/Script/Composite.CompositeProcessingPasses")

+PackageRedirects=(OldName="/HoldoutComposite/MaterialFunctions/MF_HoldoutComposite_ProjectedUVs", NewName="/Composite/MaterialFunctions/MF_Composite_ProjectedUVs")
+PackageRedirects=(OldName="/HoldoutComposite/Materials/MPC_HoldoutComposite", NewName="/Composite/Materials/MPC_Composite")
+PackageRedirects=(OldName="/HoldoutComposite/Materials/M_HoldoutComposite_KeyedProjection", NewName="/Composite/Materials/M_Composite_KeyedProjection")

; 5.7

+ClassRedirects=(OldName="/Script/Composite.CompositePassLayerBase", NewName="/Script/Composite.CompositeLayerBase")
+ClassRedirects=(OldName="/Script/Composite.CompositePassLayerMainRender", NewName="/Script/Composite.CompositeLayerMainRender")
+ClassRedirects=(OldName="/Script/Composite.CompositePassLayerPlate", NewName="/Script/Composite.CompositeLayerPlate")
+ClassRedirects=(OldName="/Script/Composite.CompositePassLayerShadowReflection", NewName="/Script/Composite.CompositeLayerShadowReflection")
+ClassRedirects=(OldName="/Script/Composite.CompositeCenteredScalePass", NewName="/Script/Composite.CompositePassCenteredScale")
+ClassRedirects=(OldName="/Script/Composite.CompositeFXAAPass", NewName="/Script/Composite.CompositePassFXAA")

+PropertyRedirects=(OldName="/Script/Composite.CompositeActor.CompositeMeshActors", NewName="/Script/Composite.CompositeLayerPlate.CompositeMeshes")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerPlate.Passes", NewName="/Script/Composite.CompositeLayerPlate.LayerPasses")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerPlate.Subpasses", NewName="/Script/Composite.CompositeLayerPlate.LayerPasses")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerPlate.MediaTexture", NewName="/Script/Composite.CompositeLayerPlate.Texture")

+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerPlate.Preprocessing", NewName="/Script/Composite.CompositeLayerPlate.MediaPasses")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerPlate.Postprocessing", NewName="/Script/Composite.CompositeLayerPlate.LayerPasses")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerSceneCapture.Postprocessing", NewName="/Script/Composite.CompositeLayerSceneCapture.LayerPasses")

+EnumRedirects=(OldName="/Script/Composite.CompositeDeltaKeyerVisualization",NewName="/Script/Composite.CompositeColorKeyerVisualization")
+EnumRedirects=(OldName="/Script/Composite.CompositeDeltaKeyerScreenType",NewName="/Script/Composite.CompositeColorKeyerScreenType")
+ClassRedirects=(OldName="/Script/Composite.CompositePassDeltaKeyer", NewName="/Script/Composite.CompositePassColorKeyer")

; 5.8

+PropertyRedirects=(OldName="/Script/Composite.CompositeViewProjectionComponent.MatrixParameterName", NewName="/Script/Composite.CompositeViewProjectionComponent.ViewProjectionMatrixParameter")
+ClassRedirects=(OldName="/Script/Composite.CompositePassColorGrade",NewName="/Script/Composite.CompositePassColorGrading")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassCenteredScale.SourceAspectRatio", NewName="/Script/Composite.CompositePassCenteredScale.ContainerAspectRatio")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassCenteredScale.TargetAspectRatio", NewName="/Script/Composite.CompositePassCenteredScale.ContentAspectRatio")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassCenteredScale.ScaleFactor", NewName="/Script/Composite.CompositePassCenteredScale.ManualScale")
+ClassRedirects=(OldName="/Script/Composite.CompositePassAlphaTransform",NewName="/Script/Composite.CompositePassTranslucency")
+ClassRedirects=(OldName="/Script/Composite.CompositePassCenteredScale",NewName="/Script/Composite.CompositePassTransform2D")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassDilation.Mode", Removed=True)
+PropertyRedirects=(OldName="/Script/Composite.CompositePassDilation.AlphaThreshold", NewName="/Script/Composite.CompositePassDilation.Threshold")
+PropertyRedirects=(OldName="/Script/Composite.CompositeActor.SceneCapturesPerLayer", NewName="/Script/Composite.CompositeActor.SceneCapturesPerPass")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassColorGrading.bUnpremult", NewName="/Script/Composite.CompositePassColorGrading.bInputIsPremultiplied")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassMasking.bUnpremult", NewName="/Script/Composite.CompositePassMasking.bInputIsPremultiplied")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerMainRender.bBlendInDisplaySpace", NewName="/Script/Composite.CompositeLayerMainRender.bUltimatteBlend")
+PropertyRedirects=(OldName="/Script/Composite.CompositeLayerPlate.bBlendInDisplaySpace", NewName="/Script/Composite.CompositeLayerPlate.bUltimatteBlend")
+PropertyRedirects=(OldName="/Script/Composite.CompositePassOpenColorIO.bUnpremult", NewName="/Script/Composite.CompositePassOpenColorIO.bInputIsPremultiplied")
+PropertyRedirects=(OldName="/Script/Composite.CompositeColorGradingPassSettings.bUnpremult", NewName="/Script/Composite.CompositeColorGradingPassSettings.bInputIsPremultiplied")
