// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Public/Platform.ush"

#include "CompositeCommon.ush"

// See https://www.rastergrid.com/blog/2010/09/efficient-gaussian-blur-with-linear-sampling/

Texture2D Input_Texture;
SamplerState Input_Sampler;
FScreenTransform SvPositionToInputTextureUV;
SCREEN_PASS_TEXTURE_VIEWPORT(Input)
float2 Dir;
float Radius;
uint bInvertedAlpha;

#if ALPHA_ONLY
    #define BLUR_TYPE float
    #define TO_BLUR_TYPE(x) (x).a
#else
    #define BLUR_TYPE float4
    #define TO_BLUR_TYPE(x) (x)
#endif

float NormalDistribution1D(float X)
{
    float X2 = PI * max(X, 0.0);
    return exp(-0.5 * X2 * X2);
}

float2 ClampInputUV(float2 UV)
{
    return clamp(UV, Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);
}

/** Samples the input texture, normalizing inverted alpha (e.g. UE scene color) to standard opacity for processing. */
float4 SampleInputTexture(float2 UV)
{
    float4 Color = Input_Texture.SampleLevel(Input_Sampler, ClampInputUV(UV), 0.0);
    Color.a = InvertAlpha(Color.a, bInvertedAlpha);
    
    return Color;
}

BLUR_TYPE GaussianBlur(float2 UV)
{
    float2 Scale = Dir * Input_ViewportSizeInverse;
    BLUR_TYPE Accum = (BLUR_TYPE)0;
    float Total = 0.0;
    float InvRadius = 1.0 / max(Radius, 1e-6);

    // Handle center first
    Accum += TO_BLUR_TYPE(SampleInputTexture(UV));
    Total += 1.0;

    // Determine how many pairs we can process symmetrically with bilinear taps
    int NumPairs = (int)floor(Radius / 2.0);

    // Want to replace two samples (Sample(d1) * w1) + (Sample(d2) * 22) with one Sample(O) * (w1 + w2).
    // Per article, offset is the weighted average O = ((d1 * w1) + (d2 * w2)) / (w1 + w2).
    // By replacing d2 with (d1 - 1) and simplifying, we get O = d1 - w2 / (w1 + w2).
    LOOP
    for (int Index = 0; Index < NumPairs; ++Index)
    {
        // Calculate distances for the two texels in this pair
        float D1 = Radius - (Index * 2);
        float D2 = D1 - 1.0f;

        float W1 = NormalDistribution1D(D1 * InvRadius);
        float W2 = NormalDistribution1D(D2 * InvRadius);

        float CombinedWeight = W1 + W2;
        float Offset = D1 - W2 / CombinedWeight;
        float2 UVOffset = Scale * Offset;

        Accum += TO_BLUR_TYPE(SampleInputTexture(UV + UVOffset)) * CombinedWeight;
        Accum += TO_BLUR_TYPE(SampleInputTexture(UV - UVOffset)) * CombinedWeight;
        Total += CombinedWeight * 2.0;
    }

    // Handle the leftover pixel if Radius is odd
    if ((int)Radius % 2 != 0 && Radius >= 1.0)
    {
        float W = NormalDistribution1D(1.0 * InvRadius);
        float2 UVOffset = Scale * 1.0;

        Accum += TO_BLUR_TYPE(SampleInputTexture(UV + UVOffset)) * W;
        Accum += TO_BLUR_TYPE(SampleInputTexture(UV - UVOffset)) * W;
        Total += W * 2.0;
    }

    return Accum / max(Total, 1e-6);
}

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 InputUV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);

    BLUR_TYPE Blurred = GaussianBlur(InputUV);

/** Restore original alpha convention on output (symmetric with SampleInputTexture). */
#if ALPHA_ONLY
    OutColor = float4(SampleInputTexture(InputUV).rgb, InvertAlpha(Blurred, bInvertedAlpha));
#else
    OutColor = float4(Blurred.rgb, InvertAlpha(Blurred.a, bInvertedAlpha));
#endif
}
