// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Private/TonemapCommon.ush"
#include "/Engine/Private/ACES/ACESCommon.ush"

#include "CompositeCommon.ush"

static const float DefaultTemperature = 6500.0;
static const float DefaultTint = 0.0;
static const float Epsilon = 1e-8;

Texture2D Input_Texture;
SamplerState Input_Sampler;
FScreenTransform SvPositionToInputTextureUV;

uint bIsTemperatureWhiteBalance;
float WhiteTemp;
float WhiteTint;

float4 Global_Saturation;
float4 Global_Contrast;
float4 Global_Gamma;
float4 Global_Gain;
float4 Global_Offset;

float4 Shadows_Saturation;
float4 Shadows_Contrast;
float4 Shadows_Gamma;
float4 Shadows_Gain;
float4 Shadows_Offset;

float4 Midtones_Saturation;
float4 Midtones_Contrast;
float4 Midtones_Gamma;
float4 Midtones_Gain;
float4 Midtones_Offset;

float4 Highlights_Saturation;
float4 Highlights_Contrast;
float4 Highlights_Gamma;
float4 Highlights_Gain;
float4 Highlights_Offset;

uint bInputIsPremultiplied;
uint bInvertedAlpha;
float ShadowsMax;
float HighlightsMin;
float HighlightsMax;

float3 ColorCorrect( float3 WorkingColor,
	float4 ColorSaturation,
	float4 ColorContrast,
	float4 ColorGamma,
	float4 ColorGain,
	float4 ColorOffset )
{
	float Luma = dot( WorkingColor, AP1_RGB2Y );
	WorkingColor = max( 0, lerp( Luma.xxx, WorkingColor, ColorSaturation.xyz*ColorSaturation.w ) );
	WorkingColor = pow( WorkingColor * (1.0 / 0.18), ColorContrast.xyz*ColorContrast.w ) * 0.18;
	WorkingColor = pow( WorkingColor, 1.0 / (ColorGamma.xyz*ColorGamma.w) );
	WorkingColor = WorkingColor * (ColorGain.xyz * ColorGain.w) + (ColorOffset.xyz + ColorOffset.w);
	return WorkingColor;
}

// Nuke-style Color Correct
float3 ColorCorrectAll( float3 WorkingColor )
{
	float Luma = dot( WorkingColor, AP1_RGB2Y );

	// Shadow CC
	float3 CCColorShadows = ColorCorrect(WorkingColor, 
		Shadows_Saturation*Global_Saturation, 
		Shadows_Contrast*Global_Contrast, 
		Shadows_Gamma*Global_Gamma, 
		Shadows_Gain*Global_Gain, 
		Shadows_Offset+Global_Offset);
	float CCWeightShadows = 1- smoothstep(0, ShadowsMax, Luma);
	
	// Highlight CC
	float3 CCColorHighlights = ColorCorrect(WorkingColor, 
		Highlights_Saturation*Global_Saturation, 
		Highlights_Contrast*Global_Contrast, 
		Highlights_Gamma*Global_Gamma, 
		Highlights_Gain*Global_Gain, 
		Highlights_Offset+Global_Offset);
	float CCWeightHighlights = smoothstep(HighlightsMin, HighlightsMax, Luma);

	// Midtone CC
	float3 CCColorMidtones = ColorCorrect(WorkingColor, 
		Midtones_Saturation*Global_Saturation, 
		Midtones_Contrast*Global_Contrast, 
		Midtones_Gamma*Global_Gamma, 
		Midtones_Gain*Global_Gain, 
		Midtones_Offset+Global_Offset);
	float CCWeightMidtones = 1 - CCWeightShadows - CCWeightHighlights;

	// Blend Shadow, Midtone and Highlight CCs
	float3 WorkingColorSMH = CCColorShadows*CCWeightShadows + CCColorMidtones*CCWeightMidtones + CCColorHighlights*CCWeightHighlights;
	
	return WorkingColorSMH;
}

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 InputUV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);
    float4 LinearColor = Input_Texture.Sample(Input_Sampler, InputUV);
	
	/** Normalize inverted alpha (e.g. UE scene color) to standard opacity for processing. */
	LinearColor.a = InvertAlpha(LinearColor.a, bInvertedAlpha);

	if(bInputIsPremultiplied)
	{
		LinearColor = Unpremultiply(LinearColor);
	}

    BRANCH
	if( abs(WhiteTemp - DefaultTemperature) > Epsilon || abs(WhiteTint - DefaultTint) > Epsilon )
	{
		LinearColor.rgb = WhiteBalance(LinearColor.rgb, WhiteTemp, WhiteTint, bIsTemperatureWhiteBalance, (float3x3) WorkingColorSpace.ToXYZ, (float3x3) WorkingColorSpace.FromXYZ);
	}

    float4 ColorAP1 = mul(WorkingColorSpace.ToAP1, LinearColor);
    
    ColorAP1.rgb = ColorCorrectAll(ColorAP1.rgb);

	LinearColor = mul(WorkingColorSpace.FromAP1, ColorAP1);
	
	if(bInputIsPremultiplied)
	{
		LinearColor = Premultiply(LinearColor);
	}

	/** Restore original alpha convention on output. */
	LinearColor.a = InvertAlpha(LinearColor.a, bInvertedAlpha);

	OutColor = LinearColor;
}
