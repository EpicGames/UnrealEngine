// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

#include "CompositeColorKeyerCommon.ush"

Texture2D Input_Texture;
SamplerState Input_Sampler;

Texture2D CleanPlate_Texture;
SamplerState CleanPlate_Sampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)
FScreenTransform SvPositionToInputTextureUV;

float3 KeyColor;
float4 Params0;
float4 Params1;
uint KeyerMode;
uint bDisplaySpaceLuminance;
uint ScreenType;
uint Visualization;
uint bInvertAlpha;
uint bPreserveVignetteAfterKey;

float2 UVToScreenPos(float2 UV, float2 Extent)
{
	return (UV * Extent - Input_ScreenPosToViewportBias) / Input_ScreenPosToViewportScale;
}

// Output color function with debug convenience
float4 Output(float4 Color)
{
	// Saturate and optionally invert alpha
	Color.a = saturate(select(bInvertAlpha > 0, 1.0f - Color.a, Color.a));

	if(Visualization == 1)
	{
		// Opaque to show fill
		Color.a = 1.0f;
	}
	else if(Visualization == 2)
	{
		// Optionally show the alpha key matte
		Color.rgb = sRGBToLinear(Color.a);
		Color.a = 1.0f;
	}

	// Keyed output colors should always be premultiplied
	return Premultiply(Color);
}

float4 MainPS(float4 SvPosition : SV_POSITION) : SV_Target0
{
	float2 InputUV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);

#if USE_CHROMA_SHIFT
	float ChromaShift = Params1.w;
	float4 ForegroundCenter = Input_Texture.SampleLevel(Input_Sampler, InputUV, 0);
	float2 ShiftedUV = InputUV + float2(ChromaShift * rcp(Input_Extent.x), 0.0f);
	float4 Foreground = Input_Texture.SampleLevel(Input_Sampler, ShiftedUV, 0);
	Foreground.rgb += Luminance(ForegroundCenter.rgb) - Luminance(Foreground.rgb);
	Foreground.a = ForegroundCenter.a;
#else
	float4 Foreground = Input_Texture.SampleLevel(Input_Sampler, InputUV, 0);
#endif
#if USE_CLEAN_PLATE
	float3 Background = CleanPlate_Texture.SampleLevel(CleanPlate_Sampler, InputUV, 0).rgb;
#else
	float3 Background = KeyColor;
#endif

	float4 Result = Foreground;

#if !USE_DESPILL_ONLY
	float2 ScreenPos = UVToScreenPos(InputUV, Input_Extent);
	float AspectRatio = Output_ViewportSize.y * Output_ViewportSizeInverse.x;
	float DevignetteStrength = Params1.z;
	float AlphaMin = Params1.x;
	float AlphaMax = Params1.y;
	float VignetteMask = Vignette(ScreenPos, AspectRatio, DevignetteStrength);

	BRANCH
	if (KeyerMode == 1) // Luma key: derive alpha from pixel luminance.
	{
		// C++ packs LumaRange into Params1.xy when KeyerMode == Luma.
		float3 LumaInput = bDisplaySpaceLuminance ? LinearToSrgb(Foreground.rgb) : Foreground.rgb;
		float Luma = Luminance(LumaInput);
		Result.a = Linearstep(AlphaMin, AlphaMax, Luma) * Foreground.a;
	}
	else // Chroma key (KeyerMode == 0)
	{
		float3 ChannelWeights = Params0.xyz;

		if(DevignetteStrength > ZeroEpsilon)
		{
			Result.rgb *= rcp(VignetteMask);
		}

		BRANCH
		if(ScreenType == 0)
		{
			Result = RedScreenKeyer(Result, Background, ChannelWeights[1], ChannelWeights[2]);
		}
		else if(ScreenType == 1)
		{
			Result = GreenScreenKeyer(Result, Background, ChannelWeights[0], ChannelWeights[2]);
		}
		else if(ScreenType == 2)
		{
			Result = BlueScreenKeyer(Result, Background, ChannelWeights[0], ChannelWeights[1]);
		}
	}
#endif

	float DespillStrength = Params0.w;

	BRANCH
	if(DespillStrength > 0.0f)
	{
		// Map the source color onto what we call the "foreground".
		// Convert both to Luv color space
		float3 Foreground_Luv = RGB_to_Luv(Result.rgb);
		float3 Background_Luv = RGB_to_Luv(Background);

		// Project the foreground color onto the background in uv-space
		float2 ProjectedDiff = ProjectClamped(Foreground_Luv.yz, Background_Luv.yz);

		// Despill the foreground
		Foreground_Luv.yz -= DespillStrength * ProjectedDiff;

    	// Convert back to RGB
		Result.rgb = Luv_to_RGB(Foreground_Luv);
	}

#if !USE_DESPILL_ONLY
	// Chroma-only post-processing: vignette re-application and alpha threshold.
	// In luma mode DespillStrength and DevignetteStrength are packed as 0, so both branches
	// are natural no-ops, but we still gate on KeyerMode to make the intent explicit.
	if (KeyerMode == 0)
	{
		if(bPreserveVignetteAfterKey > 0 && DevignetteStrength > ZeroEpsilon)
		{
			Result.rgb *= VignetteMask;
		}

		// Apply final alpha threshold
		Result.a = Linearstep(AlphaMin, AlphaMax, Result.a);
	}
#endif

	return Output(Result);
}
