// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/ACES/ACESCommon.ush"
#include "/Engine/Private/ColorSpace.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"
#include "/Engine/Private/PostProcessCommon.ush"

#include "CompositeCommon.ush"

// CIE 1931 D65 white point
static const float3 XYZ_n = xyY_2_XYZ(float3(0.3127, 0.3290, 1.0));
static const float ZeroEpsilon = 1.0e-6;
static const float DeltaE_JND = 1.0;
static const float Lstar_CIELUV = 100.0;
static const float Lstar_CIELUV_Inv = 1.0 / Lstar_CIELUV;

float LuminanceSRGB(float3 InColor)
{
	return dot(InColor, float3(0.2126390059, 0.7151686788, 0.0721923154));
}

// Linear equivalent of smoothstep
float Linearstep(float Edge0, float Edge1, float X)
{
	return saturate((X - Edge0) / (Edge1 - Edge0));
}

float2 Project(float2 VecA, float2 VecB)
{
	float AdotB = dot(VecA, VecB);
	float BdotB = dot(VecB, VecB);
	
	return select(BdotB < ZeroEpsilon, float2(0, 0), VecB * (AdotB / BdotB));
}

// Project vector A onto B, and return 0 if the projection was negative
float2 ProjectClamped(float2 VecA, float2 VecB)
{
	float AdotB = max(0.0f, dot(VecA, VecB));
	float BdotB = dot(VecB, VecB);
	
	return select(BdotB < ZeroEpsilon, float2(0, 0), VecB * (AdotB / BdotB));
}

float3 ProjectClamped(float3 VecA, float3 VecB)
{
	float AdotB = max(0.0f, dot(VecA, VecB));
	float BdotB = dot(VecB, VecB);

	return select(BdotB < ZeroEpsilon, float3(0, 0, 0), VecB * (AdotB / BdotB));
}

float GetAngle(float2 A, float2 B)
{
	return acos(dot(normalize(A), normalize(B)));
}

float2 Rotate2D(float2 V, float Theta)
{
	float S, C;
	sincos(Theta, S, C);
	return float2(C*V.x - S*V.y,
				  S*V.x + C*V.y);
}

// Convert color from XYZ to CIE 1976 L*, u*, v* color space.
// We use the latter for its improved perceptual uniformity, as a way to measure chromaticity distance for keying.
float3 CIE_XYZ_to_Luv(float3 XYZ)
{
	const float Epsilon = 216.0f / 24389.0f; // (6/29)^3
	const float Kappa = 24389.0f / 27.0f; // (29/3)^3

	float L = select(XYZ[1] / XYZ_n[1] > Epsilon, 116 * pow(XYZ[1]/ XYZ_n[1], 1.0/3.0) - 16, Kappa * XYZ[1] / XYZ_n[1]);

	if (L < ZeroEpsilon)
	{
		return float3(0.0f, 0.0f, 0.0f);
	}
	
	float Denom = XYZ[0] + 15 * XYZ[1] + 3 * XYZ[2];
	float Denom_n = XYZ_n[0] + 15 * XYZ_n[1] + 3 * XYZ_n[2];
	
	float up = select(Denom > 0.0f, 4 * XYZ[0] / Denom, 0.0f);
	float vp = select(Denom > 0.0f, 9 * XYZ[1] / Denom, 0.0f);

	float u_n = 4 * XYZ_n[0] / Denom_n;
	float v_n = 9 * XYZ_n[1] / Denom_n;

	float u = 13 * L * (up - u_n);
	float v = 13 * L * (vp - v_n);

	return float3(L, u, v);
}

// Convert color from CIE 1976 L*, u*, v* to XYZ color space.
float3 CIE_Luv_to_XYZ(float3 Luv)
{
	const float Epsilon = 216.0f / 24389.0f; // (6/29)^3
	const float Kappa = 24389.0f / 27.0f; // (29/3)^3

	float L = Luv[0];
	float u = Luv[1];
	float v = Luv[2];

	// Handle the case where L is zero, which corresponds to black.
	if (L < ZeroEpsilon)
	{
		return float3(0.0f, 0.0f, 0.0f);
	}	

	float Y_ratio = select(L > Kappa * Epsilon, pow((L + 16.0f) / 116.0f, 3.0f), L / Kappa);
	float Y = Y_ratio * XYZ_n[1];

	float Denom_n = XYZ_n[0] + 15.0f * XYZ_n[1] + 3.0f * XYZ_n[2];
	float u_n = 4.0f * XYZ_n[0] / Denom_n;
	float v_n = 9.0f * XYZ_n[1] / Denom_n;
	
	float up = u / (13.0f * L) + u_n;
	float vp = v / (13.0f * L) + v_n;

	float X = Y * (9.0f * up) / (4.0f * vp);
	float Z = Y * (12.0f - 3.0f * up - 20.0f * vp) / (4.0f * vp);
	
	return float3(X, Y, Z);
}

float3 RGB_to_Luv(float3 InRGB)
{
	return CIE_XYZ_to_Luv(LinearRGB_2_XYZ(InRGB));
}

float3 Luv_to_RGB(float3 InLuv)
{
	return XYZ_2_LinearRGB(CIE_Luv_to_XYZ(InLuv));
}

float3 GreenDespillAvg(float3 Color)
{
	float Avg = 0.5f * (Color.r + Color.b);

	return float3(
		Color.r,
		select(Color.g > Avg, Avg, Color.g),
		Color.b
	);
}

float3 BlueDespillAvg(float3 Color)
{
	float Avg = 0.5f * (Color.r + Color.g);

	return float3(
		Color.r,
		Color.g,
		select(Color.b > Avg, Avg, Color.b)
	);
}

float Vignette(float2 ScreenPos, float AspectRatio, float Intensity)
{
	// Scale vignette to always be a circle with consistent corner intensity.
	// We compute the raw cosine-fourth-law scalar directly rather than using
	// ComputeVignetteMask(), which now returns a float3 color tint intended for
	// the post-process stack. The devignette path needs a plain 0-1 scalar to
	// divide out the lens falloff.
	float2 VignetteCircleSpacePos = VignetteSpace(ScreenPos, VIGNETTE_TYPE_COSINE_FOURTH_LAW, AspectRatio);
	VignetteCircleSpacePos *= Intensity;
	float Tan2Angle = dot( VignetteCircleSpacePos, VignetteCircleSpacePos );
	
	return Square( rcp( Tan2Angle + 1 ) );
}

float4 GreenScreenKeyer(
	float4 Foreground,
	float3 Background,
	float  RedWeight,
	float  BlueWeight)
{
	// Shared weighted sums
	float wF = RedWeight * Foreground.r + BlueWeight * Foreground.b;
	float wB = RedWeight * Background.r + BlueWeight * Background.b;

	// Green minus weighted red/blue
	float Fg = Foreground.g - wF;
	float Bg = Background.g - wB;

	// Ratios
	float InvBg       = SafeRcp(Bg);
	float GreenRatio  = saturate(Fg * InvBg);
	float Alpha       = 1 - GreenRatio;

	// Built-in "despill"
	float3 DespillF = float3(Foreground.r, wF, Foreground.b);
	float3 DespillB = float3(Background.r, wB, Background.b);

	// Masked background
	float3 MaskedB = DespillB * GreenRatio;

	// Handle screen brighter than sample color
	float3 Delta = Foreground.rgb - Background;
	float Mg     = Delta.g - (RedWeight * Delta.r + BlueWeight * Delta.b);
	MaskedB.g   += saturate(Mg * InvBg * wB);

	// Fill
	float3 Fill = DespillF - MaskedB;

	// Re-insert removed green
	Fill.g += min(Fg, 0.0);
	// Prevent negativtes
	Fill = max(Fill, 0.0);

	// Hack to make sure the bright areas of the plate are solid
	Alpha = max(Alpha, Fill.r);
	// Preserve original alpha
	Alpha *= Foreground.a;
	// Clamp alpha
	Alpha = saturate(Alpha);

	// Fill is already considered premultiplied, so we un-premultiply it for subsequent color operations.
	Fill *= SafeRcp(Alpha);

	return float4(Fill, Alpha);
}

float3 SwizzleRG(float3 InColor)
{
	return float3(InColor.g, InColor.r, InColor.b);
}

float3 SwizzleGB(float3 InColor)
{
	return float3(InColor.r, InColor.b, InColor.g);
}

float4 SwizzleRG(float4 InColor)
{
	return float4(InColor.g, InColor.r, InColor.b, InColor.a);
}

float4 SwizzleGB(float4 InColor)
{
	return float4(InColor.r, InColor.b, InColor.g, InColor.a);
}

float4 RedScreenKeyer(float4 Foreground, float3 Background, float GreenWeight, float BlueWeight)
{
	float4 SwizzledForeground = SwizzleRG(Foreground);
	float3 SizzledBackground = SwizzleRG(Background);
	float4 Output = GreenScreenKeyer(SwizzledForeground, SizzledBackground, GreenWeight, BlueWeight);
	Output.rgb = SwizzleRG(Output.rgb);

	return Output;
}

float4 BlueScreenKeyer(float4 Foreground, float3 Background, float RedWeight, float GreenWeight)
{
	float4 SwizzledForeground = SwizzleGB(Foreground);
	float3 SizzledBackground = SwizzleGB(Background);
	float4 Output = GreenScreenKeyer(SwizzledForeground, SizzledBackground, RedWeight, GreenWeight);
	Output.rgb = SwizzleGB(Output.rgb);

	return Output;
}

// Experimental
float4 LuvDistanceKeyer(float3 Foreground_Luv, float3 Background_Luv, float MinDistance, float MaxDistance)
{
	const float kDistanceScaleFactor_CIELUV = 2 * DeltaE_JND / Lstar_CIELUV;
	float Delta = distance(Foreground_Luv.yz, Background_Luv.yz) * kDistanceScaleFactor_CIELUV;

	float Alpha = Linearstep(MinDistance, MaxDistance, Delta);

	// TODO: Modulate Foreground_Luv.x
	return float4(Foreground_Luv, Alpha);
}

// Experimental
float4 LuvAngleKeyer(float3 Foreground_Luv, float3 Background_Luv, float Acceptance, float Scale)
{
	// Similar to approach described in Video Demystified  - A Handbook for the Digital Engineer
	float AcceptanceAngle = 2 * PI * Acceptance;
	
	// Angle between the background and the "u" axis.
	float AxisAngle = atan2(Background_Luv.z, Background_Luv.y);
	
	// Axis-aligned foreground uv
	float2 Foreground_uvaa = Rotate2D(Foreground_Luv.yz, AxisAngle);

	float X = Foreground_uvaa.x;
	float Z_sub = abs(Foreground_uvaa.y) / tan(0.5 * AcceptanceAngle);
	float Key = select(X < Z_sub, 0.0f, X - Z_sub);
	
	float Alpha = saturate(1.0 - Lstar_CIELUV_Inv * Key * Scale);

	// TODO: Modulate Foreground_Luv.x
	return float4(Foreground_Luv, Alpha);
}

// Branchless 2-value sort
void Sort2(inout float4 A, inout float4 B)
{
    float4 Temp = A;
    A = min(A, B);
    B = max(Temp, B);
}

// Median of 9 colors (https://bertdobbelaere.github.io/median_networks.html#N9L19D7)
float4 Median9(float4 A0, float4 A1, float4 A2, float4 A3, float4 A4, float4 A5, float4 A6, float4 A7, float4 A8)
{
	// For future optimization, the operations on each line could be done in parallel on separate threads or lanes.
	Sort2(A0, A7); Sort2(A1, A2); Sort2(A3, A5); Sort2(A4, A8);
	Sort2(A0, A2); Sort2(A1, A5); Sort2(A3, A8); Sort2(A4, A7);
	Sort2(A0, A3); Sort2(A1, A4); Sort2(A2, A8); Sort2(A5, A7);
	Sort2(A3, A4); Sort2(A5, A6);
	Sort2(A2, A5); Sort2(A4, A6);
	Sort2(A2, A3); Sort2(A4, A5);
	Sort2(A3, A4);

	return A4;
}

// Denoising with per-channel median filter of 9 colors
float4 DenoiseMedianFilter9(const float4 V[9])
{
    return Median9(V[0], V[1], V[2], V[3], V[4], V[5], V[6], V[7], V[8]);
}
