// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/GammaCorrectionCommon.ush"

#define ESourceEncoding_None  0
#define ESourceEncoding_Gamma 1
#define ESourceEncoding_sRGB  2

// Safe reciprocal
float SafeRcp(float x)
{
	return x > 0.0 ? rcp(x) : 0.0;
}

float4 Unpremultiply(float4 InColor)
{
	return float4(InColor.xyz * SafeRcp(InColor.w), InColor.w);
}

float4 Premultiply(float4 InColor)
{
	return float4(InColor.xyz * InColor.w, InColor.w);
}

float InvertAlpha(float InAlpha, uint bInvertedAlpha)
{
	return select(bInvertedAlpha > 0, 1.0 - InAlpha, InAlpha);
}

float3 Decode(float3 InColor, uint InEncoding)
{
    BRANCH
    if(InEncoding == ESourceEncoding_Gamma)
    {
        return pow(InColor, 2.2);
    }
    else if(InEncoding == ESourceEncoding_sRGB)
    {
        return sRGBToLinear(InColor);
    }
    else
    {
        return InColor;
    }
}

float3 Encode(float3 InColor, uint InEncoding)
{
    BRANCH
    if(InEncoding == ESourceEncoding_Gamma)
    {
        return pow(InColor, 1.0 / 2.2);
    }
    else if(InEncoding == ESourceEncoding_sRGB)
    {
        return LinearToSrgb(InColor);
    }
    else
    {
        return InColor;
    }
}

