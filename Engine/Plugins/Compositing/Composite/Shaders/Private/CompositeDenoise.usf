// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

#include "CompositeColorKeyerCommon.ush"

#ifndef TILE_SIZE
	#error TILE_SIZE must be defined
#endif

#define FILTER_SIZE 1
#define FILTER_COUNT (1 + 2 * FILTER_SIZE) * (1 + 2 * FILTER_SIZE)
// Width in texels of the depth tile cached into LDS.
#define LDS_TILE_WIDTH (TILE_SIZE + 2 * FILTER_SIZE)
// Total number of texels cached in the scene color tile.
#define LDS_ARRAY_SIZE (LDS_TILE_WIDTH * LDS_TILE_WIDTH)
#define THREADGROUP_TOTAL (TILE_SIZE * TILE_SIZE)

Texture2D Input_Texture;
SamplerState Input_Sampler;
SCREEN_PASS_TEXTURE_VIEWPORT(Input)

RWTexture2D<float4> RWOutputTexture;

groupshared float4 SharedColor[LDS_ARRAY_SIZE];

static const int2 kOffsets[FILTER_COUNT] = {
	int2( 0,  0),
	int2(-1,  0),
	int2( 1,  0),
	int2( 0,  1),
	int2( 0, -1),
	int2(-1,  1),
	int2( 1,  1),
	int2( 1, -1),
	int2(-1, -1),
};

// Get the index within the LDS array.
uint GetTileArrayIndexFromPixelOffset(in uint2 GroupThreadId, in int2 PixelOffset)
{
	uint2 TilePos = GroupThreadId + uint2(PixelOffset + FILTER_SIZE);
	return TilePos.x + TilePos.y * (FILTER_SIZE * 2 + TILE_SIZE);
}

float4 SampleCachedColors(in uint2 GroupThreadId, int2 PixelOffset = int2(0,0))
{
	return SharedColor[GetTileArrayIndexFromPixelOffset(GroupThreadId, PixelOffset)];
}


[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex,
	uint2 DispatchThreadId : SV_DispatchThreadID)
{
	{
		// Init LDS, same logic as CompositeCoreDilate & TAA.
		uint2 GroupTexelOffset = GroupId * uint2(TILE_SIZE, TILE_SIZE) - FILTER_SIZE;
		// Each warp thread loads extra texels for the filtering neighbors
		const uint LoadCount = (LDS_ARRAY_SIZE + THREADGROUP_TOTAL - 1) / THREADGROUP_TOTAL;
		uint LinearGroupThreadId = GroupThreadIndex;
		
		UNROLL
		for (uint i = 0; i < LoadCount; i++)
		{
			uint2 TexelLocation =  GroupTexelOffset + uint2(
				LinearGroupThreadId % LDS_TILE_WIDTH,
				LinearGroupThreadId / LDS_TILE_WIDTH);
			TexelLocation = min(TexelLocation, uint2(Input_ViewportSize) - 1);

			if ((LinearGroupThreadId < LDS_ARRAY_SIZE) ||
				(i != LoadCount - 1) ||
				(LDS_ARRAY_SIZE % THREADGROUP_TOTAL) == 0)
			{
				SharedColor[LinearGroupThreadId] = Input_Texture[TexelLocation + uint2(Input_ViewportMin)];
			}
			LinearGroupThreadId += THREADGROUP_TOTAL;
		}
	}

	GroupMemoryBarrierWithGroupSync();

    // Discard out-of-bound texels
    if (any(DispatchThreadId >= uint2(Input_ViewportSize)))
    {
        return;
    }

	float4 TexelColor = SampleCachedColors(GroupThreadId);

	{
		float4 Neighbors[FILTER_COUNT];
		UNROLL
		for(int i=0; i<FILTER_COUNT; i++)
		{
		    Neighbors[i] = SampleCachedColors(GroupThreadId, kOffsets[i]);
		}

		TexelColor = DenoiseMedianFilter9(Neighbors);
	}

	RWOutputTexture[DispatchThreadId] = TexelColor;
}
