// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/LensDistortion.ush"
#include "/Engine/Private/ScreenPass.ush"

Texture2D Input_Texture;
SamplerState Input_Sampler;
SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

#define EDistortionUV_None        0
#define EDistortionUV_Distorted   1
#define EDistortionUV_Undistorted 2

Texture2D<float2> DistortingDisplacementTexture;
SamplerState DistortingDisplacementSampler;
Texture2D<float2> UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;

uint DistortionUV;

float2 CalcInputUVs(float2 ViewportUV, float2 ViewportSize, float2 ViewportMin, float2 ExtentInverse, uint InDistortionUV)
{
    BRANCH
    if(InDistortionUV == EDistortionUV_Distorted)
    {
        ViewportUV = ApplyLensDistortionOnViewportUV(DistortingDisplacementTexture, DistortingDisplacementSampler, ViewportUV);
    }
    else if(InDistortionUV == EDistortionUV_Undistorted)
    {
        ViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, ViewportUV);
    }

    return (ViewportUV * ViewportSize + ViewportMin) * ExtentInverse;
}

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 ViewportUV = (SvPosition.xy - Output_ViewportMin) * Output_ViewportSizeInverse;

    float2 Texture_UV = CalcInputUVs(ViewportUV, Input_ViewportSize, Input_ViewportMin, Input_ExtentInverse, DistortionUV);

	OutColor = Input_Texture.SampleLevel(Input_Sampler, Texture_UV, 0);
}
