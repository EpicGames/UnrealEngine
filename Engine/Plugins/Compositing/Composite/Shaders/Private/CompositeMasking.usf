// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

#include "CompositeCommon.ush"

#define INPUT_PASS_TEXTURE_MEMBER(StructName, MemberType, MemberName) MemberType StructName##_##MemberName;

#define INPUT_PASS_TEXTURE(StructName) \
	INPUT_PASS_TEXTURE_MEMBER(StructName, Texture2D, Texture) \
	INPUT_PASS_TEXTURE_MEMBER(StructName, SamplerState, Sampler) \
    INPUT_PASS_TEXTURE_MEMBER(StructName, uint, bInvertedAlpha) \
    INPUT_PASS_TEXTURE_MEMBER(StructName, uint, SourceEncoding)

INPUT_PASS_TEXTURE(Input);
INPUT_PASS_TEXTURE(Mask);

FScreenTransform SvPositionToInputTextureUV;
FScreenTransform SvPositionToMaskTextureUV;

uint MaskSourceChannel;
uint bInputIsPremultiplied;
uint bPremultipliedInDisplaySpace;
uint bInvert;

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 InputUV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);
    float2 MaskUV = ApplyScreenTransform(SvPosition.xy, SvPositionToMaskTextureUV);
    float4 InputColor = Input_Texture.SampleLevel(Input_Sampler, InputUV, 0);
	float4 MaskColor = Mask_Texture.SampleLevel(Mask_Sampler, MaskUV, 0);

    // Linearize source colors
    InputColor.rgb = Decode(InputColor.rgb, Input_SourceEncoding);
    MaskColor.rgb = Decode(MaskColor.rgb, Mask_SourceEncoding);

	// Normalize alpha as opacity
	InputColor.a = InvertAlpha(InputColor.a, Input_bInvertedAlpha);
	MaskColor.a = InvertAlpha(MaskColor.a, Mask_bInvertedAlpha);

	/** ============ Start Masking ============ */
	float AlphaMask = saturate(MaskColor[MaskSourceChannel]);
	AlphaMask = select(bInvert > 0, 1.0f - AlphaMask, AlphaMask);

	// Optionally unpremultiply, in display space when the input was premultiplied in display
	// (e.g. an Ultimatte fill where the upstream processor multiplied by the matte in sRGB).
	BRANCH
	if (bInputIsPremultiplied > 0)
	{
		BRANCH
		if (bPremultipliedInDisplaySpace > 0)
		{
			const float3 RgbDisp = LinearToSrgb(InputColor.rgb);
			const float3 StraightDisp = saturate(RgbDisp * SafeRcp(saturate(AlphaMask)));
			InputColor.rgb = sRGBToLinear(StraightDisp);
		}
		else
		{
			InputColor = Unpremultiply(InputColor);
		}
	}

	// Apply mask to alpha.
	InputColor.a *= AlphaMask;
	/** ============  End Masking  ============ */

	// Optionally re-premultiply (in linear, against the new alpha).
	InputColor = select(bInputIsPremultiplied > 0, Premultiply(InputColor), InputColor);

	// Optionally re-invert alpha
	InputColor.a = InvertAlpha(InputColor.a, Input_bInvertedAlpha);

	// Optionally re-encode
	InputColor.rgb = Encode(InputColor.rgb, Input_SourceEncoding);

	OutColor = InputColor;
}
