// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D InputTexture;
SamplerState InputSampler;
float ReflectionStrength;

void MainPS(
	noperspective float4 InUVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	float2 UV = float2(InUVAndScreenPos.x, 1.0f - InUVAndScreenPos.y);
	const float4 InputColor = InputTexture.Sample(InputSampler, UV);
	// Reflection blurring/filtering could be integrated here directly.

	OutColor = lerp(0.0f, InputColor, ReflectionStrength);
}
