// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ShadowFilteringCommon.ush"

Texture2D       SceneDepthTex;
SamplerState    SceneDepthSampler;

Texture2D       ShadowDepthTexture;
SamplerState    ShadowDepthTextureSampler;

float4          SceneDepthPositionScaleBias;
float4x4        SceneDepthScreenToWorld;

float4x4        ShadowMatrix;
float4          ShadowBufferSize;
float4          ShadowInvDeviceZToWorldZ;
float4          ShadowStrengthBias;
float3          LightWorldDirection;

uint            bIsPerspectiveProjection;

// Returns the screen position for projection matrix calculations depending on the type of projection the view is using
float2 GetScreenPositionForProj(in float2 ScreenPosition, in float SceneDepth)
{
    return select(bIsPerspectiveProjection == 0, ScreenPosition, ScreenPosition * SceneDepth);
}

// Calculate world position from scene depth
float4 CalcWorldFromSceneDepth(in float2 UV, in float Depth)
{
    float2 ScreenPosition = (UV - SceneDepthPositionScaleBias.wz) / SceneDepthPositionScaleBias.xy;
    float4 HomogeneousWorldPosition = mul(float4(GetScreenPositionForProj(ScreenPosition, Depth), Depth, 1), SceneDepthScreenToWorld);
    float3 WorldPosition = HomogeneousWorldPosition.xyz / HomogeneousWorldPosition.w;

    return float4(WorldPosition, 1.0);
}

// Convert device-z depth to scene depth
float ShadowDeviceZToSceneDepth(float DeviceZ)
{
    return DeviceZ * ShadowInvDeviceZToWorldZ[0] + ShadowInvDeviceZToWorldZ[1] + 1.0f / (DeviceZ * ShadowInvDeviceZToWorldZ[2] - ShadowInvDeviceZToWorldZ[3]);
}

// Percentage-close filtering shadow mapping
float ManualPCF(in float4 ShadowPosition, in float NoL)
{
	float SceneDepth = ShadowDeviceZToSceneDepth(ShadowPosition.z) - saturate(1 - NoL) * ShadowStrengthBias.y;

    FPCFSamplerSettings Settings;
    Settings.ShadowDepthTexture = ShadowDepthTexture;
    Settings.ShadowDepthTextureSampler = ShadowDepthTextureSampler;
    Settings.ShadowBufferSize = ShadowBufferSize;
    Settings.TransitionScale = 1.0; // TODO: Parametrize like FProjectedShadowInfo::ComputeTransitionSize
    Settings.SceneDepth = SceneDepth;
    Settings.bSubsurface = false;
    Settings.bTreatMaxDepthUnshadowed = false;
    Settings.DensityMulConstant = 0;
    Settings.ProjectionDepthBiasParameters = 0;
    float Shadow = Manual5x5PCF(ShadowPosition.xy, Settings);

    return Shadow;
}

void MainPS(
    noperspective float4 InUVAndScreenPos : TEXCOORD0,
    out float4 OutColor : SV_Target0)
{
    float2 UV = InUVAndScreenPos.xy;
    float SceneDepth = SceneDepthTex.SampleLevel(SceneDepthSampler, UV, 0).r;
    
    float4 WorldPosition = CalcWorldFromSceneDepth(UV, SceneDepth);
    float4 ShadowPosition = mul(WorldPosition, ShadowMatrix);
    float2 ShadowUV = ShadowPosition.xy;
    
    if(any(ShadowUV < 0.0) || any(ShadowUV > 1.0))
    {
        OutColor.rgb = 1.0;
    }
    else
    {
        float3 Ddx = ddx(WorldPosition);
        float3 Ddy = ddy(WorldPosition);
        float3 WorldNormal = normalize(cross(Ddx, Ddy));
        const float NoL = saturate(abs(dot(WorldNormal, LightWorldDirection)));

        float ShadowVisibility = ManualPCF(ShadowPosition, NoL);

		OutColor.rgb = lerp(1.0, ShadowVisibility, ShadowStrengthBias.x);
	}

    OutColor.a = 1.0;
}
