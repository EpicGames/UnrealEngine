// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

Texture2D Input_Texture;
SamplerState Input_Sampler;
FScreenTransform SvPositionToOutputViewportUV;
FScreenTransform OutputViewportUVToInputTextureUV;
float2 Pivot;
float2 ScaleUV;
float RotationRadians;
float2 Translation;
float AspectRatio;

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 OutputViewportUV = ApplyScreenTransform(SvPosition.xy, SvPositionToOutputViewportUV);

    // 2D affine transform: output UV -> input UV (sampling direction).
    // User-facing Y convention is Y-up (Translation.Y > 0 moves image up), but screen UV
    // is Y-down, so flip the Y components of Translation and Pivot for sampling.
    float2 UVPivot = float2(Pivot.x, 1.0 - Pivot.y);
    float2 UV = OutputViewportUV;
    UV -= float2(Translation.x, -Translation.y);
    UV -= UVPivot;

    // Aspect-correct rotation: enter isotropic space (equal physical distance per unit
    // in both axes), rotate, then leave isotropic space. Dividing by AR before rotation
    // ensures perpendicular vectors remain perpendicular in screen pixel space.
    UV.y /= AspectRatio;
    float c = cos(RotationRadians);
    float s = sin(RotationRadians);
    UV = float2(UV.x * c - UV.y * s,
                UV.x * s + UV.y * c);
    UV *= ScaleUV;
    UV.y *= AspectRatio;
    UV += UVPivot;

    float2 InputUV = ApplyScreenTransform(UV, OutputViewportUVToInputTextureUV);
    OutColor = Input_Texture.Sample(Input_Sampler, InputUV);
}
