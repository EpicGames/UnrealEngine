// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

#include "CompositeCommon.ush"

FScreenTransform SvPositionToInputTextureUV;
Texture2D Input_Texture;
SamplerState Input_Sampler;

uint PremultOp;
float Fade;
uint OverrideOutputAlpha;
uint bInvertedAlpha;

#define PremultOp_None 0u
#define PremultOp_Premultiply 1u
#define PremultOp_Unpremultiply 2u

#define OverrideOutput_None 0u
#define OverrideOutput_Zero 1u
#define OverrideOutput_One 2u

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 TextureUV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);
    float4 Color = Input_Texture.SampleLevel(Input_Sampler, TextureUV, 0);

	Color.a = InvertAlpha(Color.a, bInvertedAlpha);

	if(PremultOp == PremultOp_Premultiply)
	{
		Color.rgb *= Color.a;
		Color *= Fade;
	}
	else if(PremultOp == PremultOp_Unpremultiply)
	{
		Color.rgb *= SafeRcp(Color.a);
		Color.a *= Fade;
	}
	else
	{
		Color *= Fade;
	}

	if(OverrideOutputAlpha == OverrideOutput_Zero)
	{
		Color.a = 0.0f;
	}
	else if(OverrideOutputAlpha == OverrideOutput_One)
	{
		Color.a = 1.0f;
	}

	Color.a = InvertAlpha(Color.a, bInvertedAlpha);

	OutColor = Color;
}
