// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Composite : ModuleRules
{
	public Composite(ReadOnlyTargetRules Target) : base(Target)
	{
		// The Composite plugin is distributed with engine hot fixes and thus isn't tied to binary
		// compatibility between hotfixes by only using Public/ interface of the renderer, but also Internal/ ones.
		bTreatAsEngineModule = true;
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"CompositeCore",
				"OpenColorIO",
				"MediaAssets",
				"GameplayTags",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"RenderCore",
				"Renderer",
				"RHI",
				"CameraCalibrationCore",
				"MediaProfile",
				"MediaFrameworkUtilities",
				"ConcertSyncCore",
				"VPRoles",
				"MovieScene",
				"LevelSequence",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"LevelEditor",
					"Sequencer",
					"UnrealEd",
					"Slate",
					"SlateCore",
					"SourceControl",
					"GeometryCore",
					"ModelingComponentsEditorOnly",
					"CQTest",
				}
			);
		}

		// bUseUnity = false;
		// OptimizeCode = CodeOptimization.Never;
		// PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;
	}
}
