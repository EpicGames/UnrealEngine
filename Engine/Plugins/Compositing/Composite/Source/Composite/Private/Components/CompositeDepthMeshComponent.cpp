// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CompositeDepthMeshComponent.h"

#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetUtils/CreateStaticMeshUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Misc/TransactionObjectEvent.h"
#endif

#define LOCTEXT_NAMESPACE "CompositingUtilities"

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			static FLazyName CompositeDepthTextureName(TEXT("CompositeDepthTexture"));
			static FLazyName CompositeScaleFactorName(TEXT("ScaleFactor"));

			static const TCHAR* DefaultDepthMeshAssetPath = TEXT("/Composite/Meshes/SM_Grid_960_540.SM_Grid_960_540");

#if WITH_EDITOR
			FString GetAssetPath(UCompositeDepthMeshComponent& InCompositeDepthMesh)
			{
				FString Res_X = FString::FromInt(InCompositeDepthMesh.GridResolution.X);
				FString Res_Y = FString::FromInt(InCompositeDepthMesh.GridResolution.Y);
				FString AssetPath = FString::Printf(TEXT("/Game/_Composure/Generated/SM_Grid_%s_%s"), *Res_X, *Res_Y);

				return AssetPath;
			}

			UStaticMesh* CreateStaticMeshAsset(UE::Geometry::FDynamicMesh3* Mesh, const FString& AssetPath)
			{
				UStaticMesh* NewStaticMesh = nullptr;

				UE::AssetUtils::FStaticMeshAssetOptions AssetOptions;

				AssetOptions.NewAssetPath = AssetPath;
				AssetOptions.NumSourceModels = 1;

				AssetOptions.bEnableRecomputeNormals = false;
				AssetOptions.bEnableRecomputeTangents = false;
				AssetOptions.bGenerateNaniteEnabledMesh = false;
				AssetOptions.NaniteSettings.FallbackPercentTriangles = 1.0;

				AssetOptions.bCreatePhysicsBody = true;
				AssetOptions.CollisionType = ECollisionTraceFlag::CTF_UseComplexAsSimple;

				AssetOptions.SourceMeshes.DynamicMeshes.Add(Mesh);

				UE::AssetUtils::FStaticMeshResults ResultData;
				UE::AssetUtils::ECreateStaticMeshResult AssetResult = UE::AssetUtils::CreateStaticMeshAsset(AssetOptions, ResultData);

				if (AssetResult == UE::AssetUtils::ECreateStaticMeshResult::Ok)
				{
					NewStaticMesh = ResultData.StaticMesh;
					NewStaticMesh->MarkPackageDirty();
					FAssetRegistryModule::AssetCreated(NewStaticMesh);
				}

				return NewStaticMesh;
			}
#endif
		}
	}
}

UCompositeDepthMeshComponent::UCompositeDepthMeshComponent(const FObjectInitializer& ObjectInitializer)
	: UStaticMeshComponent(ObjectInitializer)
	, GridResolution{1920/2, 1080/2}
	, DepthTexture{}
{
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultDepthMaterial;
		FConstructorStatics()
			: DefaultDepthMaterial(TEXT("/Composite/Materials/M_CompositeDepthMesh_Lit_Masked.M_CompositeDepthMesh_Lit_Masked"))
		{
		}
	};

	static FConstructorStatics ConstructorStatics;
	if (ensure(ConstructorStatics.DefaultDepthMaterial.Object != nullptr))
	{
		SetMaterial(0, ConstructorStatics.DefaultDepthMaterial.Object);
	}

	SetEnableGravity(false);
	SetGenerateOverlapEvents(false);
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SetCastShadow(false);
	SetEvaluateWorldPositionOffset(true);
}

void UCompositeDepthMeshComponent::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		const bool bLegacyPreCDOFix = GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID)
			< FUE5ReleaseStreamObjectVersion::CompositeDepthMeshDefaultRemovedFromCDO;

		if (bLegacyPreCDOFix && GetStaticMesh() == nullptr)
		{
			UStaticMesh* DefaultMesh = LoadObject<UStaticMesh>(nullptr, UE::Composite::Private::DefaultDepthMeshAssetPath);
			if (ensureMsgf(DefaultMesh, TEXT("Failed to load default composite depth mesh at %s"), UE::Composite::Private::DefaultDepthMeshAssetPath))
			{
				SetStaticMesh(DefaultMesh);
			}
		}
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdateMaterial();
	}
}

void UCompositeDepthMeshComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (IsTemplate())
	{
		return;
	}

	if (GetStaticMesh() != nullptr)
	{
		// Respect an explicit override from a Blueprint subclass or external caller.
		return;
	}

	UStaticMesh* DefaultMesh = LoadObject<UStaticMesh>(nullptr, UE::Composite::Private::DefaultDepthMeshAssetPath);
	if (ensureMsgf(DefaultMesh, TEXT("Failed to load default composite depth mesh at %s"), UE::Composite::Private::DefaultDepthMeshAssetPath))
	{
		SetStaticMesh(DefaultMesh);
	}
}

void UCompositeDepthMeshComponent::OnRegister()
{
	Super::OnRegister();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdateMaterial();
	}
}

void UCompositeDepthMeshComponent::OnUnregister()
{
	Super::OnUnregister();
}

FBoxSphereBounds UCompositeDepthMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const FVector Origin = LocalToWorld.TransformPosition(FVector::ZeroVector);
	return FBoxSphereBounds(FBox::BuildAABB(Origin, FVector::One() * UE_LARGE_HALF_WORLD_MAX));
}

void UCompositeDepthMeshComponent::UpdateMaterial()
{
	UMaterialInterface* MaterialInterface = GetMaterial(0);
	if (!IsValid(MaterialInterface))
	{
		return;
	}

	TArray<FMaterialParameterInfo> ParameterInfo;
	TArray<FGuid> ParameterIds;
	MaterialInterface->GetAllTextureParameterInfo(ParameterInfo, ParameterIds);

	// Avoid conversion to MID if parameter doesn't exist
	const bool bHasTextureParameter = ParameterInfo.ContainsByPredicate([](const FMaterialParameterInfo& ParamInfo)
		{
			return ParamInfo.Name.IsEqual(UE::Composite::Private::CompositeDepthTextureName);
		}
	);

	if (bHasTextureParameter)
	{
		UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(MaterialInterface);
		if (!MID)
		{
			MID = CreateAndSetMaterialInstanceDynamic(0);
		}

		if (ensureMsgf(MID, TEXT("Expected a material instance dynamic")))
		{
			MID->SetTextureParameterValue(UE::Composite::Private::CompositeDepthTextureName, DepthTexture);
			MID->SetScalarParameterValue(UE::Composite::Private::CompositeScaleFactorName, ScaleFactor);
		}
	}
}

#if WITH_EDITOR
void UCompositeDepthMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	using namespace UE::Composite::Private;

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, GridResolution))
	{
		GenerateGridMesh_Editor();
	}

	// To be safe, we update the material on any mesh change
	UpdateMaterial();
}

void UCompositeDepthMeshComponent::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);

	if (InTransactionEvent.HasPropertyChanges())
	{
		UpdateMaterial();
	}
}

void UCompositeDepthMeshComponent::GenerateGridMesh_Editor()
{
	// Do we already have this mesh?
	FString AssetPath = UE::Composite::Private::GetAssetPath(*this);
	UStaticMesh* NewStaticMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *AssetPath));

	if (!IsValid(NewStaticMesh))
	{
		UE::Geometry::FRectangleMeshGenerator RectGen;
		RectGen.Width = 1;
		RectGen.Height = 1;
		RectGen.WidthVertexCount = GridResolution.X + 1;
		RectGen.HeightVertexCount = GridResolution.Y + 1;
		RectGen.bSinglePolyGroup = true;
		UE::Geometry::FDynamicMesh3 GridMesh(&RectGen.Generate());

		NewStaticMesh = UE::Composite::Private::CreateStaticMeshAsset(&GridMesh, AssetPath);
	}

	SetStaticMesh(NewStaticMesh);
}
#endif

#undef LOCTEXT_NAMESPACE
