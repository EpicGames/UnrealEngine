// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CompositeMeshComponent.h"

#include "Async/Async.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"

#define LOCTEXT_NAMESPACE "Composite"

UCompositeMeshComponent::UCompositeMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UStaticMesh> DefaultMesh;
		ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultLitMaskedMaterial;
		ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultUnlitAlphaCompositeMaterial;
		FConstructorStatics()
			: DefaultMesh(TEXT("/Composite/Meshes/SM_DefaultCompositeMesh.SM_DefaultCompositeMesh"))
			, DefaultLitMaskedMaterial(TEXT("/Composite/Materials/M_CompositeMesh_Lit_Masked.M_CompositeMesh_Lit_Masked"))
			, DefaultUnlitAlphaCompositeMaterial(TEXT("/Composite/Materials/M_CompositeMesh_Unlit_AlphaComposite.M_CompositeMesh_Unlit_AlphaComposite"))
		{
		}
	};

	static FConstructorStatics ConstructorStatics;
	if (ensure(ConstructorStatics.DefaultMesh.Object != nullptr))
	{
		SetStaticMesh(ConstructorStatics.DefaultMesh.Object);
		SetCastShadow(false);
	}

	if (ensure(ConstructorStatics.DefaultLitMaskedMaterial.Object))
	{
		DefaultLitMaskedMaterial = ConstructorStatics.DefaultLitMaskedMaterial.Object;
	}

	if (ensure(ConstructorStatics.DefaultUnlitAlphaCompositeMaterial.Object))
	{
		DefaultUnlitAlphaCompositeMaterial = ConstructorStatics.DefaultUnlitAlphaCompositeMaterial.Object;
	}
}

void UCompositeMeshComponent::PostInitProperties()
{
	Super::PostInitProperties();
}

void UCompositeMeshComponent::OnRegister()
{
	Super::OnRegister();
}

void UCompositeMeshComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
}

void UCompositeMeshComponent::PostEditImport()
{
	Super::PostEditImport();
}

void UCompositeMeshComponent::PostLoad()
{
	Super::PostLoad();
}

void UCompositeMeshComponent::OnUnregister()
{
	Super::OnUnregister();
}

void UCompositeMeshComponent::SetMaterialType(ECompositeMeshMaterialType InMaterialType)
{
	MaterialType = InMaterialType;

	switch (MaterialType)
	{
	case ECompositeMeshMaterialType::DefaultLitMasked:
		for (int32 ElementIndex = 0; ElementIndex < GetMaterials().Num(); ++ElementIndex)
		{
			SetMaterial(ElementIndex, DefaultLitMaskedMaterial);
		}
		break;
	case ECompositeMeshMaterialType::DefaultUnlitAlphaComposite:
		for (int32 ElementIndex = 0; ElementIndex < GetMaterials().Num(); ++ElementIndex)
		{
			SetMaterial(ElementIndex, DefaultUnlitAlphaCompositeMaterial);
		}
		break;
	case ECompositeMeshMaterialType::Custom:
	default:
		// no-op
		break;
	}
}

void UCompositeMeshComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	TWeakObjectPtr<UCompositeMeshComponent> WeakThis = this;

	// Catch any material change here, and keep our material type enum up-to-date.
	AsyncTask(ENamedThreads::GameThread, [WeakThis = MoveTemp(WeakThis)]()
		{
			TStrongObjectPtr<UCompositeMeshComponent> CompositeMesh = WeakThis.Pin();

			if (CompositeMesh.IsValid())
			{
				UMaterialInterface* CurrentMaterial = CompositeMesh->GetMaterial(0);
				if (IsValid(CurrentMaterial))
				{
					UMaterial* ParentMaterial = CurrentMaterial->GetBaseMaterial();
					if (ParentMaterial == CompositeMesh->DefaultLitMaskedMaterial)
					{
						CompositeMesh->MaterialType = ECompositeMeshMaterialType::DefaultLitMasked;
					}
					else if (ParentMaterial == CompositeMesh->DefaultUnlitAlphaCompositeMaterial)
					{
						CompositeMesh->MaterialType = ECompositeMeshMaterialType::DefaultUnlitAlphaComposite;
					}
					else
					{
						CompositeMesh->MaterialType = ECompositeMeshMaterialType::Custom;
					}
				}
			}
		});
}

#if WITH_EDITOR
void UCompositeMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, MaterialType))
	{
		SetMaterialType(MaterialType);
	}
}
#endif

#undef LOCTEXT_NAMESPACE
