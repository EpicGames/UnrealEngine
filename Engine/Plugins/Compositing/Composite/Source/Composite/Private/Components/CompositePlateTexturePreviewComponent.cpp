// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePlateTexturePreviewComponent.h"

#include "CompositeActor.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/Texture.h"
#include "Layers/CompositeLayerPlate.h"
#include "Passes/CompositePassColorKeyer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SLevelViewport.h"
#endif

#if WITH_EDITOR
bool UCompositePlateTexturePreviewComponent::GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut)
{
	if (TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = PlateToPreview.Pin())
	{
		if (UTexture* Texture = PinnedPlate->Texture)
		{
			// Editor preview widget uses ViewOut.AspectRatio to determine the size to make the preview widget, so set the apsect ratio
			// to match the texture being previewed if there is one
			ViewOut.bConstrainAspectRatio = true;
			ViewOut.AspectRatio = float(Texture->GetSurfaceWidth()) / float(Texture->GetSurfaceHeight());
		}
	}

	return true;
}

TSharedPtr<SWidget> UCompositePlateTexturePreviewComponent::GetCustomEditorPreviewWidget()
{
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = PlateToPreview.Pin();
	if (!PinnedPlate.IsValid())
	{
		return nullptr;
	}

	UTexture* Texture = PinnedPlate->Texture;
	if (!Texture)
	{
		return nullptr;
	}

	const int64 Width = Texture->GetSurfaceWidth();
	const int64 Height = Texture->GetSurfaceHeight();

	if (!PreviewBrush.IsValid())
	{
		PreviewBrush = MakeShared<FSlateImageBrush>(Texture, FVector2D(Width, Height), FLinearColor::White, ESlateBrushTileType::Both);
	}
	else
	{
		PreviewBrush->SetResourceObject(Texture);
		PreviewBrush->SetImageSize(FVector2D(Width, Height));
	}

	return SNew(SScaleBox)
		.Stretch(EStretch::ScaleToFit)
		[
			SNew(SImage)
			.Image(PreviewBrush.Get())
		];
}

bool UCompositePlateTexturePreviewComponent::ShowPreview(TObjectPtr<UCompositePassColorKeyer> InRequestingKeyer)
{
	Activate();
	
	UCompositeLayerPlate* Plate = InRequestingKeyer->GetTypedOuter<UCompositeLayerPlate>();
	if (!Plate)
	{
		return false;
	}

	AActor* PlateOwner = Plate->GetTypedOuter<AActor>();
	if (!PlateOwner || PlateOwner != GetOwner())
	{
		return false;
	}

	if (!Plate->Texture)
	{
		return false;
	}

	if (FLevelEditorViewportClient* ViewportClient = GetActiveViewportClient())
	{
		if (TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(ViewportClient->GetEditorViewportWidget()))
		{
			PlateToPreview = Plate;
			bShowPreview = true;

			if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
			{
				CachedCameraPreviewSize = ViewportSettings->CameraPreviewSize;
				ViewportSettings->CameraPreviewSize = InRequestingKeyer->PlateTexturePreviewSize;
			}
			
			LevelViewport->SetActorAlwaysPreview(GetOwner(), true);
			return true;
		}
	}
	
	return false;
}

void UCompositePlateTexturePreviewComponent::HidePreview()
{
	Deactivate();
	
	if (FLevelEditorViewportClient* ViewportClient = GetActiveViewportClient())
	{
		if (TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(ViewportClient->GetEditorViewportWidget()))
		{
			LevelViewport->SetActorAlwaysPreview(GetOwner(), false);
		}
	}
	
	PlateToPreview = nullptr;
	bShowPreview = false;

	if (CachedCameraPreviewSize.IsSet())
	{
		if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
		{
			ViewportSettings->CameraPreviewSize = CachedCameraPreviewSize.GetValue();
		}

		CachedCameraPreviewSize.Reset();
	}
}

FLevelEditorViewportClient* UCompositePlateTexturePreviewComponent::GetActiveViewportClient()
{
	if (const FViewport* ActiveViewport = GEditor ? GEditor->GetActiveViewport() : nullptr)
	{
		for (FLevelEditorViewportClient* LevelViewportClient : GEditor->GetLevelViewportClients())
		{
			if (LevelViewportClient->Viewport == ActiveViewport)
			{
				return LevelViewportClient;
			}
		}
	}
	return nullptr;
}
#endif