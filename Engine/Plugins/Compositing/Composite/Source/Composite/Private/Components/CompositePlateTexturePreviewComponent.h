// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "CompositePlateTexturePreviewComponent.generated.h"

class FLevelEditorViewportClient;
class UCompositeLayerPlate;
class UCompositePassColorKeyer;

/**
 * A component that is used to display the plate texture of a composite actor's plate layer in the level viewport's 'picture-in-picture' preview widget.
 * Used by keyer passes to allow color picking from raw plate textures instead of from potentially post-processed textures on the composite actor's meshes
 */
UCLASS()
class UCompositePlateTexturePreviewComponent : public UActorComponent
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	
	// UActorComponent interface
	virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
	virtual TSharedPtr<SWidget> GetCustomEditorPreviewWidget() override;
	// End of UActorComponent interface

	/**
	 * Displays the plate texture preview in the level editor's 'picture-in-picture'
	 * @param InRequestingKeyer The keyer pass on the plate layer that is requesting the preview be displayed
	 * @return true if the preview was started
	 */
	bool ShowPreview(TObjectPtr<UCompositePassColorKeyer> InRequestingKeyer);

	/** Hides the plate texture preview */
	void HidePreview();

	/** Gets whether the plate texture preview is currently being displayed */
	bool IsPreviewActive() const { return bShowPreview; }

private:
	/** Gets the active level editor viewport client from the level editor */
	FLevelEditorViewportClient* GetActiveViewportClient();
#endif
	
private:
	/** Indicates if the preview should be shown */
	bool bShowPreview = false;

	/** Cached camera preview size from the editor preferences config, restored when the plate preview is hidden */
	TOptional<float> CachedCameraPreviewSize = TOptional<float>();

	/** A reference to the plate whose texture is being previewed */
	TWeakObjectPtr<UCompositeLayerPlate> PlateToPreview = nullptr;
	
	/** The slate brush used to display the preview texture */
	TSharedPtr<FSlateBrush> PreviewBrush = nullptr;
};
