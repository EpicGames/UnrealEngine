// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CompositeSceneCapture2DComponent.h"

#include "Camera/CameraComponent.h"
#include "CompositeActor.h"
#include "CompositeRenderTargetPool.h"
#include "Engine/TextureRenderTarget2D.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "Composite"

UCompositeSceneCapture2DComponent::UCompositeSceneCapture2DComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;

	// We issue scene capture render updates manually in FCompositeCoreSceneViewExtension
	bCaptureEveryFrame = false;
	bAlwaysPersistRenderingState = true;
	bCaptureOnMovement = false;

	CaptureSource = ESceneCaptureSource::SCS_FinalColorHDR;
	bUseRayTracingIfEnabled = true;
	bMainViewFamily = true;
	bMainViewResolution = false;
	bMainViewCamera = true;
	// Inherit main view camera post-process settings to match main view DoF for example.
	bInheritMainViewCameraPostProcessSettings = true;
	bIgnoreScreenPercentage = true;
	bConsiderUnrenderedOpaquePixelAsFullyTranslucent = true;
	UnlitViewmode = ESceneCaptureUnlitViewmode::CaptureOrCustomRenderPass;

	PostProcessSettings.bOverride_ReflectionMethod = true;
	PostProcessSettings.ReflectionMethod = EReflectionMethod::Lumen;
	PostProcessSettings.bOverride_DynamicGlobalIlluminationMethod = true;
	PostProcessSettings.DynamicGlobalIlluminationMethod = EDynamicGlobalIlluminationMethod::Lumen;
	PostProcessSettings.bOverride_LumenFinalGatherScreenTraces = true;
	PostProcessSettings.LumenFinalGatherScreenTraces = false;
	PostProcessSettings.bOverride_LumenReflectionsScreenTraces = true;
	PostProcessSettings.LumenReflectionsScreenTraces = false;
	// While hit lighting is more expensive, it significantly minimizes artifacts & noise. Here we default with higher-quality settings.
	PostProcessSettings.bOverride_LumenRayLightingMode = true;
	PostProcessSettings.LumenRayLightingMode = ELumenRayLightingModeOverride::HitLighting;

	ShowFlags.SetDeferredAtmospherePass(false);
	ShowFlags.SetCloud(false);
	ShowFlags.SetAllowPrimitiveAlphaHoldout(false);
	ShowFlags.SetBloom(false);
	ShowFlags.SetEyeAdaptation(false);
	ShowFlags.SetLocalExposure(false);
	ShowFlags.SetPostProcessMaterial(false);
	/**
	 * This can cause temporal ghosting artifacts on moving shadows, but remains preferable over the otherwise default shadow noise.
	 * Automtically ignored with custom render passes, where post-processing does not apply.
	 */
	ShowFlags.SetTemporalAA(true);
}

void UCompositeSceneCapture2DComponent::BeginDestroy()
{
	FCompositeRenderTargetPool::Get().ReleaseTarget(TextureTarget);

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UCompositeSceneCapture2DComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

const AActor* UCompositeSceneCapture2DComponent::GetViewOwner() const
{
	// View owner that matches the main view camera
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (IsValid(CompositeActor))
	{
		return CompositeActor->GetCameraActor().Get();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
