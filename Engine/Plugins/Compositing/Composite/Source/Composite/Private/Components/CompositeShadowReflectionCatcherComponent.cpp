// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CompositeShadowReflectionCatcherComponent.h"

#include "CompositeRenderTargetPool.h"
#include "Engine/TextureRenderTarget2D.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "Composite"

UCompositeShadowReflectionCatcherComponent::UCompositeShadowReflectionCatcherComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_RenderScenePrimitives;

	ShowFlags.SetAtmosphere(false);
	ShowFlags.SetGrain(false);
	ShowFlags.SetScreenSpaceReflections(false);
}

#undef LOCTEXT_NAMESPACE
