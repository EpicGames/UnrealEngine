// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CompositeViewProjectionComponent.h"
#include "CompositeActor.h"

#include "Camera/CameraComponent.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "CompositeCoreSubsystem.h"
#include "Kismet/GameplayStatics.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#define LOCTEXT_NAMESPACE "Composite"

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			// Replicated code from FViewMatrices::ScreenToClipProjectionMatrix()
			inline FMatrix ScreenToClipProjectionMatrix(const FMatrix& InProjectionMatrix)
			{
				//Screen to clip matrix should not utilise scene depth for the w component in ortho projections, but is needed for perspective
				const bool bIsPerspectiveProjection = InProjectionMatrix.M[3][3] < 1.0f;
				
				if (bIsPerspectiveProjection)
				{
					return FMatrix(
						FPlane(1, 0, 0, 0),
						FPlane(0, 1, 0, 0),
						FPlane(0, 0, InProjectionMatrix.M[2][2], 1.0f),
						FPlane(0, 0, InProjectionMatrix.M[3][2], 0.0f));
				}
				else
				{
					return FMatrix(
						FPlane(1, 0, 0, 0),
						FPlane(0, 1, 0, 0),
						FPlane(0, 0, InProjectionMatrix.M[2][2], 0.0f),
						FPlane(0, 0, InProjectionMatrix.M[3][2], 1.0f));
				}
			}

			// Replicated code from SceneView.h
			const FMatrix InvertProjectionMatrix(const FMatrix& M)
			{
				if (M.M[1][0] == 0.0f &&
					M.M[3][0] == 0.0f &&
					M.M[0][1] == 0.0f &&
					M.M[3][1] == 0.0f &&
					M.M[0][2] == 0.0f &&
					M.M[1][2] == 0.0f &&
					M.M[0][3] == 0.0f &&
					M.M[1][3] == 0.0f &&
					M.M[2][3] == 1.0f &&
					M.M[3][3] == 0.0f)
				{
					// Solve the common case directly with very high precision.
					/*
					M =
					| a | 0 | 0 | 0 |
					| 0 | b | 0 | 0 |
					| s | t | c | 1 |
					| 0 | 0 | d | 0 |
					*/

					double a = M.M[0][0];
					double b = M.M[1][1];
					double c = M.M[2][2];
					double d = M.M[3][2];
					double s = M.M[2][0];
					double t = M.M[2][1];

					return FMatrix(
						FPlane(1.0 / a, 0.0f, 0.0f, 0.0f),
						FPlane(0.0f, 1.0 / b, 0.0f, 0.0f),
						FPlane(0.0f, 0.0f, 0.0f, 1.0 / d),
						FPlane(-s / a, -t / b, 1.0f, -c / d)
					);
				}
				else
				{
					return M.Inverse();
				}
			}

			bool UpdateMatrixParameters(
				UWorld* InWorld,
				UMaterialParameterCollectionInstance* InMaterialParameterCollectionInstance,
				const FName& MatrixParameterName,
				const FMatrix44f& InMatrix)
			{
				if (!IsValid(InWorld) || !IsValid(InMaterialParameterCollectionInstance))
				{
					return false;
				}

				const UMaterialParameterCollection* MPC = InMaterialParameterCollectionInstance->GetCollection();

				int32 ParameterProjection = INDEX_NONE;
				for (int32 ParameterIndex = 0; ParameterIndex < MPC->VectorParameters.Num(); ParameterIndex++)
				{
					FName CurrentParameterName = MPC->VectorParameters[ParameterIndex].ParameterName;
					if (!CurrentParameterName.IsNone())
					{
						if (CurrentParameterName == MatrixParameterName)
						{
							ParameterProjection = ParameterIndex;
						}
					}
				}

				// Ensure there's space for 4 output vectors for the projection matrix
				if (ParameterProjection != INDEX_NONE && ParameterProjection + 4 <= MPC->VectorParameters.Num())
				{
					// Store the vectors to the collection instance
					const FLinearColor* MatrixVectors = (const FLinearColor*)&InMatrix;
					for (int32 ElementIndex = 0; ElementIndex < 4; ElementIndex++)
					{
						InMaterialParameterCollectionInstance->SetVectorParameterValue(MPC->VectorParameters[ParameterProjection + ElementIndex].ParameterName, MatrixVectors[ElementIndex]);
					}

					return true;
				}

				return false;
			}
		}
	}
}

UCompositeViewProjectionComponent::UCompositeViewProjectionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static ConstructorHelpers::FObjectFinder<UMaterialParameterCollection> DefaultMPC = TEXT("/Composite/Materials/MPC_Composite.MPC_Composite");

	MaterialParameterCollection = DefaultMPC.Object;
	ViewProjectionMatrixParameter = TEXT("CameraViewProjectionMatrix");
	ClipToWorldMatrixParameter = TEXT("CameraClipToWorldMatrix");
	ScreenToClipMatrixParameter = TEXT("CameraScreenToClipMatrix");

	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = true;
	bTickInEditor = true;
	bAutoActivate = true;
}

void UCompositeViewProjectionComponent::ForceUpdate()
{
	LastViewProjectionMatrix = FMatrix::Identity;
}

UCameraComponent* UCompositeViewProjectionComponent::GetCameraComponent() const
{
	if (CameraActor.IsValid())
	{
		return CameraActor->GetComponentByClass<UCameraComponent>();
	}

	return nullptr;
}

void UCompositeViewProjectionComponent::SetCameraComponent(UCameraComponent* InComponent)
{
	CameraActor = IsValid(InComponent) ? InComponent->GetOwner() : nullptr;

	ForceUpdate();
}

const TSoftObjectPtr<AActor>& UCompositeViewProjectionComponent::GetCameraActor()
{
	return CameraActor;
}

void UCompositeViewProjectionComponent::SetCameraActor(const TSoftObjectPtr<AActor>& InActor)
{
#if WITH_EDITOR
	Modify();
#endif
	CameraActor = InActor;

	ForceUpdate();
}

void UCompositeViewProjectionComponent::InitDefaultCamera()
{
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	
	if (IsValid(CompositeActor))
	{
		// Special case when the component is owned by the composite actor: we use its camera reference directly.
		SetCameraActor(CompositeActor->GetCameraActor());
	}
	else
	{
		SetCameraActor(GetOwner());
	}
}

void UCompositeViewProjectionComponent::PostInitProperties()
{
	Super::PostInitProperties();
}

void UCompositeViewProjectionComponent::OnRegister()
{
	Super::OnRegister();

	InitDefaultCamera();

	/*
	* NOTE: We must (unfortunately) rely on this after-tick event to update the view matrix projection,
	* since viewport camera transform updates after ticking all world objects.
	*
	* The CompositePlane plugin updated this information on tick, but a lag becomes visible when
	* piloting the source projection camera for example.
	*/
	OnWorldPreSendAllEndOfFrameUpdatesHandle = FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.AddUObject(this, &UCompositeViewProjectionComponent::UpdateProjection);
}

void UCompositeViewProjectionComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	InitDefaultCamera();
}

void UCompositeViewProjectionComponent::PostEditImport()
{
	Super::PostEditImport();

	InitDefaultCamera();
}

void UCompositeViewProjectionComponent::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::CompositeCameraReferenceRefactor)
	{
		const UCameraComponent* CameraComponent = Cast<UCameraComponent>(TargetCameraComponent_DEPRECATED.GetComponent(TargetCameraComponent_DEPRECATED.OtherActor.Get()));

		if (IsValid(CameraComponent))
		{
			CameraActor = CameraComponent->GetOwner();
		}

		TargetCameraComponent_DEPRECATED.OverrideComponent.Reset();
		TargetCameraComponent_DEPRECATED.OtherActor.Reset();
	}
#endif // WITH_EDITORONLY_DATA
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UCompositeViewProjectionComponent::OnUnregister()
{
	FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.Remove(OnWorldPreSendAllEndOfFrameUpdatesHandle);

	Super::OnUnregister();
}

void UCompositeViewProjectionComponent::UpdateProjection(UWorld* InWorld) const
{
	using namespace UE::Composite::Private;

	if (!IsValid(InWorld))
	{
		return;
	}

	if (!bIsEnabled)
	{
		return;
	}

	const ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (IsValid(CompositeActor) && !CompositeActor->IsRendering())
	{
		// Since the component can be used without the composite actor, we only stop updates when it is valid but not rendering.
		return;
	}

	UMaterialParameterCollection* MPC = MaterialParameterCollection.Get();
	UCameraComponent* CameraComponent = GetCameraComponent();

	if (IsValid(CameraComponent) && IsValid(MPC))
	{
		FMinimalViewInfo DesiredView;
		FMatrix ViewMatrix, ProjectionMatrix, ViewProjectionMatrix;

		CameraComponent->GetCameraView(FApp::GetDeltaTime(), DesiredView);

		UGameplayStatics::GetViewProjectionMatrix(DesiredView, ViewMatrix, ProjectionMatrix, ViewProjectionMatrix);

		if (!LastViewProjectionMatrix.Equals(ViewProjectionMatrix))
		{
			bool bUpdateMPC = false;
			UMaterialParameterCollectionInstance* InstanceMPC = InWorld->GetParameterCollectionInstance(MPC);

			bUpdateMPC |= UpdateMatrixParameters(InWorld, InstanceMPC, ViewProjectionMatrixParameter, FMatrix44f(ViewProjectionMatrix));

			const FMatrix44f ClipToWorldMatrix{ InvertProjectionMatrix(ProjectionMatrix) * ViewMatrix.Inverse() };
			bUpdateMPC |= UpdateMatrixParameters(InWorld, InstanceMPC, ClipToWorldMatrixParameter, ClipToWorldMatrix);

			const FMatrix44f ScreenToClipMatrix{ ScreenToClipProjectionMatrix(ProjectionMatrix) };
			bUpdateMPC |= UpdateMatrixParameters(InWorld, InstanceMPC, ScreenToClipMatrixParameter, ScreenToClipMatrix);

			if (bUpdateMPC)
			{
				InstanceMPC->UpdateRenderState(false);
			}

			LastViewProjectionMatrix = ViewProjectionMatrix;
		}
	}
}

void UCompositeViewProjectionComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	MarkForNeededEndOfFrameUpdate();
}

#if WITH_EDITOR
void UCompositeViewProjectionComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Force MPC updates on any property change.
	ForceUpdate();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

#undef LOCTEXT_NAMESPACE
