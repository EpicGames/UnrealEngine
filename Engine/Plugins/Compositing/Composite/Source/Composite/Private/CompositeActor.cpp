// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeActor.h"

#include "CompositeWorldSubsystem.h"
#include "CompositeModule.h"
#include "CompositeCoreSettings.h"
#include "CompositeRenderTargetPool.h"

#include "Components/CompositeShadowReflectionCatcherComponent.h"
#include "Components/CompositeViewProjectionComponent.h"
#include "Layers/CompositeLayerShadowReflection.h"
#include "Layers/CompositeLayerPlate.h"
#include "Layers/CompositeLayerMainRender.h"
#include "VPRolesSubsystem.h"

#include "Algo/Rotate.h"
#include "Components/PrimitiveComponent.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/Texture.h"
#include "Kismet/GameplayStatics.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "SceneView.h"
#include "Components/CompositePlateTexturePreviewComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Misc/TransactionObjectEvent.h"
#include "UnrealClient.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeActor)

DECLARE_CYCLE_STAT(TEXT("Composite - Actor Tick"), STAT_Composite_Actor_Tick, STATGROUP_Composite);
DECLARE_CYCLE_STAT(TEXT("Composite - Actor EndOfFrame Update"), STAT_Composite_Actor_EndOfFrameUpdate, STATGROUP_Composite);

namespace UE
{
	namespace Composite
	{
		/** Tag that will be added on every component generated by the composite actor. */
		const FLazyName DefaultCompositeTag = TEXT("Composite Generated Component");

		namespace Private
		{
			/** Convert the allowed view mode setting into a set of view mode indices. */
			inline TSet<EViewModeIndex> GetViewModeIndices(ECompositeAllowedViewModes InMode)
			{
				switch (InMode)
				{
				case ECompositeAllowedViewModes::Default:
					return { VMI_Lit, VMI_PathTracing, VMI_Unknown };
				case ECompositeAllowedViewModes::MediaProfileUnknown:
					return { VMI_Unknown };
				default:
					break;
				}

				return {};
			}

			/** Resolve the per-frame active-viewport filter from the actor's policy. */
			inline TOptional<UE::CompositeCore::FActiveViewportFilter> ResolveActiveViewportFilter(const ACompositeActor& InActor)
			{
				if (!InActor.bRestrictToActiveViewport)
				{
					return {};
				}

				UE::CompositeCore::FActiveViewportFilter Filter;

				// View.ViewActor matches piloted cameras and the PIE/game primary view.
				Filter.TargetViewActor = InActor.GetCameraActor().Get();

#if WITH_EDITOR
				// Active level viewport, matched by render-target pointer.
				if (GEditor != nullptr)
				{
					Filter.TargetRenderTarget = GEditor->GetActiveViewport();
				}
#endif

				return Filter;
			}

			/** Return solo layer from list. */
			TObjectPtr<UCompositeLayerBase> FindSoloLayer(TArrayView<const TObjectPtr<UCompositeLayerBase>> InLayers)
			{
				TObjectPtr<UCompositeLayerBase> SoloLayer;

				// Iterate backwards so that lower passes in the UI are executed first
				for (const TObjectPtr<UCompositeLayerBase>& Layer : InLayers)
				{
					if (!IsValid(Layer) || !Layer->GetIsEnabled())
					{
						continue;
					}

					if (Layer->bIsSolo)
					{
						SoloLayer = Layer;
					}
				}

				return SoloLayer;
			}

			/**
			* Convenience function to register layers in the traversal context, while returning proxies for each layer input.
			* 
			* Returns an index indicating the first proxy whose output can no longer be shared between post-processing locations.
			*/
			int32 RegisterLayers(FCompositeTraversalContext& Context, TArrayView<const TObjectPtr<UCompositeLayerBase>> InLayers, FSceneRenderingBulkObjectAllocator& InFrameAllocator, TArray<FCompositeCorePassProxy*>& OutProxies)
			{
				int32 LastLayerIndex = INDEX_NONE;
				int32 BranchLayerIndex = INDEX_NONE;
				OutProxies.Reserve(InLayers.Num());

				for (const TObjectPtr<UCompositeLayerBase>& Layer : InLayers)
				{
					if (!IsValid(Layer) || !Layer->GetIsActive())
					{
						continue;
					}

					UE::CompositeCore::FPassInputDecl LayerInputDecl;
					FCompositeCorePassProxy* OutProxy = Layer->GetProxy(LayerInputDecl, Context, InFrameAllocator);
					if (OutProxy != nullptr)
					{
						OutProxies.Add(OutProxy);
						Context.bIsFirstPass = false;

						const int32 ProxyIndex = OutProxies.Num() - 1;
						// If the pass needed scene textures specific to each post-processing location, its output
						// cannot be recyled in multiple locations. We therefore mark it as the "branch" layer.
						if (Context.bNeedsSceneTextures)
						{
							if (BranchLayerIndex == INDEX_NONE)
							{
								BranchLayerIndex = ProxyIndex;
							}
						}

						// Since merge passes handle color encodings & lens distortion, and
						// since SSR Input & tonemap differ in both regards, we must at least
						// run one merge pass for each location currently.
						LastLayerIndex = ProxyIndex;
					}
				}

				return (BranchLayerIndex != INDEX_NONE) ? BranchLayerIndex : LastLayerIndex;
			}

			/* Gather render work (per frame) given the current composite actor properties. */
			UE::CompositeCore::FRenderWork GatherFrameRenderWork(
				const ACompositeActor& InCompositeActor,
				const TMap<TWeakObjectPtr<const UCompositePassBase>, FSceneCaptureComponentArray>& InSceneCapturesPerPass)
			{
				if (InCompositeActor.GetCompositeLayers().IsEmpty())
				{
					return {};
				}

				UE::CompositeCore::FRenderWork RenderWork;
				TArray<TObjectPtr<UCompositeLayerBase>> Layers;

				TObjectPtr<UCompositeLayerBase> SoloLayer = FindSoloLayer(InCompositeActor.GetCompositeLayers());
				if (IsValid(SoloLayer))
				{
					Layers.Add(SoloLayer);
				}
				else
				{
					Layers = InCompositeActor.GetCompositeLayers();
					// TODO: Remove once we have UI customization displaying layers in the correct bottom-up order
					Algo::Reverse(Layers);
				}

				TArray<FCompositeCorePassProxy*> LayerProxies;
				FCompositeTraversalContext Context;
				Context.bIsSolo = (Layers.Num() == 1);

				const int32 BranchLayerIndex = RegisterLayers(Context, Layers, *RenderWork.FrameAllocator, LayerProxies);
				
				if (LayerProxies.IsEmpty())
				{
					// Early return when there is no actively rendering layer
					return {};
				}

				/**
				* Register pre-processing passes before SSR branch below to prevent duplicate work.
				* These passes only run once at the start of a frame (see ApplyPreprocessing call site in FCompositeCoreSceneViewExtension).
				*/
				RenderWork.PreprocessingPasses = Context.PreprocessingPasses;

				if (InCompositeActor.bEnableScreenSpaceReflections)
				{
					const int32 BranchSourceIndex = BranchLayerIndex - 1;

					TArray<FCompositeCorePassProxy*> BranchedLayerProxies;
					Context.bIsFirstPass = true;
					RegisterLayers(Context, Layers, *RenderWork.FrameAllocator, BranchedLayerProxies);

					// Trim all proxies before our branch index (for the tonemap location) to instead recycle previous work.
					if (BranchedLayerProxies.IsValidIndex(BranchLayerIndex))
					{
						BranchedLayerProxies.RemoveAt(0, BranchLayerIndex);
					}

					if (ensureMsgf(!BranchedLayerProxies.IsEmpty(), TEXT("Unexpected empty branch proxy array.")) && LayerProxies.IsValidIndex(BranchSourceIndex))
					{
						// This is where we connect to the previous SSR input work branch.
						// Input0 will be merged on top of Input1, the previous proxy's output.
						const int32 NumInputs = BranchedLayerProxies[0]->GetNumDeclaredInputs();
						BranchedLayerProxies[0]->SetDeclaredInput(NumInputs-1, UE::CompositeCore::MakePassInput(LayerProxies[BranchSourceIndex]));
					}

					RenderWork.FramePasses.Emplace(ISceneViewExtension::EPostProcessingPass::SSRInput, LayerProxies);
					RenderWork.FramePasses.Emplace(ISceneViewExtension::EPostProcessingPass::Tonemap, BranchedLayerProxies);
				}
				else
				{
					RenderWork.FramePasses.Emplace(ISceneViewExtension::EPostProcessingPass::Tonemap, LayerProxies);
				}

				RenderWork.ExternalInputs = Context.GetExternalTextures();

				switch (InCompositeActor.MainRenderOutput)
				{
				case ECompositeMainRenderOutputMode::FinalColorHDR:
					RenderWork.MainRenderMode = ESceneCaptureSource::SCS_FinalColorHDR;
					break;
				case ECompositeMainRenderOutputMode::FinalToneCurveHDR:
					RenderWork.MainRenderMode = ESceneCaptureSource::SCS_FinalToneCurveHDR;
					break;
				default:
					break;
				}
				
				RenderWork.AllowedViewModes = GetViewModeIndices(InCompositeActor.AllowedViewModes);
				RenderWork.ActiveViewportFilter = ResolveActiveViewportFilter(InCompositeActor);

				// Heuristic reservation; actual count per pass may vary.
				RenderWork.SceneCapturesUpdateQueue.Reserve(InSceneCapturesPerPass.Num() * 2);

				for (const auto& Pair : InSceneCapturesPerPass)
				{
					const TWeakObjectPtr<const UCompositePassBase>& Pass = Pair.Key;
					if (Pass.IsValid() && Pass->GetIsActive())
					{
						for (const TObjectPtr<UCompositeSceneCapture2DComponent>& SceneCapture : Pair.Value.Components)
						{
							RenderWork.SceneCapturesUpdateQueue.Add(SceneCapture.Get());
						}
					}
				}

				return RenderWork;
			}
		}
	}
}

ACompositeActor::ACompositeActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ViewProjectionComponent{}
	, bIsActive{true}
	, bIsEnabled{true}
	, bEnableScreenSpaceReflections{true}
	, bOverridesViewUserFlags{true}
	, ViewUserFlags{1}
	, MainRenderOutput{ECompositeMainRenderOutputMode::Default}
	, AllowedViewModes{ECompositeAllowedViewModes::Default}
	, bRestrictToActiveViewport{false}
	, bIsModifyingAProperty{false}
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bTickEvenWhenPaused = true;
	PrimaryActorTick.TickGroup = TG_PostUpdateWork;
	PrimaryActorTick.bStartWithTickEnabled = true;
	bAllowTickBeforeBeginPlay = true;

	SetActorEnableCollision(false);

	RootComponent = CreateDefaultSubobject<USceneComponent>(USceneComponent::GetDefaultSceneRootVariableName());

	ViewProjectionComponent = CreateDefaultSubobject<UCompositeViewProjectionComponent>(TEXT("CameraViewProjectionComponent"));
	AddInstanceComponent(ViewProjectionComponent);

#if WITH_EDITOR
	PlatePreviewComponent = CreateEditorOnlyDefaultSubobject<UCompositePlateTexturePreviewComponent>(TEXT("EditorPreviewComponent"));
#endif
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ACompositeActor::~ACompositeActor() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ACompositeActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	/**
	* Refactored scene capture management.
	* Instead of polluting the actor with old data structures, we remove old components and rely on their recreation.
	*/
	const bool bHandleSceneCaptureRefactor = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CompositeActorSceneCaptureRefactor;
	const bool bHandleDerivedSceneCaptureTypes = GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::CompositePluginDerivedSceneCaptures;

	if (bHandleSceneCaptureRefactor || bHandleDerivedSceneCaptureTypes)
	{
		TArray<UActorComponent*> ActorComponents;
		GetComponents(ActorComponents);

		for (auto It = ActorComponents.CreateIterator(); It; ++It)
		{
			if (USceneCaptureComponent2D* CaptureComponent = Cast<USceneCaptureComponent2D>(*It))
			{
				bool bDestroyComponent = false;

				if (bHandleSceneCaptureRefactor && CaptureComponent->IsA<UCompositeShadowReflectionCatcherComponent>())
				{
					bDestroyComponent = true;
				}

				if (bHandleDerivedSceneCaptureTypes && (CaptureComponent->GetClass() == USceneCaptureComponent2D::StaticClass()))
				{
					bDestroyComponent = true;
				}

				if (bDestroyComponent)
				{
					RemoveInstanceComponent(CaptureComponent);
					CaptureComponent->UnregisterComponent();
					CaptureComponent->DestroyComponent();
				}
			}
		}
	}

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::CompositeCameraReferenceRefactor)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const UCameraComponent* CameraComponent = Cast<UCameraComponent>(Camera_DEPRECATED.GetComponent(Camera_DEPRECATED.OtherActor.Get()));

		if (IsValid(CameraComponent))
		{
			CameraActor = CameraComponent->GetOwner();
		}

		Camera_DEPRECATED.OverrideComponent.Reset();
		Camera_DEPRECATED.OtherActor.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA

	if (VPRole.IsValid())
	{
		// Note that if multiple actors have the same role, this could shift which actor is active on load.
		SetIsActive(IsInVPRole());
	}
}

void ACompositeActor::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
	{
		Subsystem->RegisterActor(this);
	}
}

void ACompositeActor::PostRegisterAllComponents()
{
	using namespace UE::Composite::Private;

	Super::PostRegisterAllComponents();

#if WITH_EDITOR
	if (IsTemplate() || bIsModifyingAProperty)
	{
		return;
	}
#endif

	if (IsRendering())
	{
		PropagateStateChange(ECompositeStateChangeType::Activate);
	}
}

void ACompositeActor::UnregisterAllComponents(bool bForReregister)
{
	using namespace UE::Composite::Private;

	Super::UnregisterAllComponents(bForReregister);

	if (bForReregister || bIsModifyingAProperty)
	{
		return;
	}

	// State removal is done without any conditions to prevent stale state
	PropagateStateChange(ECompositeStateChangeType::Release);
}

void ACompositeActor::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
	{
		Subsystem->UnregisterActor(this);
	}
}

void ACompositeActor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	SCOPE_CYCLE_COUNTER(STAT_Composite_Actor_Tick);

	if (!IsRendering())
	{
		return;
	}

	// Resolve spawnable camera if the camera binding is valid but the soft pointer is stale (e.g. after level reload with sequencer spawnables).
	// We throttle this to avoid per-frame world iteration when the sequence isn't playing yet.
	const bool bThrottledTick = (GFrameCounter + GetUniqueID()) % 60 == 0;

	if (CameraSpawnableBinding.IsValid() && !CameraActor.IsValid() && bThrottledTick)
	{
		if (AActor* ResolvedCamera = UE::Composite::TryResolveSpawnable(CameraSpawnableBinding, GetWorld()))
		{
			CameraActor = ResolvedCamera;

			if (IsValid(ViewProjectionComponent))
			{
				ViewProjectionComponent->SetCameraActor(CameraActor);
			}
		}
	}

	if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
	{
		Subsystem->SetRenderWork(UE::Composite::Private::GatherFrameRenderWork(*this, SceneCapturesPerPass));

		UE::CompositeCore::FBuiltInRenderPassOptions Options;
		if (bOverridesViewUserFlags)
		{
			Options.ViewUserFlagsOverride = ViewUserFlags;
		}
		Subsystem->SetBuiltInRenderPassOptions(Options);
	}
}

void ACompositeActor::Destroyed()
{
	Super::Destroyed();

	for (TObjectPtr<UCompositeLayerBase>& Layer : CompositeLayers)
	{
		// Note: We don't use IsValid(..) since destruction should proceed even if the actor is pending kill.
		if (Layer != nullptr)
		{
			Layer->OnRemoved(this);
		}
	}
}

#if WITH_EDITOR
void ACompositeActor::PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicateFromRoot(DupParams);

	/**
	 * Whether DupParams.DuplicateMode == EDuplicateMode::PIE or otherwise,
	 * we release transient scene capture render targets (which are not instanced properties)
	 * to prevent their duplication. Allocation must instead come from the pool.
	 *
	 * This is necessary since scene capture's TextureTarget property is not instanced.
	 */
	TInlineComponentArray<USceneCaptureComponent2D*> SceneCaptureComponents(this);
	for (USceneCaptureComponent2D* CaptureComponent : SceneCaptureComponents)
	{
		FCompositeRenderTargetPool& Pool = FCompositeRenderTargetPool::Get();

		if (Pool.IsAssignedTarget(CaptureComponent->TextureTarget))
		{
			Pool.ReleaseTarget(CaptureComponent->TextureTarget);
		}
	}
}

void ACompositeActor::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode == EDuplicateMode::Normal)
	{
		// Notify listeners (e.g. the Composite panel) that CompositeLayers is populated, since OnLevelActorAdded fires before the array deserializes.
		FProperty* CompositeLayersProperty = FindFProperty<FProperty>(ACompositeActor::StaticClass(), GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));
		FPropertyChangedEvent LayersChangedEvent(CompositeLayersProperty, EPropertyChangeType::ArrayAdd);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, LayersChangedEvent);
	}
}

void ACompositeActor::PreEditChange(FProperty* PropertyAboutToChange)
{
	TOptional<TGuardValue<bool>> ModifyingPropertyGuard;
	if (!IsActorBeingDestroyed())
	{
		if ((GEditor && GEditor->bIsSimulatingInEditor && GetWorld() != nullptr) || ReregisterComponentsWhenModified())
		{
			if (PropertyAboutToChange)
			{
				ModifyingPropertyGuard.Emplace(bIsModifyingAProperty, true);
			}
		}
	}

	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(ThisClass, CompositeLayers))
	{
		PreEditCompositeLayers = CompositeLayers;
	}
}

void ACompositeActor::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	TGuardValue<bool> ModifyingPropertyGuard{ bIsModifyingAProperty, true };

	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	const FName PropertyName = PropertyChangedChainEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bIsActive))
	{
		SetIsActive(bIsActive);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bIsEnabled))
	{
		SetIsEnabled(bIsEnabled);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, CompositeLayers))
	{
		// Verify that there isn't another container property node further down in the chain (so we don't respond to array change events from a different array!)
		bool bWasNestedContainerChanged = false;
		FEditPropertyChain::TDoubleLinkedListNode* Node = PropertyChangedChainEvent.PropertyChain.GetActiveNode()->GetNextNode();
		while (!bWasNestedContainerChanged && Node && Node->GetValue())
		{
			bWasNestedContainerChanged = Node->GetValue()->IsA<FArrayProperty>() || Node->GetValue()->IsA<FSetProperty>() || Node->GetValue()->IsA<FMapProperty>();
			Node = Node->GetNextNode();
		}

		if (!bWasNestedContainerChanged)
		{
			const int32 ArrayIndex = PropertyChangedChainEvent.GetArrayIndex(PropertyName.ToString());

			int32 MainRenderIndex = INDEX_NONE;
			
			for (int32 Index =0; Index < CompositeLayers.Num(); ++Index)
			{
				const TObjectPtr<UCompositeLayerBase>& Layer = CompositeLayers[Index];

				if (!IsValid(Layer) || !Layer->GetIsEnabled())
				{
					continue;
				}

				if (Layer->IsA(UCompositeLayerMainRender::StaticClass()))
				{
					MainRenderIndex = Index;
				}
			}

			for (UCompositeLayerBase* PreEditCompositeLayer : PreEditCompositeLayers)
			{
				// Note: We don't use IsValid(..) since destruction should proceed even if the actor is pending kill.
				if (PreEditCompositeLayer != nullptr)
				{
					if (!CompositeLayers.Contains(PreEditCompositeLayer))
					{
						PreEditCompositeLayer->OnRemoved(this);
					}
				}
			}

			PreEditCompositeLayers.Reset();

			if (PropertyChangedChainEvent.PropertyChain.Num() <= 1)
			{
				for (int32 Index = 0; Index < CompositeLayers.Num(); ++Index)
				{
					if (UCompositeLayerPlate* PlateLayer = Cast<UCompositeLayerPlate>(CompositeLayers[Index]))
					{
						// For user convenience when dragging layers, we disable the composite mesh pipeline if the plate layer runs after the main render.
						// TODO: Invert condition if/when traversal is done in the regular indexed order in the new UI.
						PlateLayer->SetPlateMode(Index > MainRenderIndex ? ECompositePlateMode::CompositeMesh : ECompositePlateMode::Texture);
					}
				}
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, CameraActor))
	{
		// SetCameraActor's early-out would trivially short-circuit here (the property editor has
		// already overwritten the member), so handle the side effects directly.
		if (IsValid(ViewProjectionComponent))
		{
			ViewProjectionComponent->SetCameraActor(CameraActor);
		}

		DetectAndStoreSpawnableBinding(CameraActor.Get());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, VPRole))
	{
		SetVPRole(VPRole);
	}
}

void ACompositeActor::PreEditUndo()
{
	Super::PreEditUndo();

	PreEditCompositeLayers = CompositeLayers;
	PreEditVPRole = VPRole;
}

void ACompositeActor::PostEditUndo()
{
	for (UCompositeLayerBase* PreEditCompositeLayer : PreEditCompositeLayers)
	{
		// Note: We don't use IsValid(..) since destruction should proceed even if the actor is pending kill.
		if (PreEditCompositeLayer != nullptr)
		{
			if (!CompositeLayers.Contains(PreEditCompositeLayer))
			{
				PreEditCompositeLayer->OnRemoved(this);
			}
		}
	}

	PreEditCompositeLayers.Reset();

	// Deactivate the current actor if any other actor is already active
	// This can a occur when a previously active actor deletion is undone.
	if (GetIsActive())
	{
		if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
		{
			for (const TWeakObjectPtr<ACompositeActor> Actor : Subsystem->GetActors())
			{
				ACompositeActor* OtherActor = Actor.Get();
				if ((OtherActor != this) && IsValid(OtherActor) && OtherActor->GetIsActive())
				{
					SetIsActive(false);
					break;
				}
			}
		}
	}

	if (PreEditVPRole != VPRole)
	{
		// Manage activation if the vp role has changed
		SetVPRole(VPRole);
	}

	Super::PostEditUndo();
}

void ACompositeActor::SetIsTemporarilyHiddenInEditor(bool bIsHidden)
{
	Super::SetIsTemporarilyHiddenInEditor(bIsHidden);

	PropagateStateChange(IsRendering() ? ECompositeStateChangeType::Activate : ECompositeStateChangeType::Deactivate);
}

bool ACompositeActor::ShouldTickIfViewportsOnly() const
{
	return true;
}
#endif

bool ACompositeActor::GetIsActive() const
{
	return bIsActive;
}

void ACompositeActor::SetIsActive(bool bInActive)
{
	bIsActive = bInActive;

	// Deactivate all other composite actors first
	if (bIsActive)
	{
		if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
		{
			for (const TWeakObjectPtr<ACompositeActor> WeakActor : Subsystem->GetActors())
			{
				ACompositeActor* Actor = WeakActor.Get();
				if (Actor && Actor != this)
				{
					Actor->bIsActive = false;
					Actor->PropagateStateChange(ECompositeStateChangeType::Deactivate);
				}
			}
		}
	}

	PropagateStateChange(IsRendering() ? ECompositeStateChangeType::Activate : ECompositeStateChangeType::Deactivate);

	// Update the view projection matrix sent to the material parameter collection when changing which actor is active.
	if (bIsActive && IsValid(ViewProjectionComponent))
	{
		ViewProjectionComponent->ForceUpdate();
	}
}

bool ACompositeActor::GetIsEnabled() const
{
	return bIsEnabled;
}

void ACompositeActor::SetIsEnabled(bool bInEnabled)
{
	bIsEnabled = bInEnabled;

	PropagateStateChange(IsRendering() ? ECompositeStateChangeType::Activate : ECompositeStateChangeType::Deactivate);
}

UCameraComponent* ACompositeActor::GetCameraComponent() const
{
	if (CameraActor.IsValid())
	{
		return CameraActor->GetComponentByClass<UCameraComponent>();
	}

	return nullptr;
}

void ACompositeActor::SetCameraComponent(UCameraComponent* InComponent)
{
	SetCameraActor(InComponent ? InComponent->GetOwner() : nullptr);
}

const TSoftObjectPtr<AActor>& ACompositeActor::GetCameraActor() const
{
	return CameraActor;
}

void ACompositeActor::SetCameraActor(const TSoftObjectPtr<AActor>& InActor)
{
	// Skip redundant work when sequencer evaluation calls this many times per frame.
	if (CameraActor.ToSoftObjectPath() == InActor.ToSoftObjectPath())
	{
		return;
	}

	CameraActor = InActor;

	if (IsValid(ViewProjectionComponent))
	{
		ViewProjectionComponent->SetCameraActor(InActor);
	}

	if (AActor* Actor = CameraActor.Get())
	{
		DetectAndStoreSpawnableBinding(Actor);
	}
	// On a null InActor we preserve the binding so Tick can re-resolve through sequencer despawns
	// and transient null-pushes from track-section boundaries in Multi-User. User-driven clears
	// (reset-to-default, picker -> None) flow through PostEditChangeChainProperty / the
	// customization helper, which clear the binding explicitly. BP/C++ callers that intend a full
	// clear should use ClearCameraActor() instead.
}

void ACompositeActor::ClearCameraActor()
{
#if WITH_EDITOR
	Modify();
#endif
	CameraActor.Reset();

	if (IsValid(ViewProjectionComponent))
	{
		ViewProjectionComponent->SetCameraActor(nullptr);
	}

	DetectAndStoreSpawnableBinding(nullptr);
}

TArray<UCompositeLayerBase*> ACompositeActor::GetCompositeLayers()
{
	return CompositeLayers;
}

const TArray<TObjectPtr<UCompositeLayerBase>>& ACompositeActor::GetCompositeLayers() const
{
	return CompositeLayers;
}

void ACompositeActor::SetCompositeLayers(TArray<UCompositeLayerBase*> InLayers)
{
	for (UCompositeLayerBase* PreviousLayer : CompositeLayers)
	{
		if (PreviousLayer != nullptr)
		{
			if (!InLayers.Contains(PreviousLayer))
			{
				PreviousLayer->OnRemoved(this);
			}
		}
	}

	CompositeLayers = MoveTemp(InLayers);
}

#if WITH_EDITOR
bool ACompositeActor::ShowPlatePreview(TObjectPtr<class UCompositePassColorKeyer> InRequestingKeyer)
{
	return PlatePreviewComponent->ShowPreview(InRequestingKeyer);
}

void ACompositeActor::HidePlatePreview()
{
	PlatePreviewComponent->HidePreview();
}

bool ACompositeActor::IsPlatePreviewActive() const
{
	return PlatePreviewComponent->IsPreviewActive();
}
#endif

void ACompositeActor::DetectAndStoreSpawnableBinding(AActor* InActor)
{
	CameraSpawnableBinding = UE::Composite::DetectSpawnableBinding(InActor, GetWorld());
}

FGameplayTag ACompositeActor::GetVPRole() const
{
	return VPRole;
}

void ACompositeActor::SetVPRole(FGameplayTag Value)
{
	VPRole = Value;

	SetIsActive(IsInVPRole());
}

template<>
UCompositeSceneCapture2DComponent* ACompositeActor::FindOrCreateSceneCapture(const UCompositePassBase* InPass, int32 InIndex, FName InBaseName)
{
	if (!ensureMsgf(InPass, TEXT("Expected non-null composite pass.")))
	{
		return nullptr;
	}

	if (!ensure(InIndex >= 0))
	{
		return nullptr;
	}

	if (!SceneCapturesPerPass.Contains(InPass))
	{
		SceneCapturesPerPass.Add(InPass, FSceneCaptureComponentArray{});
	}

	TArray<TObjectPtr<UCompositeSceneCapture2DComponent>>& SceneCaptures = SceneCapturesPerPass[InPass].Components;

	if (SceneCaptures.IsValidIndex(InIndex) && IsValid(SceneCaptures[InIndex]))
	{
		return SceneCaptures[InIndex];
	}

	const FName UniqueName = MakeUniqueObjectName(this, UCompositeSceneCapture2DComponent::StaticClass(), InBaseName);
	UCompositeSceneCapture2DComponent* Component = NewObject<UCompositeSceneCapture2DComponent>(this, UniqueName, RF_Transactional);
	Component->SetupAttachment(RootComponent);
	Component->RegisterComponent();
	Component->ComponentTags.Add(UE::Composite::DefaultCompositeTag);
	AddInstanceComponent(Component);
	
	const int32 MinSize = InIndex + 1;
	if (SceneCaptures.Num() < MinSize)
	{
		SceneCaptures.SetNumZeroed(MinSize);
	}
	SceneCaptures[InIndex] = Component;

#if WITH_EDITOR
	FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor").BroadcastComponentsEdited();
#endif

	return Component;
}

template<>
UCompositeShadowReflectionCatcherComponent* ACompositeActor::FindOrCreateSceneCapture(const UCompositePassBase* InPass, int32 InIndex, FName InBaseName)
{
	if (!ensureMsgf(InPass, TEXT("Expected non-null composite pass.")))
	{
		return nullptr;
	}

	if (!ensure(InIndex >= 0))
	{
		return nullptr;
	}

	if (!SceneCapturesPerPass.Contains(InPass))
	{
		SceneCapturesPerPass.Add(InPass, {});
	}

	TArray<TObjectPtr<UCompositeSceneCapture2DComponent>>& SceneCaptureArray = SceneCapturesPerPass[InPass].Components;

	if (SceneCaptureArray.IsValidIndex(InIndex) && IsValid(SceneCaptureArray[InIndex]))
	{
		return Cast<UCompositeShadowReflectionCatcherComponent>(SceneCaptureArray[InIndex]);
	}

	const FName UniqueName = MakeUniqueObjectName(this, UCompositeShadowReflectionCatcherComponent::StaticClass(), InBaseName);
	UCompositeShadowReflectionCatcherComponent* Component = NewObject<UCompositeShadowReflectionCatcherComponent>(this, UniqueName, RF_Transactional);
	Component->SetupAttachment(RootComponent);
	Component->RegisterComponent();
	Component->ComponentTags.Add(UE::Composite::DefaultCompositeTag);
	AddInstanceComponent(Component);

	const int32 MinSize = InIndex + 1;
	if (SceneCaptureArray.Num() < MinSize)
	{
		SceneCaptureArray.SetNumZeroed(MinSize);
	}
	SceneCaptureArray[InIndex] = Component;

#if WITH_EDITOR
	FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor").BroadcastComponentsEdited();
#endif

	return Component;
}

void ACompositeActor::DestroySceneCaptures(const UCompositePassBase* InPass)
{
	if (!ensureMsgf(InPass, TEXT("Expected non-null composite pass.")))
	{
		return;
	}

	bool bComponentRemoved = false;

	FSceneCaptureComponentArray RemovedArray;
	if (SceneCapturesPerPass.RemoveAndCopyValue(InPass, RemovedArray))
	{
		for (TObjectPtr<UCompositeSceneCapture2DComponent>& RemovedComponent : RemovedArray.Components)
		{
			if (IsValid(RemovedComponent))
			{
				RemoveInstanceComponent(RemovedComponent);
				RemovedComponent->UnregisterComponent();
				RemovedComponent->DestroyComponent();

				bComponentRemoved = true;
			}
		}
	}

#if WITH_EDITOR
	if (bComponentRemoved)
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditorModule->BroadcastComponentsEdited();
		}
	}
#endif
}

bool ACompositeActor::IsRendering() const
{
	// Hidden in game
	const UWorld* World = GetWorld();
	if (IsValid(World) && World->IsGameWorld() && IsHidden())
	{
		return false;
	}

	// Hidden in editor
#if WITH_EDITOR
	if (IsTemporarilyHiddenInEditor())
	{
		return false;
	}
#endif

	return GetIsEnabled() && GetIsActive();
}

bool ACompositeActor::IsInVPRole() const
{
	// We are in a valid camera role if the user has not assigned a role or the current VPSettings role matches the
	// assigned role.
	UVirtualProductionRolesSubsystem* VPRolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();

	return !VPRole.IsValid() || (VPRolesSubsystem && VPRolesSubsystem->HasActiveRole(VPRole));
}

void ACompositeActor::PropagateStateChange(ECompositeStateChangeType ChangeType)
{
	for (TObjectPtr<UCompositeLayerBase>& Layer : CompositeLayers)
	{
		if (IsValid(Layer))
		{
			Layer->OnRenderingStateChange(ChangeType);
		}
	}

	// Remove state case
	if (ChangeType != ECompositeStateChangeType::Activate)
	{
		if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
		{
			Subsystem->ResetRenderWork();
			Subsystem->ResetBuiltInRenderPassOptions();
		}
	}
}


