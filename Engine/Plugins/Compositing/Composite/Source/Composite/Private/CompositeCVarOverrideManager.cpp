// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCVarOverrideManager.h"

#include "CompositeModule.h"
#include "HAL/IConsoleManager.h"

static bool GCompositeAllowCVarOverrides = true;
static FAutoConsoleVariableRef CVarCompositeAllowCVarOverrides(
	TEXT("Composite.AllowCVarOverrides"),
	GCompositeAllowCVarOverrides,
	TEXT("Read-only variable to disallow the composite cvar manager from overriding other engine cvars.\n")
	TEXT("0: Disallow plugin overrides\n")
	TEXT("1: Allow plugin overrides"),
	ECVF_ReadOnly
);

FCompositeCVarOverrideManager& FCompositeCVarOverrideManager::Get()
{
	static FCompositeCVarOverrideManager Instance;
	
	return Instance;
}

template <typename T>
void FCompositeCVarOverrideManager::Override(const TCHAR* CVarName, T NewValue)
{
	if (!GCompositeAllowCVarOverrides)
	{
		return;
	}

	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CVarName);

	if (!CVar)
	{
		UE_LOGF(LogComposite, Warning, "Could not find CVar '%ls'", CVarName);
		return;
	}

	if (!OverrideCache.Contains(CVarName))
	{
		FCachedCVarData Data;
		Data.CVarPtr = CVar;
		Data.OriginalValue = CVar->GetString(); // String is valid for all types

		OverrideCache.Add(CVarName, Data);
	}

	CVar->Set(NewValue, ECVF_SetByCode);

	UE_LOGF(LogComposite, Display, "Applying '%ls=%ls' override.", CVarName, *CVar->GetString());
}

template void FCompositeCVarOverrideManager::Override<float>(const TCHAR* CVarName, float NewValue);
template void FCompositeCVarOverrideManager::Override<int32>(const TCHAR* CVarName, int32 NewValue);
template void FCompositeCVarOverrideManager::Override<bool>(const TCHAR* CVarName, bool NewValue);

void FCompositeCVarOverrideManager::Restore(const TCHAR* CVarName)
{
	FCachedCVarData Data;
	if (OverrideCache.RemoveAndCopyValue(CVarName, Data))
	{
		if (Data.CVarPtr)
		{
			Data.CVarPtr->Set(*Data.OriginalValue, ECVF_SetByCode);

			UE_LOGF(LogComposite, Display, "Restored '%ls=%ls'", CVarName, *Data.CVarPtr->GetString());
		}
	}
}

void FCompositeCVarOverrideManager::RestoreAll()
{
	for (auto It = OverrideCache.CreateIterator(); It; ++It)
	{
		FCachedCVarData& Data = It.Value();
		if (Data.CVarPtr)
		{
			Data.CVarPtr->Set(*Data.OriginalValue, ECVF_SetByCode);
		}
	}

	OverrideCache.Empty();
}

void FCompositeCVarOverrideManager::PrintActiveOverrides()
{
	if (OverrideCache.IsEmpty())
	{
		UE_LOGF(LogComposite, Display, "No active overrides.");
		return;
	}

	UE_LOGF(LogComposite, Display, "=== Active Overrides (%d) ===", OverrideCache.Num());

	for (auto It = OverrideCache.CreateConstIterator(); It; ++It)
	{
		const FString& CVarName = It.Key();
		const FCachedCVarData& Data = It.Value();

		FString CurrentValue = TEXT("INVALID");
		if (Data.CVarPtr)
		{
			// Get the live value from the engine
			CurrentValue = Data.CVarPtr->GetString();
		}

		UE_LOGF(LogComposite, Log, "  %ls : Original[%ls] -> Current[%ls]",
			*CVarName,
			*Data.OriginalValue,
			*CurrentValue);
	}

	UE_LOGF(LogComposite, Display, "==================================================");
}
