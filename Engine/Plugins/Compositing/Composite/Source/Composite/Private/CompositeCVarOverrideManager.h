// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Containers/SortedMap.h"

class IConsoleVariable;

/** Console variable override manager. */
class FCompositeCVarOverrideManager
{
public:
	/** Singleton getter. */
	static FCompositeCVarOverrideManager& Get();

	FCompositeCVarOverrideManager(const FCompositeCVarOverrideManager&) = delete;
	FCompositeCVarOverrideManager& operator=(const FCompositeCVarOverrideManager&) = delete;

	/** Overrides a console variable with a new value. */
	template <typename T>
	void Override(const TCHAR* CVarName, T NewValue);

	/** Restore a specific console variable to its original cached value. */
	void Restore(const TCHAR* CVarName);

	/** Restore all overrides to their original values. */
	void RestoreAll();

	/** Debug function to print all active overrides. */
	void PrintActiveOverrides();

private:
	FCompositeCVarOverrideManager() = default;
	~FCompositeCVarOverrideManager() = default;

	struct FCachedCVarData
	{
		IConsoleVariable* CVarPtr = nullptr;
		FString OriginalValue;
	};

	/** Cached original cvar data */
	TSortedMap<FString, FCachedCVarData> OverrideCache;
};
