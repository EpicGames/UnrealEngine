// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeDepthMeshActor.h"

#include "Components/CompositeDepthMeshComponent.h"

ACompositeDepthMeshActor::ACompositeDepthMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CompositeDepthMeshComponent = CreateDefaultSubobject<UCompositeDepthMeshComponent>(TEXT("DefaultCompositeMeshComponent"));
	SetRootComponent(CompositeDepthMeshComponent);
}

ACompositeDepthMeshActor::~ACompositeDepthMeshActor() = default;

