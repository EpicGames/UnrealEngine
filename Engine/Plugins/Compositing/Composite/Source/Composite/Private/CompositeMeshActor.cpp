// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeMeshActor.h"

#include "Components/CompositeMeshComponent.h"

ACompositeMeshActor::ACompositeMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CompositeMeshComponent = CreateDefaultSubobject<UCompositeMeshComponent>(TEXT("DefaultCompositeMeshComponent"));
	SetRootComponent(CompositeMeshComponent);
}

ACompositeMeshActor::~ACompositeMeshActor() = default;

