// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeModule.h"

#include "ConcertSyncSettings.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

#include "CompositeRenderTargetPool.h"

DEFINE_LOG_CATEGORY(LogComposite);

void FCompositeModule::StartupModule()
{
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/Composite"), FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("Composite"))->GetBaseDir(), TEXT("Shaders")));

	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FCompositeModule::OnEnginePreExit);
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FCompositeModule::OnAllModuleLoadingPhasesComplete);
}

void FCompositeModule::ShutdownModule()
{
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

void FCompositeModule::OnAllModuleLoadingPhasesComplete()
{
	// Update MU filter
	UConcertSyncConfig* SyncConfig = UConcertSyncConfig::Get();
	if (SyncConfig)
	{
		TArray<FSoftClassPath> IncludedPaths;
		IncludedPaths.Reserve(3);

		IncludedPaths.Add(FSoftClassPath(TEXT("/Script/Composite.CompositeAssetUserData")));
		IncludedPaths.Add(FSoftClassPath(TEXT("/Script/Composite.CompositePassBase")));
		IncludedPaths.Add(FSoftClassPath(TEXT("/Script/Composite.CompositeLayerBase")));

		const FSoftClassPath OuterClassPath = FSoftClassPath(TEXT("/Script/Engine.World"));

		for (const FSoftClassPath& IncludedPath : IncludedPaths)
		{
			FTransactionClassFilter Filter;
			Filter.ObjectOuterClass = OuterClassPath;
			Filter.ObjectClasses.Add(IncludedPath);
			SyncConfig->IncludeObjectClassFilters.Add(Filter);
		}
	}
}

void FCompositeModule::OnEnginePreExit()
{
	FCompositeRenderTargetPool::Get().Empty();
}

IMPLEMENT_MODULE( FCompositeModule, Composite )
