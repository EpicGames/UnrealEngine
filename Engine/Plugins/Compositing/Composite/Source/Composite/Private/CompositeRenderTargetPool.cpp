// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeRenderTargetPool.h"

#include "CompositeModule.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Logging/StructuredLog.h"
#include "UObject/Package.h"

static TAutoConsoleVariable<int32> CVarCompositePoolFramesUntilFlush(
	TEXT("Composite.Pool.FramesUntilFlush"),
	1,
	TEXT("In editor, you can alter the target size and render format. ")
	TEXT("This may leave render resources pooled, but never reclaimed. Auto-Flushing returns those resources. ")
	TEXT("For values greater than zero, the pooling system will wait that number of frames before a target is considered 'stale'."));

const FIntPoint FCompositeRenderTargetPool::DefaultSize = FIntPoint(1920, 1080);

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			/** Convenience function to release & mark as garbage a render target. */
			void Release(TObjectPtr<UTextureRenderTarget2D>& InTarget)
			{
				if (IsValid(InTarget))
				{
					InTarget->ReleaseResource();
					InTarget->RemoveFromRoot();
					InTarget->MarkAsGarbage();
				}
				InTarget = nullptr;
			}
		}
	}
}

FCompositeRenderTargetPool& FCompositeRenderTargetPool::Get()
{
	static TUniquePtr<FCompositeRenderTargetPool> PoolInstance = MakeUnique<FCompositeRenderTargetPool>();

	return *PoolInstance;
}

TObjectPtr<UTextureRenderTarget2D> FCompositeRenderTargetPool::AcquireTarget(const UObject* InAssignee, const FIntPoint& InSize, ETextureRenderTargetFormat InFormat)
{
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;

	FRenderTargetDesc Desc;
	Desc.Size = InSize;
	Desc.Format = InFormat;
	if (TArray<FPooledRenderTarget>* MatchedTargetsPtr = PooledTargets.Find(Desc))
	{
		TArray<FPooledRenderTarget>& MatchedTargets = *MatchedTargetsPtr;

		if (MatchedTargets.Num() > 0)
		{
			RenderTarget = MatchedTargets.Pop().Texture;

			UE_LOGFMT(LogComposite, VeryVerbose, "Acquiring pooled render target: {Name}", RenderTarget->GetFName());
		}

		if (MatchedTargets.IsEmpty())
		{
			PooledTargets.Remove(Desc);
		}
	}
	
	if(!IsValid(RenderTarget))
	{
		const FName RTName = MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass(), TEXT("CompositePooledRT"));
		RenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), RTName, RF_Transient);
		RenderTarget->SizeX = InSize.X;
		RenderTarget->SizeY = InSize.Y;
		RenderTarget->RenderTargetFormat = InFormat;
		RenderTarget->bForceLinearGamma = true;
		RenderTarget->bSupportsUAV = true;
		RenderTarget->AddressX = TA_Clamp;
		RenderTarget->AddressY = TA_Clamp;

		RenderTarget->UpdateResource();

		UE_LOGFMT(LogComposite, VeryVerbose, "Creating render target: {Name}", RenderTarget->GetFName());
	}

	AssignedTargets.Add(RenderTarget, InAssignee);

	return RenderTarget;
}

bool FCompositeRenderTargetPool::ConditionalAcquireTarget(const UObject* InAssignee, TObjectPtr<UTextureRenderTarget2D>& InOutTarget, const FIntPoint& InSize, ETextureRenderTargetFormat InFormat)
{
	if (InSize.X <= 0 || InSize.Y <= 0)
	{
		UE_LOGFMT(LogComposite, Verbose, "Requested render target size of {0}x{1} is invalid.", InSize.X, InSize.Y);
		return false;
	}

	if (IsValid(InOutTarget)
		&& InOutTarget->SizeX == InSize.X
		&& InOutTarget->SizeY == InSize.Y
		&& InOutTarget->RenderTargetFormat == InFormat)
	{
		// No update required
		return false;
	}

	// Optionally return the existing render target in case it was resized.
	ReleaseTarget(InOutTarget);

	// Update with found or created render target
	InOutTarget = AcquireTarget(InAssignee, InSize, InFormat);

	return true;
}

bool FCompositeRenderTargetPool::IsAssignedTarget(const TObjectPtr<UTextureRenderTarget2D>& InTarget) const
{
	return AssignedTargets.Contains(InTarget);
}

void FCompositeRenderTargetPool::ReleaseTarget(TObjectPtr<UTextureRenderTarget2D>& InTarget)
{
	if (!IsValid(InTarget))
	{
		return;
	}

	UE_LOGFMT(LogComposite, VeryVerbose, "Releasing render target: {Name}", InTarget->GetFName());

	if (AssignedTargets.Remove(InTarget) > 0)
	{
		const FRenderTargetDesc Desc{ FIntPoint(InTarget->SizeX, InTarget->SizeY), InTarget->RenderTargetFormat };
		PooledTargets.FindOrAdd(Desc).Push(InTarget);

		InTarget = nullptr;
	}
	else
	{
		UE_LOGF(LogComposite, Warning, "Attempting to release a render target that doesn't belong to this pool.");
	}
}

void FCompositeRenderTargetPool::ReleaseAssigneeTargets(UObject* InAssignee)
{
	if (!IsValid(InAssignee))
	{
		return;
	}

	for (auto TargetIt = AssignedTargets.CreateIterator(); TargetIt; ++TargetIt)
	{
		if (InAssignee == TargetIt->Value)
		{
			const TObjectPtr<UTextureRenderTarget2D>& RenderTarget = TargetIt->Key;
			const FRenderTargetDesc Desc{ FIntPoint(RenderTarget->SizeX, RenderTarget->SizeY), RenderTarget->RenderTargetFormat };
			PooledTargets.FindOrAdd(Desc).Push(RenderTarget);

			TargetIt.RemoveCurrent();
		}
	}
}

void FCompositeRenderTargetPool::EmptyPool(bool bInForceCollectGarbage)
{
	UE_LOGF(LogComposite, VeryVerbose, "Empty pool.");

	for (auto DescIt = PooledTargets.CreateIterator(); DescIt; ++DescIt)
	{
		for (auto TargetIt = DescIt->Value.CreateIterator(); TargetIt; ++TargetIt)
		{
			UE::Composite::Private::Release(TargetIt->Texture);

			TargetIt.RemoveCurrent();
		}
	}

	PooledTargets.Empty();

	if (bInForceCollectGarbage)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
}

void FCompositeRenderTargetPool::Empty()
{
	constexpr bool bForceCollectGarbage = false;
	EmptyPool(bForceCollectGarbage);

	UE_LOGF(LogComposite, VeryVerbose, "Empty assigned render target.");

	for (auto TargetIt = AssignedTargets.CreateIterator(); TargetIt; ++TargetIt)
	{
		UE::Composite::Private::Release(TargetIt->Key);
	}

	AssignedTargets.Empty();

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
}

int32 FCompositeRenderTargetPool::Num() const
{
	return PooledTargets.Num() + AssignedTargets.Num();
}

void FCompositeRenderTargetPool::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto DescIt = PooledTargets.CreateIterator(); DescIt; ++DescIt)
	{
		for (FPooledRenderTarget& PooledTarget : DescIt->Value)
		{
			Collector.AddReferencedObject(PooledTarget.Texture);
		}
	}

	for (auto TargetIt = AssignedTargets.CreateIterator(); TargetIt; ++TargetIt)
	{
		Collector.AddReferencedObject(TargetIt->Key);
	}
}

FString FCompositeRenderTargetPool::GetReferencerName() const
{
	return TEXT("FCompositeRenderTargetPool");
}

#if WITH_EDITOR
void FCompositeRenderTargetPool::Tick(float /*DeltaSeconds*/)
{
	const int32 FramesUntilFlush = CVarCompositePoolFramesUntilFlush.GetValueOnGameThread();

	// Since we can run in the editor, and could continuously alter the target's render size,
	// we want to flush unused targets (out of fear that they'd never be used again) - targets 
	// regularly used should still be claimed at this point
	if (FramesUntilFlush > 0)
	{
		bool bHasRemovedTarget = false;

		for (auto DescIt = PooledTargets.CreateIterator(); DescIt; ++DescIt)
		{
			for (auto TargetIt = DescIt->Value.CreateIterator(); TargetIt; ++TargetIt)
			{
				if (TargetIt->StaleFrameCount >= FramesUntilFlush)
				{
					TObjectPtr<UTextureRenderTarget2D>& Target = TargetIt->Texture;
					UE_LOGFMT(LogComposite, Verbose, "Releasing stale pooled render target {Name}", Target->GetFName());

					UE::Composite::Private::Release(Target);

					TargetIt.RemoveCurrent();

					bHasRemovedTarget = true;
				}
			}

			if (DescIt->Value.IsEmpty())
			{
				DescIt.RemoveCurrent();
			}
		}

		if (bHasRemovedTarget)
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}

	for (auto DescIt = PooledTargets.CreateIterator(); DescIt; ++DescIt)
	{
		for (FPooledRenderTarget& PooledTarget : DescIt->Value)
		{
			++PooledTarget.StaleFrameCount;
		}
	}

	for (auto TargetIt = AssignedTargets.CreateIterator(); TargetIt; ++TargetIt)
	{
		const TWeakObjectPtr<const UObject>& Assignee = TargetIt->Value;
		if (!Assignee.IsValid())
		{
			TargetIt.RemoveCurrent();

			UE_LOGF(LogComposite, Display, "Releasing assigned render target with invalid assignee object - possible leak?");
		}
	}
}

//------------------------------------------------------------------------------
bool FCompositeRenderTargetPool::IsTickable() const
{
	return Num() > 0;
}

//------------------------------------------------------------------------------
TStatId FCompositeRenderTargetPool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCompositeRenderTargetPool, STATGROUP_Tickables);
}
#endif
