// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/GCObject.h"
#include "Engine/TextureRenderTarget2D.h"

#if WITH_EDITOR
#include "TickableEditorObject.h"
#endif

class UObject;
class UTextureRenderTarget2D;

/**
* While we favor RDG render targets whenever possible, scene captures rely on regular
* UTextureRenderTarget objects which by definition need to live across multiple renders
* (each with their own render dependency graphs).
* 
* This class provides a central location for transient render target management for regular scene captures.
*/
class FCompositeRenderTargetPool final : public FGCObject
#if WITH_EDITOR
	, public FTickableEditorObject
#endif
{
public:
	/** Default render target size/resolution. */
	static const FIntPoint DefaultSize;

	/** Static singleton pool getter. */
	static FCompositeRenderTargetPool& Get();

	/**
	* Acquire new or recyled render target.
	* 
	* @param InAssignee Assignee object whose lifetime is to be paired with the transient render target. If the assignee is invalid, we automatically destroy the render target resource.
	* @param InSize Render target texture size.
	* @param InFormat Render target texture format.
	* @return Render target object.
	*/
	TObjectPtr<UTextureRenderTarget2D> AcquireTarget(const UObject* InAssignee, const FIntPoint& InSize, ETextureRenderTargetFormat InFormat = ETextureRenderTargetFormat::RTF_RGBA16f);

	/**
	* Conditionally acquire new or recyled render target, only if the size & format are mismatched with the current render target.
	*
	* @param InAssignee Assignee object whose lifetime is to be paired with the transient render target. If the assignee is invalid, we automatically destroy the render target resource.
	* @param InOutTarget Render target object to update.
	* @param InSize Render target texture size.
	* @param InFormat Render target texture format.
	* @return True if the render target object was updated.
	*/
	bool ConditionalAcquireTarget(const UObject* InAssignee, TObjectPtr<UTextureRenderTarget2D>& InOutTarget, const FIntPoint& InSize, ETextureRenderTargetFormat InFormat = ETextureRenderTargetFormat::RTF_RGBA16f);

	/** Check if the render target was created by the pool and is currently assigned. */
	bool IsAssignedTarget(const TObjectPtr<UTextureRenderTarget2D>& InTarget) const;

	/**
	* Release a previously acquired render target, and return it into the pool (for 'Composite.Pool.FramesUntilFlush' number of frames).
	* @param InTarget Render target to release.
	*/
	void ReleaseTarget(TObjectPtr<UTextureRenderTarget2D>& InTarget);

	/** Release previously assigned render targets to specified UObject.*/
	void ReleaseAssigneeTargets(UObject* InAssignee);

	/** Destroy available pool render targets. */
	void EmptyPool(bool bInForceCollectGarbage = true);

	/** Destroy all render targets, both available & assigned. */
	void Empty();

	/** Number of managed render targets, both available & assigned. */
	int32 Num() const;

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject interface

		//~ Begin FTickableEditorObject interface
#if WITH_EDITOR
	virtual void Tick(float DeltaSeconds) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
#endif
	//~ End FTickableEditorObject interface

private:

	/** Render target descriptor for finding matching available targets in the pool. */
	struct FRenderTargetDesc
	{
		FIntPoint Size;

		ETextureRenderTargetFormat Format;

		bool operator==(const FRenderTargetDesc& Rhs) const
		{
			return Size == Rhs.Size && Format == Rhs.Format;
		}

		friend uint32 GetTypeHash(const FRenderTargetDesc& InTargetDesc)
		{
			return HashCombine(GetTypeHash(InTargetDesc.Size), GetTypeHash(InTargetDesc.Format));
		}
	};

	/** Pool render target struct with a stale frame counter. */
	struct FPooledRenderTarget
	{
		FPooledRenderTarget() = default;
		
		FPooledRenderTarget(TObjectPtr<UTextureRenderTarget2D> InTextureTarget)
			: Texture(MoveTemp(InTextureTarget))
		{ }

		TObjectPtr<UTextureRenderTarget2D> Texture;
#if WITH_EDITOR
		int32 StaleFrameCount = 0;
#endif 
	};

	/** Available pooled render targets. */
	TMap<const FRenderTargetDesc, TArray<FPooledRenderTarget>> PooledTargets;

	/** Assigned render targets (to be returned into the pool). */
	TMap<TObjectPtr<UTextureRenderTarget2D>, TWeakObjectPtr<const UObject>> AssignedTargets;
};

