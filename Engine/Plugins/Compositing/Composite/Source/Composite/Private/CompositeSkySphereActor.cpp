// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeSkySphereActor.h"

#include "Components/SkyLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaTexture.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "UObject/ConstructorHelpers.h"

ACompositeSkySphereActor::ACompositeSkySphereActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UStaticMesh> SkySphere;
		ConstructorHelpers::FObjectFinder<UMaterialInterface> SkySphere_Material;
		ConstructorHelpers::FObjectFinder<UTexture2D> DefaultTexture;
		FConstructorStatics()
			: SkySphere(TEXT("/Composite/Meshes/SM_CompositeSkySphere.SM_CompositeSkySphere"))
			, SkySphere_Material(TEXT("/Composite/Materials/M_CompositeSkySphere.M_CompositeSkySphere"))
			, DefaultTexture(TEXT("/Composite/Textures/T_Composite_SMPTE_Color_Bars_16x9.T_Composite_SMPTE_Color_Bars_16x9"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SkySphereComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("SkySphereComponent"));
	SetRootComponent(SkySphereComponent);
	SkySphereComponent->SetCastShadow(false);
	SkySphereComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SkySphereComponent->SetEnableGravity(false);
	// SM_CompositeSkySphere has a 100 UU diameter (50 UU radius).
	// Scale 1000 gives a 50,000 UU radius — comfortably beyond the default 15,000 UU sky distance threshold.
	SkySphereComponent->SetRelativeScale3D(FVector(1000.0f));

	if (ensure(ConstructorStatics.SkySphere.Object != nullptr))
	{
		SkySphereComponent->SetStaticMesh(ConstructorStatics.SkySphere.Object);
	}

	if (ConstructorStatics.SkySphere_Material.Object != nullptr)
	{
		SkySphereComponent->SetMaterial(0, ConstructorStatics.SkySphere_Material.Object);
	}

	Texture = Cast<UTexture>(ConstructorStatics.DefaultTexture.Object);

	SkyLightComponent = CreateDefaultSubobject<USkyLightComponent>(TEXT("SkyLightComponent"));
	SkyLightComponent->SetupAttachment(SkySphereComponent);
	SkyLightComponent->bRealTimeCapture = true;
	SkyLightComponent->bLowerHemisphereIsBlack = false;
}

ACompositeSkySphereActor::~ACompositeSkySphereActor() = default;

void ACompositeSkySphereActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	
	UpdateMaterial();
}

void ACompositeSkySphereActor::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdateMaterial();
		TryOpenMediaProfileSource();
	}
}

void ACompositeSkySphereActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	TryCloseMediaProfileSource();
	Super::EndPlay(EndPlayReason);
}

void ACompositeSkySphereActor::UpdateMaterial()
{
	UMaterialInterface* CurrentMaterial = SkySphereComponent->GetMaterial(0);
	if (CurrentMaterial == nullptr)
	{
		return;
	}

	UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(CurrentMaterial);
	if (!MID)
	{
		if (Texture == nullptr)
		{
			return; // No texture and no existing MID — nothing to do.
		}
		MID = SkySphereComponent->CreateAndSetMaterialInstanceDynamic(0);
	}

	if (ensure(MID != nullptr))
	{
		MID->SetTextureParameterValue(TextureParameterName, Texture);
	}
}

void ACompositeSkySphereActor::SetTexture(UTexture* InTexture)
{
	TryCloseMediaProfileSource();
	Texture = InTexture;
	UpdateMaterial();
	TryOpenMediaProfileSource();
}

void ACompositeSkySphereActor::TryOpenMediaProfileSource()
{
	UMediaTexture* MediaTexture = Cast<UMediaTexture>(Texture);
	if (!MediaTexture)
	{
		return;
	}

	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return;
	}

	int32 MediaSourceIndex = INDEX_NONE;
	if (ActiveMediaProfile->GetPlaybackManager()->IsValidSourceMediaTexture(MediaTexture, MediaSourceIndex))
	{
		ActiveMediaProfile->GetPlaybackManager()->OpenSourceFromIndex(MediaSourceIndex, this);
	}
}

void ACompositeSkySphereActor::TryCloseMediaProfileSource()
{
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return;
	}

	UMediaProfilePlaybackManager::FCloseSourceArgs Args;
	Args.Consumer = this;
	ActiveMediaProfile->GetPlaybackManager()->CloseSourcesForConsumer(Args);
}

#if WITH_EDITOR
void ACompositeSkySphereActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange &&
		PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ACompositeSkySphereActor, Texture))
	{
		TryCloseMediaProfileSource();
	}
}

void ACompositeSkySphereActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositeSkySphereActor, Texture) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACompositeSkySphereActor, TextureParameterName))
	{
		UpdateMaterial();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositeSkySphereActor, Texture))
	{
		TryOpenMediaProfileSource();
	}
}
#endif
