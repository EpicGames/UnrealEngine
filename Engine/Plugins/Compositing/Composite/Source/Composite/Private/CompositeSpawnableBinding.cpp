// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeSpawnableBinding.h"

#include "EngineUtils.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "MovieScene.h"
#include "MovieSceneObjectBindingID.h"

#if WITH_EDITOR
#include "ISequencer.h"
#include "LevelEditorSequencerIntegration.h"
#endif

// -----------------------------------------------------------------------------
// Free functions
// -----------------------------------------------------------------------------

FCompositeSpawnableBinding UE::Composite::DetectSpawnableBinding(AActor* InActor, UWorld* InWorld)
{
	FCompositeSpawnableBinding Result;

	if (!IsValid(InActor))
	{
		return Result;
	}

#if WITH_EDITOR
	// Editor path: check open sequencer instances (works when scrubbing, not just playing).
	for (TWeakPtr<ISequencer> WeakSequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid() || !Sequencer->FindSharedPlaybackState())
		{
			continue;
		}

		// Only store bindings for transient (spawnable) actors — possessables don't need re-resolution.
		if (!InActor->HasAnyFlags(RF_Transient))
		{
			continue;
		}

		FGuid FoundGuid = Sequencer->FindObjectId(*InActor, Sequencer->GetFocusedTemplateID());
		if (!FoundGuid.IsValid())
		{
			continue;
		}

		ULevelSequence* RootLevelSequence = Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence());
		if (RootLevelSequence)
		{
			Result.BindingId = FoundGuid;
			Result.LevelSequence = RootLevelSequence;
			return Result;
		}
	}
#endif

	// Runtime path: check level sequence players (works when the sequence is playing).
	if (!IsValid(InWorld))
	{
		return Result;
	}

	for (TActorIterator<ALevelSequenceActor> It(InWorld); It; ++It)
	{
		ALevelSequenceActor* SequenceActor = *It;
		ULevelSequencePlayer* Player = SequenceActor ? SequenceActor->GetSequencePlayer() : nullptr;
		if (!Player || !Player->FindSharedPlaybackState())
		{
			continue;
		}

		const FGuid FoundGuid = Player->FindObjectId(*InActor, FMovieSceneSequenceID());
		if (!FoundGuid.IsValid())
		{
			continue;
		}

		ULevelSequence* Sequence = SequenceActor->GetSequence();
		UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
		if (MovieScene && MovieScene->FindSpawnable(FoundGuid))
		{
			Result.BindingId = FoundGuid;
			Result.LevelSequence = Sequence;
			return Result;
		}
	}

	return Result;
}

AActor* UE::Composite::TryResolveSpawnable(const FCompositeSpawnableBinding& Binding, UWorld* InWorld)
{
	if (!Binding.IsValid() || Binding.LevelSequence.IsNull())
	{
		return nullptr;
	}

#if WITH_EDITOR
	// Editor path: resolve via open sequencer instances (works when scrubbing).
	for (TWeakPtr<ISequencer> WeakSequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid() || !Sequencer->FindSharedPlaybackState())
		{
			continue;
		}

		// Verify this sequencer instance corresponds to our stored sequence to avoid cross-sequence GUID collisions.
		ULevelSequence* RootSequence = Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence());
		if (RootSequence != Binding.LevelSequence.Get())
		{
			continue;
		}

		TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindObjectsInCurrentSequence(Binding.BindingId);
		for (const TWeakObjectPtr<>& WeakObj : BoundObjects)
		{
			if (AActor* Actor = Cast<AActor>(WeakObj.Get()))
			{
				return Actor;
			}
		}
	}
#endif

	// Runtime path: resolve via level sequence players (works when the sequence is playing).
	if (!IsValid(InWorld))
	{
		return nullptr;
	}

	ULevelSequence* TargetSequence = Binding.LevelSequence.Get();
	if (!TargetSequence)
	{
		return nullptr;
	}

	for (TActorIterator<ALevelSequenceActor> It(InWorld); It; ++It)
	{
		ALevelSequenceActor* SequenceActor = *It;
		if (!SequenceActor || SequenceActor->GetSequence() != TargetSequence)
		{
			continue;
		}

		ULevelSequencePlayer* Player = SequenceActor->GetSequencePlayer();
		if (!Player)
		{
			continue;
		}

		TSharedPtr<const UE::MovieScene::FSharedPlaybackState> PlaybackState = Player->FindSharedPlaybackState();
		if (!PlaybackState)
		{
			continue;
		}

		// MovieSceneSequenceID::Root (0), not FMovieSceneSequenceID() (-1 / Invalid), identifies the
		// root sequence; the latter triggers Hierarchy lookup that ensures during sequencer teardown.
		for (const TWeakObjectPtr<>& WeakObj : Player->FindBoundObjects(Binding.BindingId, MovieSceneSequenceID::Root))
		{
			if (AActor* Actor = Cast<AActor>(WeakObj.Get()))
			{
				return Actor;
			}
		}
	}

	return nullptr;
}

// -----------------------------------------------------------------------------
// FCompositeSpawnableBinding
// -----------------------------------------------------------------------------

bool FCompositeSpawnableBinding::IsValid() const
{
	return BindingId.IsValid();
}

void FCompositeSpawnableBinding::Invalidate()
{
	BindingId.Invalidate();
	LevelSequence.Reset();
}

// -----------------------------------------------------------------------------
// FCompositeSpawnableBindings
// -----------------------------------------------------------------------------

bool FCompositeSpawnableBindings::HasBindingAt(int32 Index) const
{
	return Bindings.IsValidIndex(Index) && Bindings[Index].IsValid();
}

bool FCompositeSpawnableBindings::TryResolveStale(TArray<TSoftObjectPtr<AActor>>& Actors, UWorld* InWorld) const
{
	bool bAnyResolved = false;

	const int32 Count = FMath::Min(Actors.Num(), Bindings.Num());
	for (int32 i = 0; i < Count; ++i)
	{
		if (!Actors[i].IsValid() && Bindings[i].IsValid())
		{
			if (AActor* Resolved = UE::Composite::TryResolveSpawnable(Bindings[i], InWorld))
			{
				Actors[i] = Resolved;
				bAnyResolved = true;
			}
		}
	}

	return bAnyResolved;
}

bool FCompositeSpawnableBindings::TickResolveStale(TArray<TSoftObjectPtr<AActor>>& Actors, UWorld* InWorld, uint32 InstanceId) const
{
	return ((GFrameCounter + InstanceId) % 60 == 0) && TryResolveStale(Actors, InWorld);
}

void FCompositeSpawnableBindings::CachePreEditState(TArrayView<const TSoftObjectPtr<AActor>> Actors)
{
	// Snapshot the pre-edit bindings keyed by path. Bindings carried by path survive stale-pointer
	// transitions (sequencer despawn, Multi-User peer with transient sender path) so the Tick
	// re-resolver can recover the actor when it spawns back. Slots without a valid binding don't
	// need preservation and are skipped.
	PreEditBindingsByPath.Empty(Actors.Num());
	const int32 Count = FMath::Min(Actors.Num(), Bindings.Num());
	for (int32 i = 0; i < Count; ++i)
	{
		if (Bindings[i].IsValid())
		{
			PreEditBindingsByPath.Add(Actors[i].ToSoftObjectPath(), Bindings[i]);
		}
	}
}

void FCompositeSpawnableBindings::SyncOnPropertyChange(TArrayView<TSoftObjectPtr<AActor>> NewActors, UWorld* InWorld)
{
	Bindings.SetNum(NewActors.Num());
	for (int32 i = 0; i < NewActors.Num(); ++i)
	{
		const FSoftObjectPath ActorPath = NewActors[i].ToSoftObjectPath();

		if (const FCompositeSpawnableBinding* Found = PreEditBindingsByPath.Find(ActorPath))
		{
			// Same soft path as before — carry over its existing binding (preserves the
			// side-channel across stale-pointer windows).
			Bindings[i] = *Found;
		}
		else if (NewActors[i].IsValid())
		{
			// New, resolved actor — attempt detection.
			Bindings[i] = UE::Composite::DetectSpawnableBinding(NewActors[i].Get(), InWorld);
		}
		else
		{
			// Different soft path AND not resolved — clear any stale leftover.
			Bindings[i].Invalidate();
		}
	}

	// Drop the cache so a stale entry can't leak into a later sync.
	PreEditBindingsByPath.Reset();
}
