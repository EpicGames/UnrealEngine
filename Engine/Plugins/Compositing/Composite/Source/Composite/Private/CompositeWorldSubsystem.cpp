// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeWorldSubsystem.h"
#include "CompositeActor.h"
#include "CompositeModule.h"
#include "VPRolesSubsystem.h"

#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceControlModule.h"
#include "Misc/ConfigCacheIni.h"
#include "SourceControlHelpers.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "CompositeWorldSubsystem"

namespace
{
#if WITH_EDITOR
	void UpdateDependentPropertyInConfigFile(URendererSettings* RendererSettings, FProperty* RendererProperty)
	{
		const FString RelativePath = RendererSettings->GetDefaultConfigFilename();
		const FString FullPath = FPaths::ConvertRelativePathToFull(RelativePath);

		const bool bIsReadOnly = FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FullPath);
		bool bCheckedOutFromSourceControl = false;

		if (bIsReadOnly)
		{
			// Prefer to check out via the editor's source-control provider so the user sees the file
			// appear in their pending changelist. Falls through to a raw read-only toggle on failure
			// (provider disabled, not connected, or file untracked).
			if (ISourceControlModule::Get().IsEnabled())
			{
				bCheckedOutFromSourceControl = USourceControlHelpers::CheckOutOrAddFile(FullPath);
				if (!bCheckedOutFromSourceControl)
				{
					UE_LOGF(LogComposite, Warning, "Could not check out '%ls' via source control: %ls", *FullPath, *USourceControlHelpers::LastErrorMsg().ToString());
				}
			}

			if (!bCheckedOutFromSourceControl)
			{
				FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, false);
			}
		}

		RendererSettings->UpdateSinglePropertyInConfigFile(RendererProperty, RelativePath);

		// If we only made the file writable on the file system (no source-control checkout),
		// restore the read-only bit so a future sync from source control isn't blocked.
		if (bIsReadOnly && !bCheckedOutFromSourceControl)
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, true);
		}
	}
#endif
}

void UCompositeWorldSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UVirtualProductionRolesSubsystem* VPRolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();
	if (VPRolesSubsystem)
	{
		TWeakObjectPtr<UCompositeWorldSubsystem> WeakThis = this;

		RolesChangedDelegateHandle = VPRolesSubsystem->OnRolesChanged().AddLambda([WeakThis](const TArray<FString>&)
			{
				TStrongObjectPtr<UCompositeWorldSubsystem> This = WeakThis.Pin();
				if (!This.IsValid())
				{
					return;
				}

				const UWorld* World = This->GetWorld();
				if (!World)
				{
					return;
				}

				for (const TWeakObjectPtr<ACompositeActor>& Actor : This->CompositeActors)
				{
					ACompositeActor* CompositeActor = Actor.Get();
					if (!CompositeActor)
					{
						continue;
					}

					const FGameplayTag Role = CompositeActor->GetVPRole();
					if (Role.IsValid())
					{
						// Update activation via setter when device role changes
						CompositeActor->SetVPRole(Role);
					}
				}
			}
		);
	}
}

void UCompositeWorldSubsystem::PreDeinitialize()
{
	Super::PreDeinitialize();

	if (UVirtualProductionRolesSubsystem* VPRolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>())
	{
		VPRolesSubsystem->OnRolesChanged().Remove(RolesChangedDelegateHandle);
	}
}

void UCompositeWorldSubsystem::RegisterActor(TWeakObjectPtr<ACompositeActor> InActor)
{
	if (InActor.IsValid() && InActor->GetWorld() == GetWorld())
	{
		CompositeActors.Add(InActor);

		ValidateCompositeActorSettings();
	}
}

void UCompositeWorldSubsystem::UnregisterActor(TWeakObjectPtr<ACompositeActor> InActor)
{
	CompositeActors.Remove(InActor);
}

const TSet<TWeakObjectPtr<ACompositeActor>>& UCompositeWorldSubsystem::GetActors() const
{
	return CompositeActors;
}

bool UCompositeWorldSubsystem::IsCompositeActorSettingsValid()
{
	static const auto CVarAutoExposure = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DefaultFeature.AutoExposure"));

	const bool bAutoExposureOff = CVarAutoExposure ? !CVarAutoExposure->GetBool() : false;

	return bAutoExposureOff;
}

void UCompositeWorldSubsystem::ValidateCompositeActorSettings()
{
	if (IsCompositeActorSettingsValid())
	{
		return;
	}

#if WITH_EDITOR
	UE_CALL_ONCE([this] { CompositeActorSettingsNotification(); });
#else
	UE_CALL_ONCE([] { UE_LOGF(LogComposite, Warning, "Recommended Composite project setting is not set: disable \"Auto Exposure\"."); });
#endif
}

#if WITH_EDITOR
void UCompositeWorldSubsystem::CompositeActorSettingsNotification()
{
	URendererSettings* RendererSettings = GetMutableDefault<URendererSettings>();
	check(RendererSettings);

	const bool bAutoExposureNeedsUpdate = !!RendererSettings->bDefaultFeatureAutoExposure;

	if (!bAutoExposureNeedsUpdate)
	{
		UE_LOGF(LogComposite, Warning, "Composite recommended settings are valid, but one or more console variables have been overridden.");
		return;
	}

	const FText SubText = LOCTEXT("CompositeSettings_AutoExposure", "- Disable Auto Exposure");
	const FText PromptText = LOCTEXT("CompositeSettings_RecommendedPrompt", "A project setting does not match the recommendation for Composure. Would you like to update it?");
	const FText ConfirmText = LOCTEXT("CompositeSettings_Update", "Update");
	const FText CancelText  = LOCTEXT("CompositeSettings_NotNow", "Not Now");

	struct FSuppressDialogOptions
	{
		static bool ShouldSuppressModal()
		{
			bool bSuppressNotification = false;
			GConfig->GetBool(TEXT("Composite"), TEXT("SuppressCompositeProjectSettingsNotification"), bSuppressNotification, GEditorPerProjectIni);
			return bSuppressNotification;
		}

		static ECheckBoxState GetDontAskAgainCheckBoxState()
		{
			return ShouldSuppressModal() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
		{
			const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
			GConfig->SetBool(TEXT("Composite"), TEXT("SuppressCompositeProjectSettingsNotification"), bSuppressNotification, GEditorPerProjectIni);
		}
	};

	if (FSuppressDialogOptions::ShouldSuppressModal())
	{
		return;
	}

	FSimpleDelegate OnConfirmDelegate = FSimpleDelegate::CreateLambda(
		[WeakThis = MakeWeakObjectPtr(this), RendererSettings]()
		{
			if (IsValid(RendererSettings))
			{
				if (!!RendererSettings->bDefaultFeatureAutoExposure)
				{
					FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bDefaultFeatureAutoExposure));
					RendererSettings->PreEditChange(Property);

					RendererSettings->bDefaultFeatureAutoExposure = 0;

					FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, { RendererSettings });
					RendererSettings->PostEditChangeProperty(PropertyChangedEvent);
					UpdateDependentPropertyInConfigFile(RendererSettings, Property);
				}
			}

			TStrongObjectPtr<UCompositeWorldSubsystem> Subsystem = WeakThis.Pin();
			if (Subsystem.IsValid())
			{
				TSharedPtr<SNotificationItem> NotificationItem = Subsystem->CompositeActorNotificationItem.Pin();
				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
					NotificationItem->ExpireAndFadeout();
				}

				Subsystem->CompositeActorNotificationItem.Reset();
			}
		}
	);

	FSimpleDelegate OnCancelDelegate = FSimpleDelegate::CreateLambda(
		[WeakThis = MakeWeakObjectPtr(this)]()
		{
			TStrongObjectPtr<UCompositeWorldSubsystem> Subsystem = WeakThis.Pin();
			if (Subsystem.IsValid())
			{
				TSharedPtr<SNotificationItem> NotificationItem = Subsystem->CompositeActorNotificationItem.Pin();
				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_None);
					NotificationItem->ExpireAndFadeout();
				}

				Subsystem->CompositeActorNotificationItem.Reset();
			}
		}
	);

	FNotificationInfo Info(PromptText);
	Info.SubText = SubText;
	Info.bFireAndForget = false;
	Info.bUseLargeFont = false;
	Info.bUseThrobber = false;
	Info.bUseSuccessFailIcons = false;
	Info.ButtonDetails.Add(FNotificationButtonInfo(ConfirmText, FText(), OnConfirmDelegate));
	Info.ButtonDetails.Add(FNotificationButtonInfo(CancelText, FText(), OnCancelDelegate));

	Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&FSuppressDialogOptions::GetDontAskAgainCheckBoxState);
	Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&FSuppressDialogOptions::OnDontAskAgainCheckBoxStateChanged);
	Info.CheckBoxText = LOCTEXT("CompositeSettings_DontShowAgain", "Don't show this again");

	if (CompositeActorNotificationItem.IsValid())
	{
		CompositeActorNotificationItem.Pin()->ExpireAndFadeout();
		CompositeActorNotificationItem.Reset();
	}

	CompositeActorNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

	if (CompositeActorNotificationItem.IsValid())
	{
		CompositeActorNotificationItem.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}
#endif

#undef LOCTEXT_NAMESPACE

