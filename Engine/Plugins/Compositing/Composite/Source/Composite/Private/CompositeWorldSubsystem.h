// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeCoreSubsystem.h"

#include "CompositeWorldSubsystem.generated.h"

class ACompositeActor;
class SNotificationItem;

/**
 * Subsystem for composite actors, deriving from the CompositeCore subsystem.
 */
UCLASS(MinimalAPI)
class UCompositeWorldSubsystem : public UCompositeCoreSubsystem
{
	GENERATED_BODY()

public:
	/** Called once all UWorldSubsystems have been initialized */
	virtual void PostInitialize() override;

	/** Called once all UWorldSubsystems are about to be deinitialized */
	virtual void PreDeinitialize() override;

	/** Register composite actor. */
	void RegisterActor(TWeakObjectPtr<ACompositeActor> InActor);

	/** Unregister composite actor. */
	void UnregisterActor(TWeakObjectPtr<ACompositeActor> InActor);

	/** Get all registered composite actors. */
	const TSet<TWeakObjectPtr<ACompositeActor>>& GetActors() const;

	/** Whether the recommended project settings for Composite actors are set (auto exposure off, screen percentage manual @ 100). */
	static bool IsCompositeActorSettingsValid();

private:
	/** Once per editor session, surface a recommendation toast if Composite-actor project settings are not at recommended values. */
	void ValidateCompositeActorSettings();

#if WITH_EDITOR
	/** Shows the editor toast that offers to apply the recommended settings. */
	void CompositeActorSettingsNotification();

	/** Weak handle to the active recommendation toast so it can be dismissed when acted upon. */
	TWeakPtr<SNotificationItem> CompositeActorNotificationItem;
#endif

	/** List of registered actors. */
	TSet<TWeakObjectPtr<ACompositeActor>> CompositeActors;

	/** Event delegate for a device VP role change. */
	FDelegateHandle RolesChangedDelegateHandle;
};
