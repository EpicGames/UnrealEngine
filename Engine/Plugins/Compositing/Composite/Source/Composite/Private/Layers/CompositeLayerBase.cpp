// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerBase.h"

#include "CompositeActor.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

UCompositeLayerBase::UCompositeLayerBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Marked RF_Transactional if non-CDO in parent UCompositePassBase
}

UCompositeLayerBase::~UCompositeLayerBase() = default;

void UCompositeLayerBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const bool bDeprecatedNameMigration = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::Composite_LayerNameDeprecation;

	if (bDeprecatedNameMigration && !Name_DEPRECATED.IsEmpty() && (DisplayName.IsEmpty() || DisplayName == GetClass()->GetName()))
	{
		DisplayName = Name_DEPRECATED;
	}
#endif
}

void UCompositeLayerBase::BeginDestroy()
{
	Super::BeginDestroy();

	constexpr bool bEnabled = false;
	RegisterEndOfFrameUpdate(bEnabled);
}

#if WITH_EDITOR
bool UCompositeLayerBase::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);

	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();
		
		if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Operation))
		{
			bIsEditable = !bIsSolo;
		}
	}

	return bIsEditable;
}

void UCompositeLayerBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bIsSolo))
	{
		if (const ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>())
		{
			for (const TObjectPtr<UCompositeLayerBase>& CompositeLayer : CompositeActor->GetCompositeLayers())
			{
				if (IsValid(CompositeLayer) && CompositeLayer != this)
				{
					CompositeLayer->Modify();
					CompositeLayer->bIsSolo = false;
				}
			}
		}
	}
}
#endif //WITH_EDITOR

FCompositeCorePassProxy* UCompositeLayerBase::GetProxy(
	const UE::CompositeCore::FPassInputDecl& /*InputDecl*/,
	FCompositeTraversalContext& /*InContext*/,
	FSceneRenderingBulkObjectAllocator& /*InFrameAllocator*/) const
{
	return nullptr;
}

void UCompositeLayerBase::UpdateSceneCaptureComponents(USceneCaptureComponent2D& SceneCaptureComponent, TArrayView<TSoftObjectPtr<AActor>> InActors)
{
	auto UpdateComponentsFn = [InActors](TArray<TWeakObjectPtr<UPrimitiveComponent>>& InComponents) -> void
		{
			InComponents.Reset(InActors.Num());

			for (const TSoftObjectPtr<AActor>& SoftActor : InActors)
			{
				if (const AActor* Actor = SoftActor.Get())
				{
					for (UActorComponent* Component : Actor->GetComponents())
					{
						if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
						{
							InComponents.Add(PrimitiveComponent);
						}
					}
				}
			}
		};
	
	if (SceneCaptureComponent.PrimitiveRenderMode == ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList)
	{
		UpdateComponentsFn(SceneCaptureComponent.ShowOnlyComponents);
	}
	else
	{
		UpdateComponentsFn(SceneCaptureComponent.HiddenComponents);
	}
}

void UCompositeLayerBase::RegisterEndOfFrameUpdate(bool bInEnabled)
{
	if (bInEnabled)
	{
		if (!OnWorldPreSendAllEndOfFrameUpdatesHandle.IsValid())
		{
			OnWorldPreSendAllEndOfFrameUpdatesHandle = FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.AddUObject(this, &UCompositeLayerBase::ConditionalOnEndOfFrameUpdate);
		}
	}
	else
	{
		if (OnWorldPreSendAllEndOfFrameUpdatesHandle.IsValid())
		{
			FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.Remove(OnWorldPreSendAllEndOfFrameUpdatesHandle);
			OnWorldPreSendAllEndOfFrameUpdatesHandle.Reset();
		}
	}
}

void UCompositeLayerBase::AddChildPasses(UE::CompositeCore::FPassInputDecl& InBasePassInput, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator, TArrayView<const TObjectPtr<UCompositePassBase>> InPasses) const
{
	// Iterate backwards so that lower passes in the UI are executed first
	// TODO: Flip back once we have UI customization displaying passes in the correct bottom-up order
	for (int32 PassIndex = InPasses.Num() - 1; PassIndex >= 0; --PassIndex)
	{
		const TObjectPtr<UCompositePassBase>& SubPass = InPasses[PassIndex];

		if (IsValid(SubPass) && SubPass->GetIsActive())
		{
			FCompositeCorePassProxy* ChildPassProxy = SubPass->GetProxy(InBasePassInput, InContext, InFrameAllocator);
			if (ChildPassProxy != nullptr)
			{
				// Next pass should receive the output of the current child pass.
				InBasePassInput.Set<const FCompositeCorePassProxy*>(ChildPassProxy);
			}
		}
	}
}

ECompositeCoreMergeOp UCompositeLayerBase::GetMergeOperation(const FCompositeTraversalContext& InContext) const
{
	return InContext.bIsSolo ? ECompositeCoreMergeOp::None : Operation;
}

UE::CompositeCore::FPassInputDecl UCompositeLayerBase::GetDefaultSecondInput(const FCompositeTraversalContext& InContext) const
{
	using namespace UE::CompositeCore;

	if (InContext.bIsFirstPass)
	{
		// Force an empty previous texture on the first pass
		return MakeExternalInput(ResourceId::BuiltInEmpty);
	}
	else
	{
		return MakeInternalInput();
	}
}

void UCompositeLayerBase::ConditionalOnEndOfFrameUpdate(UWorld* InWorld)
{
	/**
	 * Only call the virtual end-of-frame method if our layer hasn't been disabled or removed.
	 * This is simpler than unregistering and re-registering if the removal is undone.
	*/
	if (GetIsEnabled())
	{
		ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();

		if (IsValid(CompositeActor) && CompositeActor->GetCompositeLayers().Contains(this))
		{
			OnEndOfFrameUpdate(InWorld);
		}
	}
}

