// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerMainRender.h"

#include "CompositeActor.h"

UCompositeLayerMainRender::UCompositeLayerMainRender(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Operation = ECompositeCoreMergeOp::Over;
}

UCompositeLayerMainRender::~UCompositeLayerMainRender() = default;

FCompositeCorePassProxy* UCompositeLayerMainRender::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	FPassInputDecl PassInput;
	FPassInputDeclArray Inputs;
	Inputs.SetNum(FixedNumLayerInputs);

	FPassInternalResourceDesc Desc0;
	Desc0.Index = 0;
	Desc0.bOriginalCopyBeforePasses = true;
	PassInput.Set<FPassInternalResourceDesc>(Desc0);

	AddChildPasses(PassInput, InContext, InFrameAllocator, LayerPasses);

	Inputs[0] = PassInput;
	Inputs[1] = GetDefaultSecondInput(InContext);

	InContext.bNeedsSceneTextures = true;
	return InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), GetMergeOperation(InContext), TEXT("MainRender"), ELensDistortionHandling::Enabled, bUltimatteBlend);
}

