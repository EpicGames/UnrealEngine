// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerPlanarReflection.h"

#include "CompositeActor.h"
#include "Components/CompositeSceneCapture2DComponent.h"
#include "CompositeRenderTargetPool.h"
#include "Misc/TransactionObjectEvent.h"
#include "Passes/CompositeCorePassProxy.h"
#include "RenderGraphBuilder.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Camera/CameraComponent.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelEditorSubsystem.h"
#include "LevelEditorViewport.h"
#endif

DECLARE_GPU_STAT_NAMED(FCompositePlanarReflection, TEXT("Composite.PlanarReflection"));

class FCompositePlanarReflectionShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePlanarReflectionShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePlanarReflectionShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER(float, ReflectionStrength)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositePlanarReflectionShader, "/Plugin/Composite/Private/CompositePlanarReflection.usf", "MainPS", SF_Pixel);

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			/** Resolve the planar reflection capture source from the composite actor output mode. */
			ESceneCaptureSource GetPlanarReflectionCaptureSource(const ACompositeActor& CompositeActor, bool bCustomRenderPass)
			{
				if (bCustomRenderPass)
				{
					return ESceneCaptureSource::SCS_SceneColorHDR;
				}

				return CompositeActor.MainRenderOutput == ECompositeMainRenderOutputMode::FinalColorHDR
					? ESceneCaptureSource::SCS_FinalColorHDR
					: ESceneCaptureSource::SCS_FinalToneCurveHDR;
			}

#if WITH_EDITOR
			/** Get the currently active level editor viewport client, if any. */
			FLevelEditorViewportClient* GetActiveLevelEditorViewportClient()
			{
				if (const FViewport* ActiveViewport = GEditor ? GEditor->GetActiveViewport() : nullptr)
				{
					for (FLevelEditorViewportClient* LevelViewportClient : GEditor->GetLevelViewportClients())
					{
						if (LevelViewportClient && LevelViewportClient->Viewport == ActiveViewport)
						{
							return LevelViewportClient;
						}
					}
				}

				return nullptr;
			}

			/**
			* Build a view from the active editor viewport when the composite camera is not being piloted.
			* This is needed here because we cannot rely on the scene capture's "MainViewCamera" behavior.
			*/
			bool TryGetEditorViewportCameraView(ACompositeActor& CompositeActor, FMinimalViewInfo& OutViewInfo)
			{
				FLevelEditorViewportClient* ViewportClient = GetActiveLevelEditorViewportClient();
				if (!ViewportClient || !ViewportClient->IsPerspective())
				{
					return false;
				}

				const AActor* CompositeCameraActor = CompositeActor.GetCameraActor().Get();

				if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
				{
					if (CompositeCameraActor && LevelEditorSubsystem->GetPilotLevelActor() == CompositeCameraActor)
					{
						return false;
					}
				}

				OutViewInfo.Location = ViewportClient->GetViewLocation();
				OutViewInfo.Rotation = ViewportClient->GetViewRotation();
				OutViewInfo.FOV = ViewportClient->ViewFOV;
				OutViewInfo.ProjectionMode = ECameraProjectionMode::Perspective;
				OutViewInfo.bConstrainAspectRatio = false;

				if (ViewportClient->Viewport)
				{
					const FIntPoint ViewportSize = ViewportClient->Viewport->GetSizeXY();
					OutViewInfo.AspectRatio = FMath::SafeDivide(static_cast<float>(ViewportSize.X), static_cast<float>(ViewportSize.Y));
				}

				return true;
			}
#endif //WITH_EDITOR
		}
	}
}

namespace UE
{
	namespace CompositeCore
	{
		class FPlanarReflectionPassProxy : public FCompositeCorePassProxy
		{
		public:
			IMPLEMENT_COMPOSITE_PASS(FPlanarReflectionPassProxy);

			using FCompositeCorePassProxy::FCompositeCorePassProxy;

			FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositePlanarReflection, "Composite.PlanarReflection");

				check(ValidateInputs(Inputs));

				const FResourceMetadata& Metadata = Inputs[0].Metadata;
				const FScreenPassTexture& Input = Inputs[0].Texture;
				FScreenPassRenderTarget Output = Inputs.OverrideOutput;

				if (!Output.IsValid())
				{
					Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("CompositePlanarReflection"));
				}

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
				const FScreenPassTextureViewport Viewport(Output);

				FCompositePlanarReflectionShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePlanarReflectionShader::FParameters>();
				Parameters->InputTexture = Input.Texture;
				Parameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				Parameters->ReflectionStrength = ReflectionStrength;
				Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

				TShaderMapRef<FCompositePlanarReflectionShader> PixelShader(GlobalShaderMap);
				AddDrawScreenPass(
					GraphBuilder,
					RDG_EVENT_NAME("Composite.PlanarReflection (%dx%d)", Viewport.Extent.X, Viewport.Extent.Y),
					InView,
					Viewport,
					Viewport,
					PixelShader,
					Parameters
				);

				return FPassTexture{ MoveTemp(Output), Metadata };
			}

			float ReflectionStrength = 0.5f;
		};
	}
}

UCompositeLayerPlanarReflection::UCompositeLayerPlanarReflection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RenderTargetResolution{ FCompositeRenderTargetPool::DefaultSize / 2 }
	, ReflectionStrength{ 0.5f }
	, PlaneHeight{ 0.0f }
	, bCustomRenderPass{ false }
{
	Operation = ECompositeCoreMergeOp::Screen;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		constexpr bool bEnabled = true;
		RegisterEndOfFrameUpdate(bEnabled);
	}
}

UCompositeLayerPlanarReflection::~UCompositeLayerPlanarReflection() = default;

void UCompositeLayerPlanarReflection::OnRemoved(ACompositeActor* LastOwner)
{
	if (LastOwner != nullptr)
	{
		LastOwner->DestroySceneCaptures(this);
	}
}

void UCompositeLayerPlanarReflection::BeginDestroy()
{
	Super::BeginDestroy();

	OnRemoved(GetTypedOuter<ACompositeActor>()); // Redundant remove call for safety
}

#if WITH_EDITOR
void UCompositeLayerPlanarReflection::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropertyName = PropertyAboutToChange->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Actors))
	{
		SpawnableBindings.CachePreEditState(Actors);
	}
}

void UCompositeLayerPlanarReflection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Actors))
	{
		SetActorsInternal(MoveTemp(Actors));
		SpawnableBindings.SyncOnPropertyChange(Actors, GetWorld());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bCustomRenderPass))
	{
		SetCustomRenderPass(bCustomRenderPass);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution))
	{
		SetRenderTargetResolution(RenderTargetResolution);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, ReflectionStrength))
	{
		SetReflectionStrength(ReflectionStrength);
	}
}

void UCompositeLayerPlanarReflection::PostEditUndo()
{
	Super::PostEditUndo();

	// Actors and SpawnableBindings.Bindings are both UPROPERTY on this object, so the transaction
	// system restores them atomically — the parallel-array invariant is preserved without an
	// explicit SyncOnPropertyChange. SetActorsInternal is re-run to refresh non-transactional
	// state (scene capture component show-only list) that the transaction does not capture.
	SetActorsInternal(MoveTemp(Actors));

	SetCustomRenderPass(bCustomRenderPass);
}

void UCompositeLayerPlanarReflection::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo &&
		TransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution)))
	{
		/**
		* When a resolution change is undone, the previous render targets are deserialized & replaced.
		* This leaves previous assignments invalid so we make sure to return them to the pool.
		*/
		if (UCompositeSceneCapture2DComponent* CaptureComponent = GetSceneCaptureComponent())
		{
			FCompositeRenderTargetPool::Get().ReleaseAssigneeTargets(CaptureComponent);
		}
	}
}
#endif

void UCompositeLayerPlanarReflection::OnEndOfFrameUpdate(UWorld* InWorld)
{
	SpawnableBindings.TickResolveStale(Actors, InWorld, GetUniqueID());

	/**
	* In order to retain the ability to reference actors in sublevels, we prefer using weak pointer components on planar reflections. However,
	* primitive components may often be recreated (see RerunConstructionScripts(), FStaticMeshComponentRecreateRenderStateContext, dynamic
	* text primitives, etc). To prevent issues with destroyed primitives disappearing in our planar reflections, we re-register them every frame.
	*/
	USceneCaptureComponent2D* PlanarReflectionComponent = GetSceneCaptureComponent();
	if (IsValid(PlanarReflectionComponent))
	{
		UpdateSceneCaptureComponents(*PlanarReflectionComponent, Actors);

		UpdatePlanarReflectionTransform(PlanarReflectionComponent);
	}
}

bool UCompositeLayerPlanarReflection::GetIsActive() const
{
	return Super::GetIsActive()
		&& IsValid(GetTypedOuter<ACompositeActor>())
		&& IsValid(GetSceneCaptureComponent())
		&& !Actors.IsEmpty();
}

FCompositeCorePassProxy* UCompositeLayerPlanarReflection::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	USceneCaptureComponent2D* CaptureComponent = GetSceneCaptureComponent();

	FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(CaptureComponent, CaptureComponent->TextureTarget, RenderTargetResolution);

	FResourceMetadata Metadata = {};
	Metadata.bInvertedAlpha = true;
	Metadata.bDistorted = CaptureComponent->ShowFlags.LensDistortion;
	Metadata.DebugName = TEXT("CompositePlanarReflectionTex");

	const ResourceId TexId = InContext.FindOrCreateExternalTexture(CaptureComponent->TextureTarget, Metadata);

	FPassInputDecl PassInput;
	PassInput.Set<FPassExternalResourceDesc>({ TexId });

	FPlanarReflectionPassProxy* PlanarReflectionProxy = InFrameAllocator.Create<FPlanarReflectionPassProxy>(FPassInputDeclArray{ PassInput });
	PlanarReflectionProxy->ReflectionStrength = ReflectionStrength;
	PassInput.Set<const FCompositeCorePassProxy*>(PlanarReflectionProxy);

	AddChildPasses(PassInput, InContext, InFrameAllocator, LayerPasses);

	FPassInputDeclArray Inputs;
	Inputs.SetNum(FixedNumLayerInputs);
	Inputs[0] = PassInput;
	Inputs[1] = GetDefaultSecondInput(InContext);

	return InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), GetMergeOperation(InContext), TEXT("PlanarReflection"), ELensDistortionHandling::Disabled);
}

const TArray<TSoftObjectPtr<AActor>> UCompositeLayerPlanarReflection::GetActors() const
{
	return Actors;
}

void UCompositeLayerPlanarReflection::SetActors(TArray<TSoftObjectPtr<AActor>> InActors)
{
	SpawnableBindings.CachePreEditState(Actors);
	SetActorsInternal(MoveTemp(InActors));
	SpawnableBindings.SyncOnPropertyChange(Actors, GetWorld());
}

void UCompositeLayerPlanarReflection::SetActorsInternal(TArray<TSoftObjectPtr<AActor>> InActors)
{
	Actors = MoveTemp(InActors);

	USceneCaptureComponent2D* PlanarReflectionComponent = GetSceneCaptureComponent();
	if (IsValid(PlanarReflectionComponent))
	{
		UpdateSceneCaptureComponents(*PlanarReflectionComponent, Actors);
	}
}

bool UCompositeLayerPlanarReflection::IsCustomRenderPass() const
{
	return bCustomRenderPass;
}

void UCompositeLayerPlanarReflection::SetCustomRenderPass(bool bInIsFastRenderPass)
{
	bCustomRenderPass = bInIsFastRenderPass;

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	UCompositeSceneCapture2DComponent* PlanarReflectionComponent = GetSceneCaptureComponent();
	if (IsValid(CompositeActor) && IsValid(PlanarReflectionComponent))
	{
		PlanarReflectionComponent->CaptureSource = UE::Composite::Private::GetPlanarReflectionCaptureSource(*CompositeActor, bCustomRenderPass);
		PlanarReflectionComponent->bRenderInMainRenderer = bCustomRenderPass;
	}
}

void UCompositeLayerPlanarReflection::SetRenderTargetResolution(FIntPoint InRenderTargetResolution)
{
	RenderTargetResolution.X = FMath::Max(InRenderTargetResolution.X, 1);
	RenderTargetResolution.Y = FMath::Max(InRenderTargetResolution.Y, 1);
}

void UCompositeLayerPlanarReflection::SetReflectionStrength(float InReflectionStrength)
{
	ReflectionStrength = FMath::Clamp(InReflectionStrength, 0.0f, 1.0f);
}

UCompositeSceneCapture2DComponent* UCompositeLayerPlanarReflection::GetSceneCaptureComponent() const
{
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (IsValid(CompositeActor))
	{
		UCompositeSceneCapture2DComponent* CaptureComponent = CompositeActor->FindOrCreateSceneCapture<UCompositeSceneCapture2DComponent>(this);
		if (IsValid(CaptureComponent))
		{
			CaptureComponent->CaptureSource = UE::Composite::Private::GetPlanarReflectionCaptureSource(*CompositeActor, bCustomRenderPass);
			CaptureComponent->bRenderInMainRenderer = bCustomRenderPass;
			CaptureComponent->bMainViewCamera = false;

			/**
			* Fixme: AllowPrimitiveAlphaHoldout is still used to prevent compositing while rendering
			* scene captures in FCompositeCoreSceneViewExtension, so we cannot allow it currently.
			*/
			//CaptureComponent->ShowFlags.SetAllowPrimitiveAlphaHoldout(true);
		}

		return CaptureComponent;
	}

	return nullptr;
}

void UCompositeLayerPlanarReflection::UpdatePlanarReflectionTransform(USceneCaptureComponent2D* CaptureComponent)
{
	if (!CaptureComponent)
	{
		return;
	}

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (!CompositeActor)
	{
		return;
	}

	UCameraComponent* CameraComponent = CompositeActor->GetCameraComponent();
	const UWorld* World = GetWorld();

	if (!World)
	{
		return;
	}

	FMinimalViewInfo ViewInfo;
	bool bHasEditorViewportView = false;

#if WITH_EDITOR
	// Media profile viewports provide their own main view camera source, so we skip the editor viewport fallback in that mode.
	if (CompositeActor->AllowedViewModes != ECompositeAllowedViewModes::MediaProfileUnknown)
	{
		bHasEditorViewportView = UE::Composite::Private::TryGetEditorViewportCameraView(*CompositeActor, ViewInfo);
	}
#endif

	if (!bHasEditorViewportView)
	{
		if (!CameraComponent)
		{
			return;
		}

		CameraComponent->GetCameraView(World->DeltaTimeSeconds, ViewInfo);
	}

	// Calculate reflected position (mirror across horizontal plane at PlaneHeight)
	float DistanceAboveGround = ViewInfo.Location.Z - PlaneHeight;
	ViewInfo.Location.Z = PlaneHeight - DistanceAboveGround;

	// Calculate reflected rotation (flip pitch and roll, preserve yaw)
	ViewInfo.Rotation.Pitch = -ViewInfo.Rotation.Pitch;
	ViewInfo.Rotation.Roll = -ViewInfo.Rotation.Roll;

	CaptureComponent->SetCameraView(ViewInfo);

	/**
	* Preserve the source view projection so non-matching render target aspect ratios do not distort the reflection.
	*
	* Note: While we could obtain the correct y-flip by adjusting the projection matrix, doing so
	* causes incorrect face culling order. It is therefore simpler to flip the image in our proxy shader.
	*/
	CaptureComponent->bUseCustomProjectionMatrix = true;
	CaptureComponent->CustomProjectionMatrix = ViewInfo.CalculateProjectionMatrix();
}

