// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerPlate.h"

#include "Components/StaticMeshComponent.h"
#include "CompositeActor.h"
#include "CompositeAssetUserData.h"
#include "CompositeCoreSettings.h"
#include "CompositeCVarOverrideManager.h"
#include "CompositeRenderTargetPool.h"
#include "CompositeWorldSubsystem.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "MediaTexture.h"
#include "EngineUtils.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Passes/CompositePassDistortion.h"
#include "Passes/CompositePassTranslucency.h"
#include "Passes/CompositePassTransform2D.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Editor.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectSaveContext.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/TransactionObjectEvent.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

const FLazyName UCompositeLayerPlate::CompositeTextureName = FLazyName(TEXT("CompositeTexture"));

#define LOCTEXT_NAMESPACE "Composite"

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			/** Set visibility on a primitive component, guarded to avoid unnecessary Modify() calls. */
			static void SetCompositeMeshVisibility(UPrimitiveComponent& InPrimitive, bool bNewVisibility)
			{
				if (InPrimitive.GetVisibleFlag() != bNewVisibility)
				{
					InPrimitive.Modify();
					InPrimitive.SetVisibility(bNewVisibility);
				}
			}

			/** Convenience function to handle state on removed composite meshes. Restores visibility when bInControlMeshVisibility is enabled. */
			static void HandleRemovedCompositeMeshes(TConstArrayView<TSoftObjectPtr<AActor>> InPreEditMeshes, TConstArrayView<TSoftObjectPtr<AActor>> InPostEditMeshes, bool bInControlMeshVisibility)
			{
				for (const TSoftObjectPtr<AActor>& PreEditMesh : InPreEditMeshes)
				{
					if (!PreEditMesh.IsValid())
					{
						continue;
					}

					if (!InPostEditMeshes.Contains(PreEditMesh))
					{
						for (UActorComponent* Component : PreEditMesh->GetComponents())
						{
							UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);

							if (IsValid(PrimitiveComponent))
							{
								PrimitiveComponent->RemoveUserDataOfClass(UCompositeAssetUserData::StaticClass());

								if (bInControlMeshVisibility)
								{
									SetCompositeMeshVisibility(*PrimitiveComponent, true);
								}
							}
						}
					}
				}
			}

			/** Convenience function to remove all composite asset user data from tracked mesh actors. */
			static void RemoveCompositeAllAssetUserData(TConstArrayView<TSoftObjectPtr<AActor>> InMeshes)
			{
				for (const TSoftObjectPtr<AActor>& Mesh : InMeshes)
				{
					if (!Mesh.IsValid())
					{
						continue;
					}

					for (UActorComponent* Component : Mesh->GetComponents())
					{
						if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
						{
							if (IsValid(PrimitiveComponent))
							{
								PrimitiveComponent->RemoveUserDataOfClass(UCompositeAssetUserData::StaticClass());
							}
						}
					}
				}
			}
			
			FCompositeMeshPrimitiveReference* FindPrimitiveReferences(TArray<FCompositeMeshPrimitiveReference>& PrimitiveReferences, const TSoftObjectPtr<AActor>& InActor)
			{
				return PrimitiveReferences.FindByPredicate([&InActor](const FCompositeMeshPrimitiveReference& Item)
				{
					return Item.CompositeMeshActor == InActor;
				});
			}
			
			const FCompositeMeshPrimitiveReference* FindPrimitiveReferences(const TArray<FCompositeMeshPrimitiveReference>& PrimitiveReferences, const TSoftObjectPtr<AActor>& InActor)
			{
				return PrimitiveReferences.FindByPredicate([&InActor](const FCompositeMeshPrimitiveReference& Item)
				{
					return Item.CompositeMeshActor == InActor;
				});
			}
			
			void CleanUpPrimitiveReference(FCompositeMeshPrimitiveReference& PrimitiveReference)
			{
				// Don't try to clean up any soft pointers that aren't explicitly loaded
				if (!PrimitiveReference.CompositeMeshActor.IsValid())
				{
					return;
				}
				
				TInlineComponentArray<UPrimitiveComponent*> PrimComponents(PrimitiveReference.CompositeMeshActor.Get());
				PrimitiveReference.CompositeMeshPrimitives.RemoveAll([&PrimComponents](const TSoftObjectPtr<UPrimitiveComponent>& Primitive)
				{
					return !Primitive.IsValid() || !PrimComponents.Contains(Primitive.Get());
				});
			}
			
			/** Return the texture size, depending on the texture type. */
			static FIntPoint GetTextureSize(const UTexture* InTexture)
			{
				if (const UMediaTexture* MediaTexture = Cast<UMediaTexture>(InTexture))
				{
					const int32 Width = MediaTexture->GetWidth();
					const int32 Height = MediaTexture->GetHeight();

					if (Width > 0 && Height > 0)
					{
						return FIntPoint(Width, Height);
					}
				}
				else if (const UTexture2D* Texture2D = Cast<UTexture2D>(InTexture))
				{
					return FIntPoint(Texture2D->GetSizeX(), Texture2D->GetSizeY());
				}
				else if (const UTexture2DDynamic* Texture2DDynamic = Cast<UTexture2DDynamic>(InTexture))
				{
					return FIntPoint(Texture2DDynamic->SizeX, Texture2DDynamic->SizeY);
				}
				else if (const UTextureRenderTarget2D* RenderTarget2D = Cast<UTextureRenderTarget2D>(InTexture))
				{
					return FIntPoint(RenderTarget2D->SizeX, RenderTarget2D->SizeY);
				}

				return FCompositeRenderTargetPool::DefaultSize;
			}
		}
	}
}

UCompositeLayerPlate::UCompositeLayerPlate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static ConstructorHelpers::FObjectFinder<UTexture2D> DefaultMediaTexture = TEXT("/Composite/Textures/T_Composite_SMPTE_Color_Bars_16x9.T_Composite_SMPTE_Color_Bars_16x9");
	
	Texture = Cast<UTexture>(DefaultMediaTexture.Object);
	bIsHoldoutEnabled = true;
	bControlMeshVisibility = true;
	PlateMode = ECompositePlateMode::CompositeMesh;
	Operation = ECompositeCoreMergeOp::Over;

	// Default undistort pass for plate media going into the scene
	ScenePasses.Add(CreateDefaultSubobject<UCompositePassDistortion>(TEXT("CompositeUndistortScenePass")));

	// Internal pass to automatically remove overscan
	OverscanPass = CreateDefaultSubobject<UCompositePassTransform2D>(TEXT("CompositeOverscanPass"));
	OverscanPass->ScaleMode = ECompositePassScaleMode::None;
	OverscanPass->bRemoveOverscan = true;

#if WITH_EDITOR
	// If we are not the class default object...
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Hook into pre/post save.
		FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &UCompositeLayerPlate::OnPreSaveWorld);
		FEditorDelegates::PostSaveWorldWithContext.AddUObject(this, &UCompositeLayerPlate::OnPostSaveWorld);
	}
#endif // WITH_EDITOR
}

UCompositeLayerPlate::~UCompositeLayerPlate() = default;

void UCompositeLayerPlate::SetIsEnabled(bool bInEnabled)
{
	Super::SetIsEnabled(bInEnabled);

	PropagateStateChange(bInEnabled ? ECompositeStateChangeType::Activate : ECompositeStateChangeType::Deactivate, GetWorld());
}

void UCompositeLayerPlate::Tick(float DeltaTime)
{
	if (!IsRendering())
	{
		return;
	}

	if (SpawnableBindings.TickResolveStale(CompositeMeshes, GetTickableGameObjectWorld(), GetUniqueID()))
	{
		UpdateCompositeMeshes();
	}

	// Update holdout on tick in case composite meshes dynamically recreate their primitives
	TryRegisterCompositePrimitives(GetPrimitives());

	// Constantly check if passes have changed to manage render targets & texture bindings on composite meshes.
	const int32 NumValidMediaPasses = GetValidPassesNum(MediaPasses)
		+ ((bRemoveOverscan && IsValid(OverscanPass) && OverscanPass->GetIsActive()) ? 1 : 0);
	const int32 NumValidScenePasses = GetValidPassesNum(ScenePasses);
	bool bPassesHaveChanged = false;

	if (NumValidMediaPasses != CachedValidMediaPasses)
	{
		if (NumValidMediaPasses == 0)
		{
			FCompositeRenderTargetPool::Get().ReleaseTarget(MediaRenderTarget);
		}

		CachedValidMediaPasses = NumValidMediaPasses;
		bPassesHaveChanged = true;
	}

	if (NumValidScenePasses != CachedValidScenePasses)
	{
		if (NumValidScenePasses == 0)
		{
			FCompositeRenderTargetPool::Get().ReleaseTarget(SceneRenderTarget);
		}

		CachedValidScenePasses = NumValidScenePasses;
		bPassesHaveChanged = true;
	}

	if (bPassesHaveChanged || bRebindRenderTarget)
	{
		bRebindRenderTarget = false;
		UpdateCompositeMeshes();
	}
}

ETickableTickType UCompositeLayerPlate::GetTickableTickType() const
{
	return HasAnyFlags(RF_ClassDefaultObject) ? ETickableTickType::Never : ETickableTickType::Conditional;
}

bool UCompositeLayerPlate::IsTickable() const
{
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();

	// Only tick when instances are registered to a non-CDO composite actor.
	return IsValid(CompositeActor) && !CompositeActor->HasAnyFlags(RF_ClassDefaultObject);
}

UWorld* UCompositeLayerPlate::GetTickableGameObjectWorld() const
{
	return GetWorld();
}

TStatId UCompositeLayerPlate::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UCompositeLayerPlate, STATGROUP_Tickables);
}

void UCompositeLayerPlate::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdateCompositeMeshes();
		TryOpenMediaProfileSource();
	}
}

void UCompositeLayerPlate::OnRemoved(ACompositeActor* LastOwner)
{
	FCompositeRenderTargetPool::Get().ReleaseTarget(MediaRenderTarget);
	FCompositeRenderTargetPool::Get().ReleaseTarget(SceneRenderTarget);

	TryCloseMediaProfileSource();

	PropagateStateChange(ECompositeStateChangeType::Release, LastOwner ? LastOwner->GetWorld() : nullptr);
}

void UCompositeLayerPlate::OnRenderingStateChange(ECompositeStateChangeType ChangeType)
{
	PropagateStateChange(ChangeType, GetWorld());
}

void UCompositeLayerPlate::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	// Only remove delegates on final destruction to preserve them for undo operations.
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
		FEditorDelegates::PostSaveWorldWithContext.RemoveAll(this);
	}
#endif // WITH_EDITOR

	OnRemoved(GetTypedOuter<ACompositeActor>()); // Redundant remove call for safety
}

const TArray<TSoftObjectPtr<AActor>> UCompositeLayerPlate::GetCompositeMeshes() const
{
	return CompositeMeshes;
}

void UCompositeLayerPlate::SetCompositeMeshes(TArray<TSoftObjectPtr<AActor>> InCompositeMeshes)
{
	SpawnableBindings.CachePreEditState(CompositeMeshes);
	SetCompositeMeshesInternal(MoveTemp(InCompositeMeshes));
	SpawnableBindings.SyncOnPropertyChange(CompositeMeshes, GetWorld());
}

void UCompositeLayerPlate::SetCompositeMeshesInternal(TArray<TSoftObjectPtr<AActor>> InCompositeMeshes)
{
	// First, we untrack current meshes
	UE::Composite::Private::HandleRemovedCompositeMeshes(CompositeMeshes, InCompositeMeshes, bControlMeshVisibility);

	CompositeMeshes = MoveTemp(InCompositeMeshes);

	CompositeMeshPrimitives.Empty();
	for (const TSoftObjectPtr<AActor>& Actor : CompositeMeshes)
	{
		FCompositeMeshPrimitiveReference NewPrimitiveReference;
		NewPrimitiveReference.CompositeMeshActor = Actor;

		CompositeMeshPrimitives.Add(NewPrimitiveReference);
	}

	UpdateCompositeMeshes();
}

void UCompositeLayerPlate::SetPlateMode(ECompositePlateMode InPlateMode)
{
	PlateMode = InPlateMode;
}

void UCompositeLayerPlate::SetIsHoldoutEnabled(bool bInIsHoldoutEnabled)
{
	bIsHoldoutEnabled = bInIsHoldoutEnabled;

	if (!bIsHoldoutEnabled)
	{
		UnregisterCompositePrimitives(GetPrimitives());
	}
}

void UCompositeLayerPlate::SetControlMeshVisibility(bool bInControlMeshVisibility)
{
	bControlMeshVisibility = bInControlMeshVisibility;

	if (!bControlMeshVisibility)
	{
		for (UPrimitiveComponent* Primitive : GetPrimitives())
		{
			UE::Composite::Private::SetCompositeMeshVisibility(*Primitive, true);
		}
	}
}

UTexture* UCompositeLayerPlate::GetCompositeTexture() const
{
	if (CachedValidScenePasses > 0)
	{		
		return GetOrCreateRenderTarget(SceneRenderTarget).Get();
	}
	else if (CachedValidMediaPasses > 0)
	{
		return GetOrCreateRenderTarget(MediaRenderTarget).Get();
	}
	else if (IsValid(Texture))
	{
		return Texture.Get();
	}

	return nullptr;
}

#if WITH_EDITOR
bool UCompositeLayerPlate::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);

	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bIsHoldoutEnabled))
		{
			if (CompositeMeshes.IsEmpty())
			{
				bIsEditable = false;
			}
		}
	}

	return bIsEditable;
}

void UCompositeLayerPlate::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (!PropertyThatWillChange)
	{
		return;
	}

	const FName PropertyName = PropertyThatWillChange->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, CompositeMeshes))
	{
		PreEditCompositeMeshes = CompositeMeshes;
		SpawnableBindings.CachePreEditState(CompositeMeshes);

		UnregisterCompositePrimitives(GetPrimitives());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Texture))
	{
		TryCloseMediaProfileSource();
	}
}

void UCompositeLayerPlate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bIsEnabled))
	{
		SetIsEnabled(bIsEnabled);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, CompositeMeshes))
	{
		const bool bChangeWasRemoval = PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove
			|| PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear;

		if (bChangeWasRemoval)
		{
			CompositeMeshPrimitives.RemoveAll([this](const FCompositeMeshPrimitiveReference& Item)
			{
				return !CompositeMeshes.Contains(Item.CompositeMeshActor);
			});

			UE::Composite::Private::HandleRemovedCompositeMeshes(PreEditCompositeMeshes, CompositeMeshes, bControlMeshVisibility);
		}
		else
		{
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd ||
				PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet ||
				PropertyChangedEvent.ChangeType == EPropertyChangeType::ResetToDefault)
			{
				const int32 AlteredIndex = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
				if (CompositeMeshes.IsValidIndex(AlteredIndex))
				{
					TSoftObjectPtr<AActor>& NewCompositeMesh = CompositeMeshes[AlteredIndex];

					if (IsCompositeMeshActorAlreadyInUse(NewCompositeMesh))
					{
						FNotificationInfo NotifyInfo(LOCTEXT("CompositeMeshAlreadyInUse", "The composite mesh is already in use by another layer."));
						NotifyInfo.ExpireDuration = 4.0f;
						FSlateNotificationManager::Get().AddNotification(NotifyInfo)->SetCompletionState(SNotificationItem::CS_Fail);

						NewCompositeMesh = nullptr;
					}
					else
					{
						FCompositeMeshPrimitiveReference* PrimitiveReference = UE::Composite::Private::FindPrimitiveReferences(CompositeMeshPrimitives, NewCompositeMesh);
						if (!PrimitiveReference)
						{
							FCompositeMeshPrimitiveReference NewPrimitiveReference;
							NewPrimitiveReference.CompositeMeshActor = NewCompositeMesh;
							
							CompositeMeshPrimitives.Add(NewPrimitiveReference);
						}
					}
				}
			}
		}

		SpawnableBindings.SyncOnPropertyChange(CompositeMeshes, GetWorld());

		// Note: While relying on tick would yield the same result, we want to capture the holdout state change in transactions.
		TryRegisterCompositePrimitives(GetPrimitives());

		PreEditCompositeMeshes.Reset();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Texture))
	{
		TryOpenMediaProfileSource();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bIsHoldoutEnabled))
	{
		SetIsHoldoutEnabled(bIsHoldoutEnabled);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bControlMeshVisibility))
	{
		SetControlMeshVisibility(bControlMeshVisibility);
	}

	// For simplicity, we make sure to update composite meshes on any property change.
	UpdateCompositeMeshes();
}

void UCompositeLayerPlate::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);

	if (InTransactionEvent.HasPropertyChanges())
	{
		const TArray<FName>& ChangedProperties = InTransactionEvent.GetChangedProperties();

		if( ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(ThisClass, Texture)))
		{
			if (InTransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
			{
				// Special processing for undo/redo as PreEdit and PostEdit are not called during undo/redo
				// Attempt to close any outstanding media profile sources this plate may have opened for Texture,
				// and then attempt to open the corresponding media source if Texture is a media profile media texture
				TryCloseMediaProfileSource();
				TryOpenMediaProfileSource();

				/**
				* When a resolution change is undone, the previous render targets are deserialized & replaced.
				* This leaves previous assignments invalid so we make sure to return them to the pool.
				*/
				FCompositeRenderTargetPool::Get().ReleaseAssigneeTargets(this);
			}
			
			UpdateCompositeMeshes();
		}
		else if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(ThisClass, CompositeMeshes)))
		{
			UpdateCompositeMeshes();
		}
	}
}
#endif


int32 UCompositeLayerPlate::FindLastValidPassIndex(TArrayView<const TObjectPtr<UCompositePassBase>> InPasses) const
{
	int32 LastValidPassIndex = INDEX_NONE;
	
	// Iterate backwards so that lower passes in the UI are executed first
	for (int32 PassIndex = InPasses.Num() - 1; PassIndex >= 0; --PassIndex)
	{
		const TObjectPtr<UCompositePassBase>& Pass = InPasses[PassIndex];

		if (IsValid(Pass) && Pass->GetIsActive())
		{
			LastValidPassIndex = PassIndex;
		}
	}

	return LastValidPassIndex;
}

UE::CompositeCore::ResourceId UCompositeLayerPlate::AddPreprocessingPasses(
	FCompositeTraversalContext& InContext,
	FSceneRenderingBulkObjectAllocator& InFrameAllocator,
	TArrayView<const TObjectPtr<UCompositePassBase>> InPasses,
	UE::CompositeCore::ResourceId TextureId,
	UE::CompositeCore::ResourceId OriginalTextureId,
	TFunction<TObjectPtr<UTextureRenderTarget2D>()> GetRenderTargetFn
) const
{
	using namespace UE::CompositeCore;

	if (InPasses.IsEmpty())
	{
		return TextureId;
	}

	ResourceId OutputTextureId = TextureId;
	bool bIsFirstValidPass = true;
	const int32 LastValidPassIndex = FindLastValidPassIndex(InPasses);

	// Iterate backwards so that lower passes in the UI are executed first
	for (int32 PassIndex = InPasses.Num() - 1; PassIndex >= 0; --PassIndex)
	{
		const TObjectPtr<UCompositePassBase>& Prepass = InPasses[PassIndex];

		if (!IsValid(Prepass) || !Prepass->GetIsActive())
		{
			continue;
		}

		FPassInputDecl PassInput;

		if (bIsFirstValidPass)
		{
			PassInput = MakeExternalInput(TextureId);
			bIsFirstValidPass = false;
		}
		else
		{
			// Default input from previous pass
			PassInput = MakeInternalInput();
		}

		FCompositeCorePassProxy* PrepassProxy = Prepass->GetProxy(PassInput, InContext, InFrameAllocator);
		if (PrepassProxy != nullptr)
		{
			if (PassIndex == LastValidPassIndex)
			{
				FResourceMetadata Metadata = {};
				Metadata.bDistorted = true;
				Metadata.DebugName = TEXT("CompositePlateTex");
				if (IsValid(Texture))
				{
					Metadata.Filter = (Texture->Filter == TF_Nearest) ? SF_Point : SF_Bilinear;
				}

				OutputTextureId = InContext.FindOrCreateExternalTexture(GetRenderTargetFn(), Metadata);

				PrepassProxy->DeclarePrimaryOutputOverride(OutputTextureId);
			}

			InContext.PreprocessingPasses.FindOrAdd(OriginalTextureId).Add(PrepassProxy);
		}
	}

	return OutputTextureId;
}

FCompositeCorePassProxy* UCompositeLayerPlate::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;
	using namespace UE::Composite::Private;

	// Default to engine black dummy texture
	ResourceId TextureId = ResourceId::BuiltInEmpty;

	if (IsValid(Texture))
	{
		// Register source texture into the frame work context (so that post-processing passes can refer to it).
		FResourceMetadata Metadata = {};
		Metadata.bDistorted = true;
		Metadata.DebugName = TEXT("CompositePlateTex");
		Metadata.Filter = (Texture->Filter == TF_Nearest) ? SF_Point : SF_Bilinear;

		TextureId = InContext.FindOrCreateExternalTexture(Texture, Metadata);
		
		const ResourceId OriginalTextureId = TextureId;
		
		const bool bImplicitOverscanRemoval = bRemoveOverscan && IsValid(OverscanPass) && OverscanPass->GetIsActive();

		TGuardValue<bool> PreprocessingGuard(InContext.bIsPreprocessing, true);

		if (bImplicitOverscanRemoval)
		{
			// Build combined media pass array: overscan (implicit) + user passes.
			// Passes iterate backwards (highest index first), so overscan at the end executes first.
			TArray<TObjectPtr<UCompositePassBase>> AllMediaPasses;
			AllMediaPasses.Reserve(MediaPasses.Num() + 1);
			AllMediaPasses.Append(MediaPasses);
			AllMediaPasses.Add(OverscanPass);

			TextureId = AddPreprocessingPasses(InContext, InFrameAllocator, AllMediaPasses, TextureId, OriginalTextureId,
				[this]() -> TObjectPtr<UTextureRenderTarget2D>
				{
					return GetOrCreateRenderTarget(MediaRenderTarget);
				}
			);
		}
		else
		{
			// Add media texture pre-processing passes
			TextureId = AddPreprocessingPasses(InContext, InFrameAllocator, MediaPasses, TextureId, OriginalTextureId,
				[this]() -> TObjectPtr<UTextureRenderTarget2D>
				{
					return GetOrCreateRenderTarget(MediaRenderTarget);
				}
			);
		}

		// Add scene-only passes. We ignore returned resource since we don't refer to it in post-processing.
		AddPreprocessingPasses(InContext, InFrameAllocator, ScenePasses, TextureId, OriginalTextureId,
			[this]() -> TObjectPtr<UTextureRenderTarget2D>
			{
				return GetOrCreateRenderTarget(SceneRenderTarget);
			}
		);
	}

	const bool bActiveCompositeMeshes = GetValidPrimitivesNum() > 0;

	FPassInputDecl PassInput;

	if (bActiveCompositeMeshes)
	{
		switch (PlateMode)
		{
		case ECompositePlateMode::Texture:
			{
				// For accurate blending through the holdout mask, the built-in custom render pass
				// for composite meshes already unpremultiplies the alpha and makes it opaque.
				// 
				// We replicate this behavior here automatically with an additional alpha transform pass
				// when composite meshes are in use, and the PlateMode is set to "Texture".
				FTranslucencyPassProxy* AlphaPassProxy = InFrameAllocator.Create<FTranslucencyPassProxy>(FPassInputDeclArray{ MakeExternalInput(TextureId) });
				AlphaPassProxy->PremultOp = ECompositeAlphaPremultiplication::Unpremultiply;
				AlphaPassProxy->OverrideOutputAlpha = ECompositeAlphaOverride::One;

				PassInput = MakePassInput(AlphaPassProxy);
			}
			break;
		
		case ECompositePlateMode::CompositeMesh:
			// Prioritize the built-in custom render pass when enabled over TextureId
			PassInput = MakeExternalInput(ResourceId::BuiltInCRP);
			break;
		
		default:
			checkNoEntry();
			break;
		}
	}
	else
	{
		PassInput = MakeExternalInput(TextureId);
	}

	AddChildPasses(PassInput, InContext, InFrameAllocator, LayerPasses);

	FPassInputDeclArray Inputs;
	Inputs.SetNum(FixedNumLayerInputs);
	Inputs[0] = PassInput;
	Inputs[1] = GetDefaultSecondInput(InContext);

	return InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), GetMergeOperation(InContext), TEXT("Plate"), ELensDistortionHandling::Enabled, bUltimatteBlend);
}

void UCompositeLayerPlate::AppendIncludedPrimitivesForActor(const TSoftObjectPtr<AActor>& InActor, TArray<UPrimitiveComponent*>& OutPrimitiveComponents) const
{
	if (const FCompositeMeshPrimitiveReference* PrimitiveReference = UE::Composite::Private::FindPrimitiveReferences(CompositeMeshPrimitives, InActor))
	{
		for (const TSoftObjectPtr<UPrimitiveComponent>& PrimitiveComponent : PrimitiveReference->CompositeMeshPrimitives)
		{
			if (PrimitiveComponent.IsValid())
			{
				OutPrimitiveComponents.Add(PrimitiveComponent.Get());
			}
		}
	}
}

void UCompositeLayerPlate::AppendPrimitivesForActor(const TSoftObjectPtr<AActor>& InActor, TArray<UPrimitiveComponent*>& OutPrimitiveComponents) const
{
	const int32 StartNum = OutPrimitiveComponents.Num();
	AppendIncludedPrimitivesForActor(InActor, OutPrimitiveComponents);

	if (OutPrimitiveComponents.Num() == StartNum)
	{
		// No specific primitives included — fall back to all valid primitives for the actor
		const UCompositeCorePluginSettings* Settings = GetDefault<UCompositeCorePluginSettings>();
		check(Settings);

		TInlineComponentArray<UPrimitiveComponent*> PrimComponents(InActor.Get());

		for (UPrimitiveComponent* PrimitiveComponent : PrimComponents)
		{
			if (Settings->IsAllowedPrimitiveClass(PrimitiveComponent))
			{
				OutPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
	}
}

TArray<UPrimitiveComponent*> UCompositeLayerPlate::GetPrimitives() const
{
	TArray<UPrimitiveComponent*> OutPrimitiveComponents;

	for (const TSoftObjectPtr<AActor>& Actor : CompositeMeshes)
	{
		if (!Actor.IsValid())
		{
			continue;
		}

		AppendPrimitivesForActor(Actor, OutPrimitiveComponents);
	}

	return OutPrimitiveComponents;
}

TArray<UPrimitiveComponent*> UCompositeLayerPlate::GetPrimitivesForActor(TSoftObjectPtr<AActor> InActor) const
{
	if (!InActor.IsValid() || !CompositeMeshes.Contains(InActor))
	{
		return TArray<UPrimitiveComponent*>();
	}

	TArray<UPrimitiveComponent*> OutPrimitiveComponents;
	AppendPrimitivesForActor(InActor, OutPrimitiveComponents);
	return OutPrimitiveComponents;
}

TArray<UPrimitiveComponent*> UCompositeLayerPlate::GetIncludedPrimitivesForActor(TSoftObjectPtr<AActor> InActor) const
{
	if (!InActor.IsValid() || !CompositeMeshes.Contains(InActor))
	{
		return TArray<UPrimitiveComponent*>();
	}

	TArray<UPrimitiveComponent*> OutPrimitiveComponents;
	AppendIncludedPrimitivesForActor(InActor, OutPrimitiveComponents);
	return OutPrimitiveComponents;
}

void UCompositeLayerPlate::SetPrimitiveIncluded(UPrimitiveComponent* InPrimitive, bool bInIncluded)
{
	AActor* PrimitiveOwner = InPrimitive->GetOwner();
	if (!CompositeMeshes.Contains(PrimitiveOwner))
	{
		return;
	}
	
	FCompositeMeshPrimitiveReference* PrimitiveReference = UE::Composite::Private::FindPrimitiveReferences(CompositeMeshPrimitives, PrimitiveOwner);
	if (!PrimitiveReference)
	{
		FCompositeMeshPrimitiveReference NewPrimitiveReference;
		NewPrimitiveReference.CompositeMeshActor = PrimitiveOwner;
		
		const int32 Index = CompositeMeshPrimitives.Add(NewPrimitiveReference);
		PrimitiveReference = &CompositeMeshPrimitives[Index];
	}
	
	const UCompositeCorePluginSettings* Settings = GetDefault<UCompositeCorePluginSettings>();
	check(Settings);
	if (!Settings->IsAllowedPrimitiveClass(InPrimitive))
	{
		return;
	}
	
	// If there are no included primitives in the list before adding the new primitive, we must unregister all the actor's valid primitives
	if (bInIncluded && PrimitiveReference->CompositeMeshPrimitives.IsEmpty())
	{
		TArray<UPrimitiveComponent*> AllPrimitives = GetPrimitivesForActor(PrimitiveOwner);
		UnregisterCompositePrimitives(AllPrimitives);
		for (UPrimitiveComponent* Primitive : AllPrimitives)
		{
			PrimitiveReference->CompositeMeshPrimitives.Remove(Primitive);

			Primitive->RemoveUserDataOfClass(UCompositeAssetUserData::StaticClass());

			if (bControlMeshVisibility)
			{
				UE::Composite::Private::SetCompositeMeshVisibility(*Primitive, true);
			}
		}
	}
	
	if (bInIncluded && !PrimitiveReference->CompositeMeshPrimitives.Contains(InPrimitive))
	{
		PrimitiveReference->CompositeMeshPrimitives.Add(InPrimitive);
		
		UCompositeAssetUserData* AssetUserData = NewObject<UCompositeAssetUserData>(InPrimitive);
		AssetUserData->OnPostEditChangeOwner.BindUObject(this, &UCompositeLayerPlate::UpdatePrimitiveComponent);
		InPrimitive->AddAssetUserData(AssetUserData);
		TryRegisterCompositePrimitives({ InPrimitive });
		UpdatePrimitiveComponent(*InPrimitive);
	}
	else if (!bInIncluded && PrimitiveReference->CompositeMeshPrimitives.Contains(InPrimitive))
	{
		UnregisterCompositePrimitives({ InPrimitive });
		PrimitiveReference->CompositeMeshPrimitives.Remove(InPrimitive);

		InPrimitive->RemoveUserDataOfClass(UCompositeAssetUserData::StaticClass());

		if (bControlMeshVisibility)
		{
			UE::Composite::Private::SetCompositeMeshVisibility(*InPrimitive, true);
		}
	}
	
	// If there are no primitives specifically included, all primitives in the actor are used as composite mesh primitives, so configure all of them
	if (!bInIncluded && PrimitiveReference->CompositeMeshPrimitives.IsEmpty())
	{
		TArray<UPrimitiveComponent*> AllPrimitives = GetPrimitivesForActor(PrimitiveOwner);
		TryRegisterCompositePrimitives(AllPrimitives);
		for (UPrimitiveComponent* Primitive : AllPrimitives)
		{
			UCompositeAssetUserData* AssetUserData = NewObject<UCompositeAssetUserData>(Primitive);
			AssetUserData->OnPostEditChangeOwner.BindUObject(this, &UCompositeLayerPlate::UpdatePrimitiveComponent);
			Primitive->AddAssetUserData(AssetUserData);
			UpdatePrimitiveComponent(*Primitive);
		}
	}
}

int32 UCompositeLayerPlate::GetValidPrimitivesNum() const
{
	const UCompositeCorePluginSettings* Settings = GetDefault<UCompositeCorePluginSettings>();
	check(Settings);

	int32 Count = 0;

	for (const TSoftObjectPtr<AActor>& Actor : CompositeMeshes)
	{
		if (!Actor.IsValid())
		{
			continue;
		}
		
		// If specific primitives are included for this actor, count only those; otherwise count all valid primitives.
		// An existing entry with an empty list means "use all" (consistent with GetPrimitivesForActor).
		const FCompositeMeshPrimitiveReference* PrimitiveReference = UE::Composite::Private::FindPrimitiveReferences(CompositeMeshPrimitives, Actor);
		if (PrimitiveReference && !PrimitiveReference->CompositeMeshPrimitives.IsEmpty())
		{
			for (const TSoftObjectPtr<UPrimitiveComponent>& PrimitiveComponent : PrimitiveReference->CompositeMeshPrimitives)
			{
				if (Settings->IsAllowedPrimitiveClass(PrimitiveComponent.Get()))
				{
					++Count;
				}
			}
		}
		else
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents(Actor.Get());

			for (UPrimitiveComponent* PrimitiveComponent : PrimComponents)
			{
				if (Settings->IsAllowedPrimitiveClass(PrimitiveComponent))
				{
					++Count;
				}
			}
		}
	}

	return Count;
}

int32 UCompositeLayerPlate::GetValidPassesNum(TArrayView<const TObjectPtr<UCompositePassBase>> InPasses) const
{
	int32 Count = 0;

	for (int32 Index = 0; Index < InPasses.Num(); ++Index)
	{
		const TObjectPtr<UCompositePassBase>& Pass = InPasses[Index];

		if (IsValid(Pass) && Pass->GetIsActive())
		{
			++Count;
		}
	}

	return Count;
}

void UCompositeLayerPlate::TryOpenMediaProfileSource()
{
	UMediaTexture* MediaTexture = Cast<UMediaTexture>(Texture);
	if (!MediaTexture)
	{
		return;
	}
	
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return;
	}
	
	int32 MediaSourceIndex = INDEX_NONE;
	if (ActiveMediaProfile->GetPlaybackManager()->IsValidSourceMediaTexture(MediaTexture, MediaSourceIndex))
	{
		ActiveMediaProfile->GetPlaybackManager()->OpenSourceFromIndex(MediaSourceIndex, this);
	}
}

void UCompositeLayerPlate::TryCloseMediaProfileSource()
{
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return;
	}

	UMediaProfilePlaybackManager::FCloseSourceArgs Args;
	Args.Consumer = this;
				
	ActiveMediaProfile->GetPlaybackManager()->CloseSourcesForConsumer(Args);
}

bool UCompositeLayerPlate::IsRendering() const
{
	if (GetIsEnabled())
	{
		const ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();

		return IsValid(CompositeActor) && CompositeActor->IsRendering();
	}

	return false;
}

#if WITH_EDITOR
void UCompositeLayerPlate::OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext)
{
	// We need to remove our asset user data before saving, as we do not need to save it out
	// and only use it to know when the static mesh component changes.
	UE::Composite::Private::RemoveCompositeAllAssetUserData(CompositeMeshes);

	// Copy the existing CompositeMeshPrimitives to preserve any non-valid soft pointers. We do this to
	// avoid having to save invalid soft pointers to primitive components while keeping them in memory so
	// that actions like undoing the deletion of a primitive on a composite mesh actor can be preserved
	PreSaveCompositeMeshPrimitives = CompositeMeshPrimitives;

	// Clean up any invalid soft pointers to primitives in the list of composite mesh primitives
	for (FCompositeMeshPrimitiveReference& PrimitiveReference : CompositeMeshPrimitives)
	{
		UE::Composite::Private::CleanUpPrimitiveReference(PrimitiveReference);
	}
}

void UCompositeLayerPlate::OnPostSaveWorld(UWorld* InWorld, FObjectPostSaveContext ObjectSaveContext)
{
	CompositeMeshPrimitives = PreSaveCompositeMeshPrimitives;
	PreSaveCompositeMeshPrimitives.Empty();

	constexpr bool bRegisterOnly = true;
	UpdateCompositeMeshes(bRegisterOnly);
}
#endif

void UCompositeLayerPlate::OverrideTranslucencyConsoleVariables(bool bOverride) const
{
	static TSet<const UCompositeLayerPlate*> ActiveOverrideLayer;

	const TCHAR* CVarAutoBeforeDOFName = TEXT("r.Translucency.AutoBeforeDOF");
	const TCHAR* CVarHoldoutLocationName = TEXT("r.Translucency.Holdout.Location");

	if (bOverride)
	{
		if (ActiveOverrideLayer.IsEmpty())
		{
			FCompositeCVarOverrideManager::Get().Override(CVarAutoBeforeDOFName, -1.0f);
			FCompositeCVarOverrideManager::Get().Override(CVarHoldoutLocationName, 1);
		}

		ActiveOverrideLayer.Add(this);
	}
	else
	{
		const int32 NumRemoved = ActiveOverrideLayer.Remove(this);

		if (NumRemoved > 0 && ActiveOverrideLayer.IsEmpty())
		{
			FCompositeCVarOverrideManager::Get().Restore(CVarAutoBeforeDOFName);
			FCompositeCVarOverrideManager::Get().Restore(CVarHoldoutLocationName);
		}
	}
}

bool UCompositeLayerPlate::HasAfterDOFTranslucentMaterial() const
{
	const TArray<UPrimitiveComponent*> Primitives = GetPrimitives();
	for (const UPrimitiveComponent* Primitive : Primitives)
	{
		if (!IsValid(Primitive))
		{
			continue;
		}

		const int32 NumMaterials = Primitive->GetNumMaterials();
		for (int32 ElementIndex = 0; ElementIndex < NumMaterials; ++ElementIndex)
		{
			UMaterialInterface* MaterialInterface = Primitive->GetMaterial(ElementIndex);
			if (IsValid(MaterialInterface) && IsUsingAfterDOFTranslucency(MaterialInterface))
			{
				return true;
			}
		}
	}

	return false;
}

void UCompositeLayerPlate::UpdateTranslucencyOverride() const
{
	const bool bNeedsOverride = IsRendering() && HasAfterDOFTranslucentMaterial();
	OverrideTranslucencyConsoleVariables(bNeedsOverride);
}

bool UCompositeLayerPlate::IsUsingAfterDOFTranslucency(UMaterialInterface* MaterialInterface) const
{
	if (IsTranslucentBlendMode(*MaterialInterface))
	{
		if (FMaterialResource* Resource = MaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform))
		{
			if (Resource->IsTranslucencyAfterDOFEnabled())
			{
				return true;
			}
		}
	}

	return false;
}

void UCompositeLayerPlate::TryRegisterCompositePrimitives(const TArray<UPrimitiveComponent*>& InPrimitives, const UWorld* SourceWorld) const
{
	if (!IsRendering())
	{
		return;
	}

	if (bIsHoldoutEnabled)
	{
		for (UPrimitiveComponent* Primitive : InPrimitives)
		{
			if (Primitive && !Primitive->bHoldout)
			{
				Primitive->Modify();
				Primitive->SetHoldout(true);
			}
		}
	}

	if (PlateMode == ECompositePlateMode::CompositeMesh)
	{
		UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(SourceWorld ? SourceWorld : GetWorld());
		if (Subsystem)
		{
			// Primitives to be marked as holdout and rendered in the built-in composite custom render pass.
			Subsystem->RegisterPrimitives(InPrimitives, ECompositeCoreHoldoutManagement::None);
		}
	}
}

void UCompositeLayerPlate::UnregisterCompositePrimitives(const TArray<UPrimitiveComponent*>& InPrimitives, const UWorld* SourceWorld) const
{	
	// Explicitely disable holdout since the subsystem below is only used in ECompositePlateMode::CompositeMesh. 
	for (UPrimitiveComponent* Primitive : InPrimitives)
	{
		if (Primitive && Primitive->bHoldout)
		{
			Primitive->Modify();
			Primitive->SetHoldout(false);
		}
	}

	// Always unregister no matter the current PlateMode
	UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(SourceWorld ? SourceWorld : GetWorld());
	if (IsValid(Subsystem))
	{
		Subsystem->UnregisterPrimitives(InPrimitives, ECompositeCoreHoldoutManagement::None);
	}
}

void UCompositeLayerPlate::UpdateCompositeMeshes(bool bRegisterOnly) const
{
	for (const TSoftObjectPtr<AActor>& MeshActor : CompositeMeshes)
	{
		if (!MeshActor.IsValid())
		{
			continue;
		}

		TArray<UPrimitiveComponent*> Primitives = GetPrimitivesForActor(MeshActor);
		for (UPrimitiveComponent* PrimitiveComponent : Primitives)
		{
			// Track meshes so that we can update their material upon changes
			if (!PrimitiveComponent->HasAssetUserDataOfClass(UCompositeAssetUserData::StaticClass()))
			{
				UCompositeAssetUserData* AssetUserData = NewObject<UCompositeAssetUserData>(PrimitiveComponent);
				AssetUserData->OnPostEditChangeOwner.BindUObject(this, &UCompositeLayerPlate::UpdatePrimitiveComponent);
				PrimitiveComponent->AddAssetUserData(AssetUserData);
			}

			if (!bRegisterOnly)
			{
				UpdatePrimitiveComponent(*PrimitiveComponent);
			}
		}
	}

	UpdateTranslucencyOverride();
}

void UCompositeLayerPlate::UpdatePrimitiveComponent(UPrimitiveComponent& InPrimitiveComponent) const
{
	for (int32 ElementIndex = 0; ElementIndex < InPrimitiveComponent.GetNumMaterials(); ++ElementIndex)
	{
		UMaterialInterface* MaterialInterface = InPrimitiveComponent.GetMaterial(ElementIndex);

		if (!IsValid(MaterialInterface))
		{
			continue;
		}

		TArray<FMaterialParameterInfo> ParameterInfo;
		TArray<FGuid> ParameterIds;
		MaterialInterface->GetAllTextureParameterInfo(ParameterInfo, ParameterIds);

		// Avoid conversion to MID if parameter doesn't exist
		const bool bHasTextureParameter = ParameterInfo.ContainsByPredicate([](const FMaterialParameterInfo& ParamInfo)
			{
				return ParamInfo.Name.IsEqual(CompositeTextureName);
			}
		);

		if (bHasTextureParameter)
		{
			UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(MaterialInterface);

			if (!MID)
			{
				/**
				* Note: Unfortunately, MIDs are the only currently viable approach to bind textures to composite meshes.
				* Texture collection could provide a nice alternative, but are experimental, have limited support and do
				* not provide the same world-unique instance functionality that MPCs do, which would cause dynamic
				* bindings to dirty a texture collection asset.
				*/
				MID = InPrimitiveComponent.CreateAndSetMaterialInstanceDynamic(ElementIndex);
			}
			
			MID->SetScalarParameterValue(TEXT("UseCompositeProjection"), static_cast<float>(PlateMode == ECompositePlateMode::CompositeMesh));
			MID->SetTextureParameterValue(TEXT("CompositeTexture"), GetCompositeTexture());
		}
	}

	UpdateTranslucencyOverride();
}

void UCompositeLayerPlate::PropagateStateChange(ECompositeStateChangeType ChangeType, const UWorld* SourceWorld) const
{
	if (ChangeType == ECompositeStateChangeType::Activate && IsRendering())
	{
		if (CompositeMeshes.IsEmpty())
		{
			return;
		}

		constexpr bool bRegisterOnly = true;
		UpdateCompositeMeshes(bRegisterOnly);

		const TArray<UPrimitiveComponent*> Primitives = GetPrimitives();

		if (bControlMeshVisibility)
		{
			for (UPrimitiveComponent* Primitive : Primitives)
			{
				UE::Composite::Private::SetCompositeMeshVisibility(*Primitive, true);
			}
		}

		// Note: We could rely solely on the tick registration, but this call is kept for symmetry.
		TryRegisterCompositePrimitives(Primitives, SourceWorld);
	}
	else
	{
		if (!CompositeMeshes.IsEmpty())
		{
			UE::Composite::Private::RemoveCompositeAllAssetUserData(CompositeMeshes);

			const TArray<UPrimitiveComponent*> Primitives = GetPrimitives();

			UnregisterCompositePrimitives(Primitives, SourceWorld);

			if (bControlMeshVisibility)
			{
				if (ChangeType == ECompositeStateChangeType::Release)
				{
					for (UPrimitiveComponent* Primitive : Primitives)
					{
						UE::Composite::Private::SetCompositeMeshVisibility(*Primitive, true);
					}
				}
				else
				{
					TSet<UPrimitiveComponent*> UsedPrimitives = GetPrimitivesUsedByOtherActors();

					for (UPrimitiveComponent* Primitive : Primitives)
					{
						// Only make a primitive invisible if it's not already used by another composite actor
						if (!UsedPrimitives.Contains(Primitive))
						{
							UE::Composite::Private::SetCompositeMeshVisibility(*Primitive, false);
						}
					}
				}
			}
		}

		constexpr bool bOverride = false;
		OverrideTranslucencyConsoleVariables(bOverride);
	}
}

bool UCompositeLayerPlate::IsCompositeMeshActorAlreadyInUse(TSoftObjectPtr<AActor> InCompositeMeshActor) const
{
	const ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (!IsValid(CompositeActor))
	{
		return false;
	}

	for (const TObjectPtr<UCompositeLayerBase>& Layer : CompositeActor->GetCompositeLayers())
	{
		const UCompositeLayerPlate* PlateLayer = Cast<UCompositeLayerPlate>(Layer.Get());
		if (IsValid(PlateLayer) && PlateLayer != this)
		{
			if (PlateLayer->CompositeMeshes.Contains(InCompositeMeshActor))
			{
				return true;
			}
		}
	}

	return false;
}

TSet<UPrimitiveComponent*> UCompositeLayerPlate::GetPrimitivesUsedByOtherActors() const
{
	// Note: as oppposed to trying to maintain a static cache of actively used primitives,
	// we simply query the world state and pay the lookup cost.

	TSet<UPrimitiveComponent*> Output;

	if (const ACompositeActor* ThisActor = GetTypedOuter<ACompositeActor>())
	{
		if (UCompositeWorldSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeWorldSubsystem>(GetWorld()))
		{
			for (const TWeakObjectPtr<ACompositeActor> WeakActor : Subsystem->GetActors())
			{
				const ACompositeActor* OtherActor = WeakActor.Get();

				if (!OtherActor || OtherActor == ThisActor || !OtherActor->IsRendering())
				{
					continue;
				}

				for (const TObjectPtr<UCompositeLayerBase>& OtherLayer : OtherActor->GetCompositeLayers())
				{
					const UCompositeLayerPlate* OtherPlateLayer = Cast<UCompositeLayerPlate>(OtherLayer.Get());

					if (!OtherPlateLayer || !ensure(OtherPlateLayer != this) || !OtherPlateLayer->IsRendering())
					{
						continue;
					}

					for (UPrimitiveComponent* OtherPrimitive : OtherPlateLayer->GetPrimitives())
					{
						if (OtherPrimitive)
						{
							Output.Add(OtherPrimitive);
						}
					}
				}
			}
		}
	}

	return Output;
}

TObjectPtr<UTextureRenderTarget2D> UCompositeLayerPlate::GetOrCreateRenderTarget(TObjectPtr<UTextureRenderTarget2D>& InRenderTarget) const
{
	const FIntPoint TextureSize = UE::Composite::Private::GetTextureSize(Texture);

	if (IsValid(InRenderTarget) && FIntPoint(InRenderTarget->SizeX, InRenderTarget->SizeY) != TextureSize)
	{
		// Automatically release our render targets, such that they are resized to match the dynamic media texture resolution.
		FCompositeRenderTargetPool::Get().ReleaseTarget(InRenderTarget);
		
		// For sequencer use, it is preferable to warmup the media texture since this rebind can occur on the next tick.
		bRebindRenderTarget = true;
	}

	if (!IsValid(InRenderTarget))
	{
		FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(this, InRenderTarget, TextureSize);
	}

	return InRenderTarget;
}

#undef LOCTEXT_NAMESPACE
