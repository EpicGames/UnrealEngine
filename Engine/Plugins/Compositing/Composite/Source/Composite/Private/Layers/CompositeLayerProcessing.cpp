// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerProcessing.h"

#include "CompositeActor.h"

UCompositeLayerProcessing::UCompositeLayerProcessing(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Operation = ECompositeCoreMergeOp::None;
}

UCompositeLayerProcessing::~UCompositeLayerProcessing() = default;

#if WITH_EDITOR
bool UCompositeLayerProcessing::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);

	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bIsSolo))
		{
			bIsEditable = false;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Operation))
		{
			bIsEditable = false;
		}
	}

	return bIsEditable;
}

void UCompositeLayerProcessing::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FCompositeCorePassProxy* UCompositeLayerProcessing::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	// Note: We ignore the passed InputDecl for now, this will be cleaned up in a subsequence refactor.
	FPassInputDecl BasePassInput = GetDefaultSecondInput(InContext);
	FCompositeCorePassProxy* OutProxy = nullptr;

	// Iterate backwards so that lower passes in the UI are executed first
	// TODO: Flip back once we have UI customization displaying passes in the correct bottom-up order
	for (int32 PassIndex = LayerPasses.Num() - 1; PassIndex >= 0; --PassIndex)
	{
		const TObjectPtr<UCompositePassBase>& Pass = LayerPasses[PassIndex];

		if (IsValid(Pass) && Pass->GetIsActive())
		{
			FCompositeCorePassProxy* PassProxy = Pass->GetProxy(BasePassInput, InContext, InFrameAllocator);
			if (PassProxy != nullptr)
			{
				OutProxy = PassProxy;

				// Next pass should receive the output of the current child pass.
				BasePassInput.Set<const FCompositeCorePassProxy*>(PassProxy);
			}
		}
	}

	return OutProxy;
}

