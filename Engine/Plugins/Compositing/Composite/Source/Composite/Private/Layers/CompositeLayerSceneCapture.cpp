// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerSceneCapture.h"

#include "CompositeActor.h"
#include "Components/PrimitiveComponent.h"
#include "Components/CompositeSceneCapture2DComponent.h"
#include "CompositeRenderTargetPool.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/Package.h"

UCompositeLayerSceneCapture::UCompositeLayerSceneCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RenderTargetResolution{ FCompositeRenderTargetPool::DefaultSize }
	, bVisibleInSceneCaptureOnly{ true } // By default, registered meshes will no longer be visible in the main render
	, bCustomRenderPass{ false }
{
	Operation = ECompositeCoreMergeOp::Over;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		constexpr bool bEnabled = true;
		RegisterEndOfFrameUpdate(bEnabled);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCompositeLayerSceneCapture::~UCompositeLayerSceneCapture() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UCompositeLayerSceneCapture::OnRemoved(ACompositeActor* LastOwner)
{
	// Note: We don't use IsValid(..) since destruction should proceed even if the actor is pending kill.
	if (LastOwner != nullptr)
	{
		LastOwner->DestroySceneCaptures(this);
	}

	RestorePrimitiveVisibilityState();
}

void UCompositeLayerSceneCapture::BeginDestroy()
{
	Super::BeginDestroy();

	OnRemoved(GetTypedOuter<ACompositeActor>()); // Redundant remove call for safety
}

#if WITH_EDITOR
void UCompositeLayerSceneCapture::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropertyName = PropertyAboutToChange->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Actors))
	{
		RestorePrimitiveVisibilityState();
		SpawnableBindings.CachePreEditState(Actors);
	}
}

void UCompositeLayerSceneCapture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Actors))
	{
		SetActorsInternal(MoveTemp(Actors));
		SpawnableBindings.SyncOnPropertyChange(Actors, GetWorld());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bCustomRenderPass))
	{
		SetCustomRenderPass(bCustomRenderPass);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bVisibleInSceneCaptureOnly))
	{
		SetVisibleInSceneCaptureOnly(bVisibleInSceneCaptureOnly);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution))
	{
		SetRenderTargetResolution(RenderTargetResolution);
	}
}

void UCompositeLayerSceneCapture::PreEditUndo()
{
	Super::PreEditUndo();

	RestorePrimitiveVisibilityState();
}

void UCompositeLayerSceneCapture::PostEditUndo()
{
	Super::PostEditUndo();

	// Actors and SpawnableBindings.Bindings are both UPROPERTY on this object, so the transaction
	// system restores them atomically — the parallel-array invariant is preserved without an
	// explicit SyncOnPropertyChange. SetActorsInternal is re-run to refresh non-transactional
	// state (primitive visibility, scene capture component setup) that the transaction does not capture.
	SetActorsInternal(MoveTemp(Actors));

	SetCustomRenderPass(bCustomRenderPass);
}

void UCompositeLayerSceneCapture::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo &&
		TransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution)))
	{
		/**
		* When a resolution change is undone, the previous render targets are deserialized & replaced.
		* This leaves previous assignments invalid so we make sure to return them to the pool.
		*/
		if (UCompositeSceneCapture2DComponent* CaptureComponent = GetSceneCaptureComponent())
		{
			FCompositeRenderTargetPool::Get().ReleaseAssigneeTargets(CaptureComponent);
		}
	}
}
#endif

void UCompositeLayerSceneCapture::OnEndOfFrameUpdate(UWorld* InWorld)
{
	if (SpawnableBindings.TickResolveStale(Actors, InWorld, GetUniqueID()))
	{
		UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);
	}

	/**
	* In order to retain the ability to reference actors in sublevels, we prefer using weak pointer components on scene captures. However,
	* primitive components may often be recreated (see RerunConstructionScripts(), FStaticMeshComponentRecreateRenderStateContext, dynamic
	* text primitives, etc). To prevent issues with destroyed primitives disappearing in our scene captures, we re-register them every frame.
	*/
	USceneCaptureComponent2D* SceneCaptureComponent = GetSceneCaptureComponent();
	if (IsValid(SceneCaptureComponent))
	{
		UpdateSceneCaptureComponents(*SceneCaptureComponent, Actors);
	}
}

bool UCompositeLayerSceneCapture::GetIsActive() const
{
	return Super::GetIsActive()
		&& IsValid(GetTypedOuter<ACompositeActor>())
		&& IsValid(GetSceneCaptureComponent())
		&& !Actors.IsEmpty();
}

FCompositeCorePassProxy* UCompositeLayerSceneCapture::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	USceneCaptureComponent2D* CaptureComponent = GetSceneCaptureComponent();

	FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(CaptureComponent, CaptureComponent->TextureTarget, RenderTargetResolution);

	FResourceMetadata Metadata = {};
	Metadata.bInvertedAlpha = true;
	Metadata.bDistorted = CaptureComponent->ShowFlags.LensDistortion;
	Metadata.DebugName = TEXT("CompositeSceneCaptureTex");

	const ResourceId TexId = InContext.FindOrCreateExternalTexture(CaptureComponent->TextureTarget, Metadata);

	FPassInputDecl PassInput;
	PassInput.Set<FPassExternalResourceDesc>({ TexId });

	AddChildPasses(PassInput, InContext, InFrameAllocator, LayerPasses);

	FPassInputDeclArray Inputs;
	Inputs.SetNum(FixedNumLayerInputs);
	Inputs[0] = PassInput;
	Inputs[1] = GetDefaultSecondInput(InContext);	

	return InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), GetMergeOperation(InContext), TEXT("SceneCapture"), ELensDistortionHandling::Disabled);
}

const TArray<TSoftObjectPtr<AActor>> UCompositeLayerSceneCapture::GetActors() const
{
	return Actors;
}

void UCompositeLayerSceneCapture::SetActors(TArray<TSoftObjectPtr<AActor>> InActors)
{
	SpawnableBindings.CachePreEditState(Actors);
	SetActorsInternal(MoveTemp(InActors));
	SpawnableBindings.SyncOnPropertyChange(Actors, GetWorld());
}

void UCompositeLayerSceneCapture::SetActorsInternal(TArray<TSoftObjectPtr<AActor>> InActors)
{
	RestorePrimitiveVisibilityState();

	Actors = MoveTemp(InActors);

	UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);

	USceneCaptureComponent2D* SceneCaptureComponent = GetSceneCaptureComponent();
	if (IsValid(SceneCaptureComponent))
	{
		// First, we empty show-only actors since our previous logic used them
		SceneCaptureComponent->ShowOnlyActors.Empty();

		UpdateSceneCaptureComponents(*SceneCaptureComponent, Actors);
	}
}

bool UCompositeLayerSceneCapture::IsCustomRenderPass() const
{
	return bCustomRenderPass;
}

void UCompositeLayerSceneCapture::SetCustomRenderPass(bool bInIsFastRenderPass)
{
	bCustomRenderPass = bInIsFastRenderPass;

	UCompositeSceneCapture2DComponent* SceneCaptureComponent = GetSceneCaptureComponent();
	if (IsValid(SceneCaptureComponent))
	{
		if (bCustomRenderPass)
		{
			SceneCaptureComponent->CaptureSource = ESceneCaptureSource::SCS_SceneColorHDR;
			SceneCaptureComponent->bRenderInMainRenderer = true;
		}
		else
		{
			SceneCaptureComponent->CaptureSource = ESceneCaptureSource::SCS_FinalColorHDR;
			SceneCaptureComponent->bRenderInMainRenderer = false;
		}
	}
}

bool UCompositeLayerSceneCapture::IsVisibleInSceneCaptureOnly() const
{
	return bVisibleInSceneCaptureOnly;
}

void UCompositeLayerSceneCapture::SetVisibleInSceneCaptureOnly(bool bInVisible)
{
	bVisibleInSceneCaptureOnly = bInVisible;

	UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);
}

void UCompositeLayerSceneCapture::SetRenderTargetResolution(FIntPoint InRenderTargetResolution)
{
	RenderTargetResolution.X = FMath::Max(InRenderTargetResolution.X, 1);
	RenderTargetResolution.Y = FMath::Max(InRenderTargetResolution.Y, 1);
}

UCompositeSceneCapture2DComponent* UCompositeLayerSceneCapture::GetSceneCaptureComponent() const
{
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (IsValid(CompositeActor))
	{
		return CompositeActor->FindOrCreateSceneCapture<UCompositeSceneCapture2DComponent>(this);
	}

	return nullptr;
}

void UCompositeLayerSceneCapture::UpdatePrimitiveVisibilityState(bool bInVisibleInSceneCaptureOnly) const
{
	for (const TSoftObjectPtr<AActor>& SoftActor : Actors)
	{
		if (const AActor* Actor = SoftActor.Get())
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
				{
					if (PrimitiveComponent->bVisibleInSceneCaptureOnly != bInVisibleInSceneCaptureOnly)
					{
						PrimitiveComponent->Modify();
						PrimitiveComponent->SetVisibleInSceneCaptureOnly(bInVisibleInSceneCaptureOnly);
					}
				}
			}
		}
	}
}

void UCompositeLayerSceneCapture::RestorePrimitiveVisibilityState() const
{
	// First, we restore current actors to class default, in case they have been deleted.
	UPrimitiveComponent* CDO = UPrimitiveComponent::StaticClass()->GetDefaultObject<UPrimitiveComponent>();

	UpdatePrimitiveVisibilityState(CDO->bVisibleInSceneCaptureOnly);
}

