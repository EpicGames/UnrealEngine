// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerShadowReflection.h"

#include "CompositeActor.h"
#include "Components/PrimitiveComponent.h"
#include "Components/CompositeShadowReflectionCatcherComponent.h"
#include "CompositeCVarOverrideManager.h"
#include "CompositeModule.h"
#include "CompositeRenderTargetPool.h"
#include "Containers/StaticArray.h"
#include "Misc/TransactionObjectEvent.h"

UCompositeLayerShadowReflection::UCompositeLayerShadowReflection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RenderTargetResolution{ 960, 540 } // Lower resolution for better performance by default
{
	Operation = ECompositeCoreMergeOp::Multiply;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		constexpr bool bEnabled = true;
		RegisterEndOfFrameUpdate(bEnabled);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCompositeLayerShadowReflection::~UCompositeLayerShadowReflection() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UCompositeLayerShadowReflection::OnRemoved(ACompositeActor* LastOwner)
{
	// Note: We don't use IsValid(..) since destruction should proceed even if the actor is pending kill.
	if (LastOwner != nullptr)
	{
		LastOwner->DestroySceneCaptures(this);
	}

	SetHiddenPrimitiveState(false);
}

void UCompositeLayerShadowReflection::PostLoad()
{
	Super::PostLoad();

	if (GetIsEnabled() && !Actors.IsEmpty())
	{
		SetHiddenPrimitiveState(true);
	}
}

void UCompositeLayerShadowReflection::BeginDestroy()
{
	Super::BeginDestroy();

	OnRemoved(GetTypedOuter<ACompositeActor>()); // Redundant remove call for safety
}

#if WITH_EDITOR
void UCompositeLayerShadowReflection::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropertyName = PropertyAboutToChange->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Actors))
	{
		// Pre-actively remove hidden state in case actors are removed
		SetHiddenPrimitiveState(false);
		SpawnableBindings.CachePreEditState(Actors);
	}
}

void UCompositeLayerShadowReflection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Actors))
	{
		SetActorsInternal(MoveTemp(Actors));
		SpawnableBindings.SyncOnPropertyChange(Actors, GetWorld());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution))
	{
		SetRenderTargetResolution(RenderTargetResolution);
	}
}

void UCompositeLayerShadowReflection::PreEditUndo()
{
	Super::PreEditUndo();

	// Pre-actively remove hidden state in case undo empties actors
	SetHiddenPrimitiveState(false);
}

void UCompositeLayerShadowReflection::PostEditUndo()
{
	Super::PostEditUndo();

	// Actors and SpawnableBindings.Bindings are both UPROPERTY on this object, so the transaction
	// system restores them atomically — the parallel-array invariant is preserved without an
	// explicit SyncOnPropertyChange. SetActorsInternal is re-run to refresh non-transactional
	// state (primitive hidden flags) that the transaction does not capture.
	SetActorsInternal(MoveTemp(Actors));
}

void UCompositeLayerShadowReflection::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo &&
		TransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution)))
	{
		/**
		* When a resolution change is undone, the previous render targets are deserialized & replaced.
		* This leaves previous assignments invalid so we make sure to return them to the pool.
		*/
		for (const TWeakObjectPtr<UCompositeShadowReflectionCatcherComponent>& CaptureWeak : CachedSceneCaptures)
		{
			if (UCompositeShadowReflectionCatcherComponent* Capture = CaptureWeak.Get())
			{
				FCompositeRenderTargetPool::Get().ReleaseAssigneeTargets(Capture);
			}
		}
	}
}
#endif

void UCompositeLayerShadowReflection::OnEndOfFrameUpdate(UWorld* InWorld)
{
	// Note: OnEndOfFrameUpdate is only called from the base layer if GetIsEnabled() is true.

	if (SpawnableBindings.TickResolveStale(Actors, InWorld, GetUniqueID()))
	{
		SetHiddenPrimitiveState(true);
	}

	/**
	* In order to retain the ability to reference actors in sublevels, we prefer using weak pointer components on scene captures. However,
	* primitive components may often be recreated (see RerunConstructionScripts(), FStaticMeshComponentRecreateRenderStateContext, dynamic
	* text primitives, etc). To prevent issues with destroyed primitives disappearing in our scene captures, we re-register them every frame.
	*/
	if (CachedSceneCaptures[1].IsValid())
	{
		UpdateSceneCaptureComponents(*CachedSceneCaptures[1], Actors);
	}

	/** 
	 * We also re-evaluate visibility-dependent flags each frame so that runtime
	 * visibility changes are present in the shadow/reflection catcher.
	 */
	if (!Actors.IsEmpty())
	{
		constexpr bool bSceneCaptureHidden = true;
		constexpr bool bConditionOnVisibility = true;

		UpdatePrimitiveVisibilityState(bSceneCaptureHidden, bSceneCaptureHidden, bSceneCaptureHidden, bConditionOnVisibility);
	}
}

bool UCompositeLayerShadowReflection::GetIsActive() const
{
	return Super::GetIsActive()
		&& IsValid(GetTypedOuter<ACompositeActor>())
		&& !Actors.IsEmpty();
}

FCompositeCorePassProxy* UCompositeLayerShadowReflection::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (IsValid(CompositeActor))
	{
		CachedSceneCaptures = FindOrCreateSceneCapturePair(*CompositeActor);
	}

	// Cached scene captures are now expected to be valid, early out if not.
	if (!ensure(CachedSceneCaptures[0].IsValid() && CachedSceneCaptures[1].IsValid()))
	{
		return nullptr;
	}

	FMergePassProxy* DivisionProxy = nullptr;

	{
		FPassInputDeclArray Inputs;
		Inputs.SetNum(2);

		static const TCHAR* DebugNames[] =
		{
			TEXT("CompositeShadowReflectionCGTex"),
			TEXT("CompositeShadowReflectionNoCGTex"),
		};

		for (int32 Index = 0; Index < 2; ++Index)
		{
			UCompositeShadowReflectionCatcherComponent* Component = CachedSceneCaptures[Index].Get();
			FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(Component, Component->TextureTarget, RenderTargetResolution);

			FResourceMetadata Metadata = {};
			Metadata.bInvertedAlpha = true;
			Metadata.bDistorted = Component->ShowFlags.LensDistortion;
			Metadata.DebugName = DebugNames[Index];
			const ResourceId TexId = InContext.FindOrCreateExternalTexture(Component->TextureTarget, Metadata);

			Inputs[Index].Set<FPassExternalResourceDesc>({ TexId });
		}

		DivisionProxy = InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), ECompositeCoreMergeOp::Divide, TEXT("ShadowRefl Div"), ELensDistortionHandling::Disabled);
		DivisionProxy->bLuminanceOnly = bLuminanceOnly;
	}

	FPassInputDecl PassInput;
	PassInput.Set<const FCompositeCorePassProxy*>(DivisionProxy);

	AddChildPasses(PassInput, InContext, InFrameAllocator, LayerPasses);

	FPassInputDeclArray Inputs;
	Inputs.SetNum(FixedNumLayerInputs);
	Inputs[0] = PassInput;
	Inputs[1] = GetDefaultSecondInput(InContext);

	return InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), GetMergeOperation(InContext), TEXT("ShadowRefl Mul"));
}

const TArray<TSoftObjectPtr<AActor>> UCompositeLayerShadowReflection::GetActors() const
{
	return Actors;
}

void UCompositeLayerShadowReflection::SetActors(TArray<TSoftObjectPtr<AActor>> InActors)
{
	SpawnableBindings.CachePreEditState(Actors);
	SetActorsInternal(MoveTemp(InActors));
	SpawnableBindings.SyncOnPropertyChange(Actors, GetWorld());
}

void UCompositeLayerShadowReflection::SetActorsInternal(TArray<TSoftObjectPtr<AActor>> InActors)
{
	// First, we make sure primitive hidden state is removed
	SetHiddenPrimitiveState(false);

	Actors = MoveTemp(InActors);

	if (GetIsEnabled() && !Actors.IsEmpty())
	{
		SetHiddenPrimitiveState(true);
	}

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (!IsValid(CompositeActor))
	{
		return;
	}

	if (!CachedSceneCaptures[0].IsValid() || !CachedSceneCaptures[1].IsValid())
	{
		CachedSceneCaptures = FindOrCreateSceneCapturePair(*CompositeActor);
	}

	// Early exit if the cached scene captures are not valid
	if (!ensure(CachedSceneCaptures[0].IsValid() && CachedSceneCaptures[1].IsValid()))
	{
		return;
	}

	// First, we empty hidden actors since our previous logic used them
	CachedSceneCaptures[1]->HiddenActors.Empty();

	UpdateSceneCaptureComponents(*CachedSceneCaptures[1], Actors);
}

void UCompositeLayerShadowReflection::SetRenderTargetResolution(FIntPoint InRenderTargetResolution)
{
	RenderTargetResolution.X = FMath::Max(InRenderTargetResolution.X, 1);
	RenderTargetResolution.Y = FMath::Max(InRenderTargetResolution.Y, 1);
}

void UCompositeLayerShadowReflection::SetIsEnabled(bool bInEnabled)
{
	Super::SetIsEnabled(bInEnabled);

	SetHiddenPrimitiveState(bInEnabled && !Actors.IsEmpty());
}

void UCompositeLayerShadowReflection::UpdatePrimitiveVisibilityState(
	bool bInHiddenInSceneCapture,
	bool bInAffectIndirectLightingWhileHidden,
	bool bInCastHiddenShadow,
	bool bConditionOnVisibility) const
{
	for (const TSoftObjectPtr<AActor>& Actor : Actors)
	{
		if (!Actor.IsValid())
		{
			continue;
		}

		for (UActorComponent* Component : Actor->GetComponents())
		{
			UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);

			if (!PrimitiveComponent)
			{
				continue;
			}

			bool bUpdatedComponent = false;
			bool bEffectiveIndirectLighting = bInAffectIndirectLightingWhileHidden;
			bool bEffectiveCastHiddenShadow = bInCastHiddenShadow;

			// Respect primitive visibility: invisible meshes should not cast hidden shadows or affect indirect lighting.
			if (bConditionOnVisibility)
			{
				const bool bShouldRender = PrimitiveComponent->ShouldRender();

				bEffectiveIndirectLighting &= bShouldRender;
				bEffectiveCastHiddenShadow &= bShouldRender;
			}

			if (PrimitiveComponent->bHiddenInSceneCapture != bInHiddenInSceneCapture)
			{
				PrimitiveComponent->SetHiddenInSceneCapture(bInHiddenInSceneCapture);
				bUpdatedComponent = true;
			}

			if (PrimitiveComponent->bAffectIndirectLightingWhileHidden != bEffectiveIndirectLighting)
			{
				PrimitiveComponent->SetAffectIndirectLightingWhileHidden(bEffectiveIndirectLighting);
				bUpdatedComponent = true;
			}

			if (PrimitiveComponent->bCastHiddenShadow != bEffectiveCastHiddenShadow)
			{
				PrimitiveComponent->SetCastHiddenShadow(bEffectiveCastHiddenShadow);
				bUpdatedComponent = true;
			}

			if (bUpdatedComponent)
			{
				PrimitiveComponent->Modify();
			}
		}
	}
}

void UCompositeLayerShadowReflection::SetHiddenPrimitiveState(bool bInHiddenInSceneCapture) const
{
	//* Note: We aim to expose megalights screen traces as an PPV setting, in which case the cvar could be removed. */
	const TCHAR* MegaLightsSceenTracesName = TEXT("r.MegaLights.ScreenTraces");
	const TCHAR* LumenShortRangeAOHWRTName = TEXT("r.Lumen.ScreenProbeGather.ShortRangeAO.HardwareRayTracing");

	static TSet<const UCompositeLayerShadowReflection*> ActiveHiddenStateLayers;

	if (bInHiddenInSceneCapture)
	{
		constexpr bool bSceneCaptureHidden = true;
		constexpr bool bConditionOnVisibility = true;

		UpdatePrimitiveVisibilityState(bSceneCaptureHidden, bSceneCaptureHidden, bSceneCaptureHidden, bConditionOnVisibility);

		if (ActiveHiddenStateLayers.IsEmpty())
		{
			FCompositeCVarOverrideManager::Get().Override(MegaLightsSceenTracesName, 0);
			FCompositeCVarOverrideManager::Get().Override(LumenShortRangeAOHWRTName, 1);
		}

		ActiveHiddenStateLayers.Add(this);
	}
	else
	{
		// Restore to class default
		UPrimitiveComponent* CDO = UPrimitiveComponent::StaticClass()->GetDefaultObject<UPrimitiveComponent>();
		constexpr bool bConditionOnVisibility = false;

		UpdatePrimitiveVisibilityState(CDO->bHiddenInSceneCapture, CDO->bAffectIndirectLightingWhileHidden, CDO->bCastHiddenShadow, bConditionOnVisibility);

		const int32 NumRemoved = ActiveHiddenStateLayers.Remove(this);

		if (NumRemoved > 0 && ActiveHiddenStateLayers.IsEmpty())
		{
			FCompositeCVarOverrideManager::Get().Restore(MegaLightsSceenTracesName);
			FCompositeCVarOverrideManager::Get().Restore(LumenShortRangeAOHWRTName);
		}
	}
}

TStaticArray<TWeakObjectPtr<UCompositeShadowReflectionCatcherComponent>, 2> UCompositeLayerShadowReflection::FindOrCreateSceneCapturePair(ACompositeActor& InOuter) const
{
	return {
		InOuter.FindOrCreateSceneCapture<UCompositeShadowReflectionCatcherComponent>(this, 0, FName("ShadowReflectionCatcher_CG")),
		InOuter.FindOrCreateSceneCapture<UCompositeShadowReflectionCatcherComponent>(this, 1, FName("ShadowReflectionCatcher_NoCG"))
	};
}

