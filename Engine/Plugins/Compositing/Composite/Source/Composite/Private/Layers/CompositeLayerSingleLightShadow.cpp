// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/CompositeLayerSingleLightShadow.h"

#include "Camera/CameraComponent.h"
#include "CompositeActor.h"
#include "CompositeRenderTargetPool.h"
#include "Misc/TransactionObjectEvent.h"
#include "Passes/CompositePassMaterial.h"
#include "Components/CompositeSceneCapture2DComponent.h"
#include "Engine/DirectionalLight.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Passes/CompositeCorePassProxy.h"
#include "RHI.h"

DECLARE_GPU_STAT_NAMED(FCompositeSingleLightShadow, TEXT("Composite.SingleLayerShadow"));

class FCompositeSingleLightShadowShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeSingleLightShadowShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositeSingleLightShadowShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTex)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowDepthTextureSampler)

		SHADER_PARAMETER(FVector4f, SceneDepthPositionScaleBias)
		SHADER_PARAMETER(FMatrix44f, SceneDepthScreenToWorld)
		SHADER_PARAMETER(FMatrix44f, ShadowMatrix)
		SHADER_PARAMETER(FVector4f, ShadowBufferSize)
		SHADER_PARAMETER(FVector4f, ShadowInvDeviceZToWorldZ)
		SHADER_PARAMETER(FVector4f, ShadowStrengthBias)
		SHADER_PARAMETER(FVector3f, LightWorldDirection)
		SHADER_PARAMETER(uint32, bIsPerspectiveProjection)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositeSingleLightShadowShader, "/Plugin/Composite/Private/CompositeSingleLightShadow.usf", "MainPS", SF_Pixel);


namespace UE
{
	namespace CompositeCore
	{
		class FCompositeSingleLightShadowProxy : public FCompositeCorePassProxy
		{
		public:
			IMPLEMENT_COMPOSITE_PASS(FCompositeSingleLightShadowProxy);

			using FCompositeCorePassProxy::FCompositeCorePassProxy;

			FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeSingleLightShadow, "Composite.SingleLayerShadow");

				check(ValidateInputs(Inputs));

				const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
				const FScreenPassTexture& SceneDepth = Inputs[0].Texture;
				const FScreenPassTexture& ShadowMap = Inputs[1].Texture;
				FScreenPassRenderTarget Output = Inputs.OverrideOutput;

				if (!Output.IsValid())
				{
					// Format is set inside CreateOutputRenderTarget.
					Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, SceneDepth.Texture->Desc, TEXT("CompositeSingleLayerShadow"));
				}

				// Calculate scene depth position scale bias
				const FIntPoint Resolution = FIntPoint(SceneDepth.ViewRect.Size().X, SceneDepth.ViewRect.Size().Y);
				FVector4f SceneDepthPositionScaleBias = FSceneView::GetScreenPositionScaleBias(Resolution, FIntRect(FIntPoint::ZeroValue, Resolution));

				// Calculate scene depth screen to world matrix (without jitter).
				FMatrix44f SceneDepthScreenToWorld = FMatrix44f(
					InView.ViewMatrices.GetScreenToClipMatrix() *
					InView.ViewMatrices.GetViewToClipNoAA().Inverse() *
					InView.ViewMatrices.GetViewToWorld()
				);

				// The shadow map texture size
				const FIntVector ShadowMapSize = ShadowMap.Texture->Desc.GetSize();

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
				FScreenPassTextureViewport Viewport = FScreenPassTextureViewport(Output);

				FCompositeSingleLightShadowShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositeSingleLightShadowShader::FParameters>();
				Parameters->SceneDepthTex				= SceneDepth.Texture;
				Parameters->SceneDepthSampler			= TStaticSamplerState<SF_Bilinear>::GetRHI();
				Parameters->ShadowDepthTexture			= ShadowMap.Texture;
				Parameters->ShadowDepthTextureSampler	= TStaticSamplerState<SF_Point>::GetRHI();
				Parameters->SceneDepthPositionScaleBias	= SceneDepthPositionScaleBias;
				Parameters->SceneDepthScreenToWorld		= SceneDepthScreenToWorld;
				Parameters->ShadowInvDeviceZToWorldZ	= ShadowInvDeviceZToWorldZ;
				Parameters->ShadowMatrix				= ShadowMatrix;
				Parameters->ShadowBufferSize			= FVector4f((float)ShadowMapSize.X, (float)ShadowMapSize.Y, FMath::SafeDivide(1.0f, (float)ShadowMapSize.X), FMath::SafeDivide(1.0f, (float)ShadowMapSize.Y));
				Parameters->ShadowStrengthBias		= ShadowStrengthBias;
				Parameters->LightWorldDirection			= LightWorldDirection;
				// We can use the current view since we the depth render matches the main view
				Parameters->bIsPerspectiveProjection	= InView.IsPerspectiveProjection();
				Parameters->RenderTargets[0]			= Output.GetRenderTargetBinding();

				TShaderMapRef<FCompositeSingleLightShadowShader> PixelShader(GlobalShaderMap);
				AddDrawScreenPass(
					GraphBuilder,
					RDG_EVENT_NAME("Composite.SingleLayerShadow (%dx%d)", Viewport.Extent.X, Viewport.Extent.Y),
					InView,
					Viewport,
					Viewport,
					PixelShader,
					Parameters
				);

				return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
			}

			FMatrix44f ShadowMatrix;
			FVector4f ShadowInvDeviceZToWorldZ;
			FVector4f ShadowStrengthBias;
			FVector3f LightWorldDirection;
		};
	}
}

UCompositeLayerSingleLightShadow::UCompositeLayerSingleLightShadow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutomaticBounds{ true }
	, OrthographicWidth{ DEFAULT_ORTHOWIDTH }
	, ShadowMapResolution{ 2048 }
	, ShadowStrength{ 0.5f }
	, RenderTargetResolution{ FCompositeRenderTargetPool::DefaultSize }
{
	Operation = ECompositeCoreMergeOp::Multiply;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		constexpr bool bEnabled = true;
		RegisterEndOfFrameUpdate(bEnabled);
	}
}

void UCompositeLayerSingleLightShadow::OnRemoved(ACompositeActor* LastOwner)
{
	// Note: We don't use IsValid(..) since destruction should proceed even if the actor is pending kill.
	if (LastOwner != nullptr)
	{
		LastOwner->DestroySceneCaptures(this);
	}
}

void UCompositeLayerSingleLightShadow::BeginDestroy()
{
	Super::BeginDestroy();

	OnRemoved(GetTypedOuter<ACompositeActor>()); // Redundant remove call for safety
}

void UCompositeLayerSingleLightShadow::SetRenderTargetResolution(FIntPoint InRenderTargetResolution)
{
	RenderTargetResolution.X = FMath::Max(InRenderTargetResolution.X, 1);
	RenderTargetResolution.Y = FMath::Max(InRenderTargetResolution.Y, 1);
}

#if WITH_EDITOR
void UCompositeLayerSingleLightShadow::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropertyName = PropertyAboutToChange->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, ShadowCastingActors))
	{
		SpawnableBindings.CachePreEditState(ShadowCastingActors);
	}
}

void UCompositeLayerSingleLightShadow::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, ShadowCastingActors))
	{
		SetShadowCastingActorsInternal(MoveTemp(ShadowCastingActors));
		SpawnableBindings.SyncOnPropertyChange(ShadowCastingActors, GetWorld());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution))
	{
		SetRenderTargetResolution(RenderTargetResolution);
	}
}

void UCompositeLayerSingleLightShadow::PostEditUndo()
{
	Super::PostEditUndo();

	// ShadowCastingActors and SpawnableBindings.Bindings are both UPROPERTY on this object, so the
	// transaction system restores them atomically — the parallel-array invariant is preserved
	// without an explicit SyncOnPropertyChange. SetShadowCastingActorsInternal is re-run to refresh
	// non-transactional state (scene capture component show-only list) that the transaction does
	// not capture.
	SetShadowCastingActorsInternal(MoveTemp(ShadowCastingActors));
}

void UCompositeLayerSingleLightShadow::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo &&
		TransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution)))
	{
		/**
		* When a resolution change is undone, the previous render targets are deserialized & replaced.
		* This leaves previous assignments invalid so we make sure to return them to the pool.
		*/
		if (UCompositeSceneCapture2DComponent* Capture = CachedSceneDepthCapture.Get())
		{
			FCompositeRenderTargetPool::Get().ReleaseAssigneeTargets(Capture);
		}
	}
}
#endif

void UCompositeLayerSingleLightShadow::OnEndOfFrameUpdate(UWorld* InWorld)
{
	SpawnableBindings.TickResolveStale(ShadowCastingActors, InWorld, GetUniqueID());

	/**
	* In order to retain the ability to reference actors in sublevels, we prefer using weak pointer components on scene captures. However,
	* primitive components may often be recreated (see RerunConstructionScripts(), FStaticMeshComponentRecreateRenderStateContext, dynamic
	* text primitives, etc). To prevent issues with destroyed primitives disappearing in our scene captures, we re-register them every frame.
	*/
	if (CachedShadowMapCapture.IsValid())
	{
		UpdateSceneCaptureComponents(*CachedShadowMapCapture, ShadowCastingActors);
	}

	if (CachedSceneDepthCapture.IsValid())
	{
		UpdateSceneCaptureComponents(*CachedSceneDepthCapture, ShadowCastingActors);
	}
}

bool UCompositeLayerSingleLightShadow::GetIsActive() const
{
	return Super::GetIsActive()
		&& IsValid(GetTypedOuter<ACompositeActor>())
		&& IsValid(Light)
		&& ShadowStrength > 0.0f
		&& !ShadowCastingActors.IsEmpty();
}

FCompositeCorePassProxy* UCompositeLayerSingleLightShadow::GetProxy(const UE::CompositeCore::FPassInputDecl& /*InputDecl*/, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (!IsValid(CompositeActor))
	{
		return nullptr;
	}

	if (!CachedShadowMapCapture.IsValid())
	{
		CachedShadowMapCapture = FindOrCreateShadowMapCapture(*CompositeActor);
	}

	if (!CachedSceneDepthCapture.IsValid())
	{
		CachedSceneDepthCapture = FindOrCreateSceneDepthCapture(*CompositeActor);
	}

	// Both cached scene captures are now expected to be valid
	if (!CachedShadowMapCapture.IsValid() || !CachedSceneDepthCapture.IsValid())
	{
		return nullptr;
	}

	// Update properties that may be dynamic
	{
		if (bAutomaticBounds)
		{
			FBox ShadowCastingBoundingBox = FBox(EForceInit::ForceInitToZero);

			for (const TSoftObjectPtr<AActor>& ShadowCastingActor : ShadowCastingActors)
			{
				const AActor* Actor = ShadowCastingActor.Get();
				if (IsValid(Actor))
				{
					ShadowCastingBoundingBox += Actor->GetComponentsBoundingBox();
				}
			}

			// Assume shadow is projected onto the ground xy-plane.
			const FQuat LightRotation = Light->GetTransform().GetRotation();
			const FVector LightDirection = LightRotation.RotateVector(FVector(1, 0, 0)).GetSafeNormal();
			const FVector Up = FVector::UpVector;
			const FVector Location = ShadowCastingBoundingBox.GetCenter() - 2.0 * LightDirection * ShadowCastingBoundingBox.GetSize().GetMax();

			const float CosTheta = FMath::Abs(FVector::DotProduct(LightDirection, FVector::UpVector));

			CachedShadowMapCapture->SetWorldTransform(FTransform(LightRotation, Location));
			
			if (CosTheta < UE_KINDA_SMALL_NUMBER)
			{
				CachedShadowMapCapture->OrthoWidth = FMath::Max(DEFAULT_ORTHOWIDTH, ShadowCastingBoundingBox.GetSize().GetMax() * 10.0f);
			}
			else
			{
				const float Theta = FMath::Acos(FMath::Clamp(CosTheta, -1.0f, 1.0f));
				const float BoxHeight = ShadowCastingBoundingBox.GetSize().Z;
				const float ShadowLength = BoxHeight * FMath::Tan(Theta);
				constexpr float ShadowLengthMargin = 2.0f;
				constexpr float MinOrthoWidth = 1.f;

				CachedShadowMapCapture->OrthoWidth = FMath::Max(MinOrthoWidth, ShadowLengthMargin * ShadowLength);
			}
		}
		else
		{
			CachedShadowMapCapture->OrthoWidth = OrthographicWidth;
			CachedShadowMapCapture->SetWorldTransform(Light->GetTransform());
		}

		// Acquire render target
		FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(CachedShadowMapCapture.Get(), CachedShadowMapCapture->TextureTarget, FIntPoint(ShadowMapResolution, ShadowMapResolution), RTF_R32f);
		FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(CachedSceneDepthCapture.Get(), CachedSceneDepthCapture->TextureTarget, RenderTargetResolution, RTF_R32f);
	}
	
	// Setup child shadow pass proxy
	FCompositeCorePassProxy* InputProxy;

	{
		// Scene depth without jitter as an external texture.
		const ResourceId SceneDepthId = InContext.FindOrCreateExternalTexture(CachedSceneDepthCapture->TextureTarget, { .DebugName = TEXT("CompositSceneDepthTex") });

		// Shadow map texture
		const ResourceId ShadowMapId = InContext.FindOrCreateExternalTexture(CachedShadowMapCapture->TextureTarget, { .DebugName = TEXT("CompositeShadowMapTex") });

		// Setup shadow pass inputs
		FPassInputDeclArray PassInputs;
		PassInputs.SetNum(2);
		PassInputs[0].Set<FPassExternalResourceDesc>({ SceneDepthId });
		PassInputs[1].Set<FPassExternalResourceDesc>({ ShadowMapId });

		FCompositeSingleLightShadowProxy* ShadowProxy = InFrameAllocator.Create<FCompositeSingleLightShadowProxy>(PassInputs);
		ShadowProxy->ShadowStrengthBias = FVector4f(ShadowStrength, 0.0f, 0.0f, 0.0f);
		GetShadowMatrices(*CachedShadowMapCapture, ShadowProxy->ShadowMatrix, ShadowProxy->ShadowInvDeviceZToWorldZ);
		ShadowProxy->LightWorldDirection = FVector3f(CachedShadowMapCapture->GetComponentTransform().TransformVector(FVector(1, 0, 0)));

		// Set input proxy as shadow pass
		InputProxy = ShadowProxy;
	}

	FPassInputDecl PassInput;
	PassInput.Set<const FCompositeCorePassProxy*>(InputProxy);

	AddChildPasses(PassInput, InContext, InFrameAllocator, LayerPasses);

	FPassInputDeclArray Inputs;
	Inputs.SetNum(FixedNumLayerInputs);
	Inputs[0] = PassInput;
	Inputs[1] = GetDefaultSecondInput(InContext);

	return InFrameAllocator.Create<FMergePassProxy>(MoveTemp(Inputs), GetMergeOperation(InContext), TEXT("SingleLightShadowCatcher"));
}

const TArray<TSoftObjectPtr<AActor>> UCompositeLayerSingleLightShadow::GetShadowCastingActors() const
{
	return ShadowCastingActors;
}

void UCompositeLayerSingleLightShadow::SetShadowCastingActors(TArray<TSoftObjectPtr<AActor>> InShadowCaster)
{
	SpawnableBindings.CachePreEditState(ShadowCastingActors);
	SetShadowCastingActorsInternal(MoveTemp(InShadowCaster));
	SpawnableBindings.SyncOnPropertyChange(ShadowCastingActors, GetWorld());
}

void UCompositeLayerSingleLightShadow::SetShadowCastingActorsInternal(TArray<TSoftObjectPtr<AActor>> InShadowCasters)
{
	ShadowCastingActors = MoveTemp(InShadowCasters);

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (!IsValid(CompositeActor))
	{
		return;
	}

	if (!CachedShadowMapCapture.IsValid())
	{
		CachedShadowMapCapture = FindOrCreateShadowMapCapture(*CompositeActor);
	}

	if (!CachedSceneDepthCapture.IsValid())
	{
		CachedSceneDepthCapture = FindOrCreateSceneDepthCapture(*CompositeActor);
	}

	if (CachedShadowMapCapture.IsValid())
	{
		UpdateSceneCaptureComponents(*CachedShadowMapCapture, ShadowCastingActors);
	}
	
	if (CachedSceneDepthCapture.IsValid())
	{
		// First, we empty hidden actors since our previous logic used them
		CachedSceneDepthCapture->HiddenActors.Empty();

		UpdateSceneCaptureComponents(*CachedSceneDepthCapture, ShadowCastingActors);
	}
}

UCompositeSceneCapture2DComponent* UCompositeLayerSingleLightShadow::FindOrCreateSceneDepthCapture(ACompositeActor& InOuter) const
{
	UCompositeSceneCapture2DComponent* SceneDepthCapture = InOuter.FindOrCreateSceneCapture<UCompositeSceneCapture2DComponent>(this, 1, FName("SceneDepthCapture"));
	
	if (ensure(IsValid(SceneDepthCapture)))
	{
		SceneDepthCapture->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_RenderScenePrimitives;
		SceneDepthCapture->CaptureSource = SCS_SceneDepth;
		SceneDepthCapture->bRenderInMainRenderer = true;
		SceneDepthCapture->bIgnoreScreenPercentage = true;
	}

	return SceneDepthCapture;
}

UCompositeSceneCapture2DComponent* UCompositeLayerSingleLightShadow::FindOrCreateShadowMapCapture(ACompositeActor& InOuter) const
{
	UCompositeSceneCapture2DComponent* ShadowMapCapture = InOuter.FindOrCreateSceneCapture<UCompositeSceneCapture2DComponent>(this, 0, FName("ShadowMapCapture"));

	if (ensure(IsValid(ShadowMapCapture)))
	{
		ShadowMapCapture->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
		ShadowMapCapture->CaptureSource = SCS_SceneDepth;
		ShadowMapCapture->bRenderInMainRenderer = true;
		ShadowMapCapture->bMainViewFamily = false;
		ShadowMapCapture->bMainViewResolution = false;
		ShadowMapCapture->bMainViewCamera = false;
		ShadowMapCapture->bInheritMainViewCameraPostProcessSettings = false;
		ShadowMapCapture->bUseRayTracingIfEnabled = false;
		ShadowMapCapture->ProjectionType = ECameraProjectionMode::Orthographic;
	}

	return ShadowMapCapture;
}

void UCompositeLayerSingleLightShadow::GetShadowMatrices(UCompositeSceneCapture2DComponent& ShadowMapCapture, FMatrix44f& OutShadowMatrix, FVector4f& OutShadowInvDeviceZToWorldZ) const
{
	FMinimalViewInfo ViewInfo;
	ShadowMapCapture.GetCameraView(GetWorld()->DeltaTimeSeconds, ViewInfo);

	// Ortho clip plane values from SceneCaptureRendering.cpp
	ViewInfo.OrthoNearClipPlane = 0;
	ViewInfo.OrthoFarClipPlane = UE_FLOAT_HUGE_DISTANCE / 4.0f;

	FMatrix ViewRotationMatrix = FInverseRotationMatrix(ViewInfo.Rotation) * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	FVector LocalViewOrigin = ViewInfo.Location;
	if (!ViewRotationMatrix.GetOrigin().IsNearlyZero(0.0f))
	{
		LocalViewOrigin += ViewRotationMatrix.InverseTransformPosition(FVector::ZeroVector);
		ViewRotationMatrix = ViewRotationMatrix.RemoveTranslation();
	}
	const FMatrix ShadowViewMatrix = FTranslationMatrix(-LocalViewOrigin) * ViewRotationMatrix;
	const FMatrix ShadowProjectionMatrix = AdjustProjectionMatrixForRHI(ViewInfo.CalculateProjectionMatrix());
	
	//TODO: Max subject depth logic will need to be updated once support perspective projection.
	float MaxSubjectDepth = 1.0f;
	float InvMaxSubjectDepth = 1.0f / MaxSubjectDepth; 
	const float ShadowResolutionFractionX = 0.5f;
	const float ShadowResolutionFractionY = 0.5f;

	// Note: See FProjectedShadowInfo::GetWorldToShadowMatrix as a reference.
	const FMatrix WorldToShadowMatrix = ShadowViewMatrix * ShadowProjectionMatrix *
		FMatrix(
			FPlane(ShadowResolutionFractionX, 0, 0, 0),
			FPlane(0, -ShadowResolutionFractionY, 0, 0),
			FPlane(0, 0, InvMaxSubjectDepth, 0),
			FPlane(ShadowResolutionFractionX, ShadowResolutionFractionY, 0, 1)
		);

	OutShadowMatrix = FMatrix44f(WorldToShadowMatrix);
	OutShadowInvDeviceZToWorldZ = CreateInvDeviceZToWorldZTransform(ShadowProjectionMatrix);
}


