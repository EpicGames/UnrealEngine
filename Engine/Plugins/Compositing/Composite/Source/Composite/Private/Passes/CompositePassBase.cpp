// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassBase.h"

#include "CompositeActor.h"

UE::CompositeCore::ResourceId FCompositeTraversalContext::FindOrCreateExternalTexture(TWeakObjectPtr<UTexture> InTexture, UE::CompositeCore::FResourceMetadata InMetadata)
{
	int32 TextureIndex = ExternalTextures.IndexOfByPredicate([&InTexture, &InMetadata](const UE::CompositeCore::FExternalTexture& InExternalTexture)
		{
			return (InExternalTexture.Texture.Get() == InTexture.Get()) && (InExternalTexture.Metadata == InMetadata);
		}
	);

	if (TextureIndex == INDEX_NONE)
	{
		ExternalTextures.Add(UE::CompositeCore::FExternalTexture{ MoveTemp(InTexture), MoveTemp(InMetadata) });

		TextureIndex = ExternalTextures.Num() - 1;
	}

	return UE::CompositeCore::MakeExternalResourceId(TextureIndex);
}

const TArray<UE::CompositeCore::FExternalTexture>& FCompositeTraversalContext::GetExternalTextures() const
{
	return ExternalTextures;
}

UCompositePassBase::UCompositePassBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Ensure instances are always transactional
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SetFlags(RF_Transactional);
	}

#if WITH_EDITOR
	DisplayName = GetClass()->GetDisplayNameText().ToString();
#endif
}

UCompositePassBase::~UCompositePassBase() = default;

bool UCompositePassBase::GetIsEnabled() const
{
	return bIsEnabled;
}

void UCompositePassBase::SetIsEnabled(bool bInEnabled)
{
	bIsEnabled = bInEnabled;
}

#if WITH_EDITOR
bool UCompositePassBase::CanEditChange(const FProperty* InProperty) const
{
	return Super::CanEditChange(InProperty);
}
#endif //WITH_EDITOR

