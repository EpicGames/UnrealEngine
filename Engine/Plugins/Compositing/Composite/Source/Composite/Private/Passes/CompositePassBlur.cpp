// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassBlur.h"

#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"

DECLARE_GPU_STAT_NAMED(FCompositeBlur, TEXT("Composite.Blur"));

class FCompositePassBlurShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassBlurShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassBlurShader, FGlobalShader);

	class FAlphaOnlyDim : SHADER_PERMUTATION_BOOL("ALPHA_ONLY");
	using FPermutationDomain = TShaderPermutationDomain<FAlphaOnlyDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		SHADER_PARAMETER(FVector2f, Dir)
		SHADER_PARAMETER(float, Radius)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositePassBlurShader, "/Plugin/Composite/Private/CompositeBlur.usf", "MainPS", SF_Pixel);


namespace UE::Composite::Private
{
	using namespace CompositeCore;

	FPassTexture FCompositePassBlurProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeBlur, "Composite.Blur");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
		const FScreenPassTexture& InputTexture = Inputs[0].Texture;
		FScreenPassRenderTarget IntermediateOutput = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, InputTexture.Texture->Desc, TEXT("CompositePassBlurIntermediate"));
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, InputTexture.Texture->Desc, TEXT("CompositePassBlurOutput"));
		}

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());

		FCompositePassBlurShader::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCompositePassBlurShader::FAlphaOnlyDim>(bAlphaOnly);

		{
			FCompositePassBlurShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePassBlurShader::FParameters>();
			Parameters->Input = GetScreenPassTextureInput(InputTexture, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());
			Parameters->SvPositionToInputTextureUV = (
				FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(IntermediateOutput), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
				FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(InputTexture), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
			Parameters->Dir = FVector2f(0, 1);
			Parameters->Radius = Radius.Y;
			Parameters->bInvertedAlpha = static_cast<uint32>(Metadata.bInvertedAlpha);
			Parameters->RenderTargets[0] = IntermediateOutput.GetRenderTargetBinding();

			TShaderMapRef<FCompositePassBlurShader> PixelShader(GlobalShaderMap, PermutationVector);
			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				GlobalShaderMap,
				RDG_EVENT_NAME("Composite.Blur.Vertical (%dx%d)", IntermediateOutput.ViewRect.Width(), IntermediateOutput.ViewRect.Height()),
				PixelShader,
				Parameters,
				IntermediateOutput.ViewRect
			);
		}

		{
			FCompositePassBlurShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePassBlurShader::FParameters>();
			Parameters->Input = GetScreenPassTextureInput(IntermediateOutput, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());
			Parameters->SvPositionToInputTextureUV = (
				FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
				FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(IntermediateOutput), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
			Parameters->Dir = FVector2f(1, 0);
			Parameters->Radius = Radius.X;
			Parameters->bInvertedAlpha = static_cast<uint32>(Metadata.bInvertedAlpha);
			Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

			TShaderMapRef<FCompositePassBlurShader> PixelShader(GlobalShaderMap, PermutationVector);
			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				GlobalShaderMap,
				RDG_EVENT_NAME("Composite.Blur.Horizontal (%dx%d)", Output.ViewRect.Width(), Output.ViewRect.Height()),
				PixelShader,
				Parameters,
				Output.ViewRect
			);
		}

		return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}
}

UCompositePassBlur::UCompositePassBlur(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Method(ECompositePassBlurMethod::Gaussian)
	, RadiusXY(1, 1)
	, bAlphaOnly(false)
{
}

UCompositePassBlur::~UCompositePassBlur() = default;

void UCompositePassBlur::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Radius_DEPRECATED > 0)
	{
		RadiusXY = FIntPoint(Radius_DEPRECATED, Radius_DEPRECATED);
		Radius_DEPRECATED = 0;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

bool UCompositePassBlur::GetIsActive() const
{
	return Super::GetIsActive() && (RadiusXY.X > 1 || RadiusXY.Y > 1);
}

FCompositeCorePassProxy* UCompositePassBlur::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	FCompositePassBlurProxy* Proxy = InFrameAllocator.Create<FCompositePassBlurProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->Radius = RadiusXY;
	Proxy->bAlphaOnly = bAlphaOnly;

	return Proxy;
}
