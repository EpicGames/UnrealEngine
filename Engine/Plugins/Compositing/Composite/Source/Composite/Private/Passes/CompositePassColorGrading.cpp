// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassColorGrading.h"

#include "PixelShaderUtils.h"
#include "SceneManagement.h"
#include "ScreenPass.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Passes/CompositeCorePassProxy.h"
#include "SceneView.h"

DECLARE_GPU_STAT_NAMED(FCompositeColorGrading, TEXT("Composite.ColorGrading"));

BEGIN_SHADER_PARAMETER_STRUCT(FColorGradingPerRangeParameters, )
	SHADER_PARAMETER(FVector4f, Saturation)
	SHADER_PARAMETER(FVector4f, Contrast)
	SHADER_PARAMETER(FVector4f, Gamma)
	SHADER_PARAMETER(FVector4f, Gain)
	SHADER_PARAMETER(FVector4f, Offset)
END_SHADER_PARAMETER_STRUCT()

class FCompositePassColorGradingShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassColorGradingShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassColorGradingShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_STRUCT_REF(FWorkingColorSpaceShaderParameters, WorkingColorSpace)
		SHADER_PARAMETER_STRUCT(FColorGradingPerRangeParameters, Global)
		SHADER_PARAMETER_STRUCT(FColorGradingPerRangeParameters, Shadows)
		SHADER_PARAMETER_STRUCT(FColorGradingPerRangeParameters, Midtones)
		SHADER_PARAMETER_STRUCT(FColorGradingPerRangeParameters, Highlights)
		SHADER_PARAMETER(uint32, bInputIsPremultiplied)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		SHADER_PARAMETER(float, ShadowsMax)
		SHADER_PARAMETER(float, HighlightsMin)
		SHADER_PARAMETER(float, HighlightsMax)
		SHADER_PARAMETER(uint32, bIsTemperatureWhiteBalance)
		SHADER_PARAMETER(float, WhiteTemp)
		SHADER_PARAMETER(float, WhiteTint)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FColorGradingPerRangeParameters ToParameters(const FColorGradePerRangeSettings& InParams)
	{
		FColorGradingPerRangeParameters Output;
		Output.Saturation = FVector4f(InParams.Saturation);
		Output.Contrast = FVector4f(InParams.Contrast);
		Output.Gamma = FVector4f(InParams.Gamma);
		Output.Gain = FVector4f(InParams.Gain);
		Output.Offset = FVector4f(InParams.Offset);

		return Output;
	}
};
IMPLEMENT_GLOBAL_SHADER(FCompositePassColorGradingShader, "/Plugin/Composite/Private/CompositeColorGrading.usf", "MainPS", SF_Pixel);

namespace UE::Composite::Private
{
	FColorGradingCompositePassProxy::FColorGradingCompositePassProxy(CompositeCore::FPassInputDeclArray InPassDeclaredInputs)
		: FCompositeCorePassProxy(MoveTemp(InPassDeclaredInputs))
	{
	}

	CompositeCore::FPassTexture FColorGradingCompositePassProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const CompositeCore::FPassInputArray& Inputs, const CompositeCore::FPassContext& PassContext) const
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeColorGrading, "Composite.ColorGrading");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
		const FScreenPassTexture& Input = Inputs[0].Texture;
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		// If the override output is provided, it means that this is the last pass in post processing.
		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("ColorGradingCompositePass"));
		}

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
		FCompositePassColorGradingShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePassColorGradingShader::FParameters>();
		FRHISamplerState* InputSampler = Metadata.GetSamplerState();
		Parameters->Input = GetScreenPassTextureInput(Input, InputSampler);
		Parameters->WorkingColorSpace = GDefaultWorkingColorSpaceUniformBuffer.GetUniformBufferRef();
		Parameters->Global = FCompositePassColorGradingShader::ToParameters(ColorGradingSettings.Global);
		Parameters->Shadows = FCompositePassColorGradingShader::ToParameters(ColorGradingSettings.Shadows);
		Parameters->Midtones = FCompositePassColorGradingShader::ToParameters(ColorGradingSettings.Midtones);
		Parameters->Highlights = FCompositePassColorGradingShader::ToParameters(ColorGradingSettings.Highlights);
		Parameters->bInputIsPremultiplied = bInputIsPremultiplied;
		Parameters->bInvertedAlpha = static_cast<uint32>(Metadata.bInvertedAlpha);
		Parameters->ShadowsMax = ColorGradingSettings.ShadowsMax;
		Parameters->HighlightsMin = ColorGradingSettings.HighlightsMin;
		Parameters->HighlightsMax = ColorGradingSettings.HighlightsMax;
		Parameters->bIsTemperatureWhiteBalance = (TemperatureSettings.TemperatureType == ETemperatureMethod::TEMP_WhiteBalance);
		Parameters->WhiteTemp = TemperatureSettings.WhiteTemp;
		Parameters->WhiteTint = TemperatureSettings.WhiteTint;
		Parameters->SvPositionToInputTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Input), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
		Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FCompositePassColorGradingShader> PixelShader(GlobalShaderMap);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			GlobalShaderMap,
			RDG_EVENT_NAME("Composite.ColorGrading (%dx%d)", Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			Parameters,
			Output.ViewRect
		);

		return CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}
} // End namespace UE::Composite::Private

UCompositePassColorGrading::UCompositePassColorGrading(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bInputIsPremultiplied{true}
	, TemperatureSettings{}
	, ColorGradingSettings{}
{
}

UCompositePassColorGrading::~UCompositePassColorGrading() = default;

FCompositeCorePassProxy* UCompositePassColorGrading::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	FColorGradingCompositePassProxy* Proxy = InFrameAllocator.Create<FColorGradingCompositePassProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->TemperatureSettings = TemperatureSettings;
	Proxy->ColorGradingSettings = ColorGradingSettings;
	Proxy->bInputIsPremultiplied = bInputIsPremultiplied;

	return Proxy;
}

