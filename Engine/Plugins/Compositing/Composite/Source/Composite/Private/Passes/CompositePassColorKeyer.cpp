// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassColorKeyer.h"

#include "Engine/Texture.h"
#include "Passes/CompositeCorePassProxy.h"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "SystemTextures.h"
#include "TextureResource.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "SceneView.h"

#if WITH_EDITOR
#include "Application/ThrottleManager.h"
#endif

DECLARE_GPU_STAT_NAMED(FCompositeDenoise, TEXT("Composite.Denoise"));
DECLARE_GPU_STAT_NAMED(FCompositeColorKeyer, TEXT("Composite.ColorKeyer"));

class FCompositePassDenoiseShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassDenoiseShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassDenoiseShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), ThreadGroupSize);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static const uint32 ThreadGroupSize = 16;
};

class FCompositePassColorKeyerShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassColorKeyerShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassColorKeyerShader, FGlobalShader);

	class FUseCleanPlate : SHADER_PERMUTATION_BOOL("USE_CLEAN_PLATE");
	class FDespillOnly : SHADER_PERMUTATION_BOOL("USE_DESPILL_ONLY");
	class FUseChromaShift : SHADER_PERMUTATION_BOOL("USE_CHROMA_SHIFT");
	using FPermutationDomain = TShaderPermutationDomain<FUseCleanPlate, FDespillOnly, FUseChromaShift>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, CleanPlate)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		SHADER_PARAMETER(FVector3f, KeyColor)
		SHADER_PARAMETER(FVector4f, Params0)
		SHADER_PARAMETER(FVector4f, Params1)
		SHADER_PARAMETER(uint32, KeyerMode)
		SHADER_PARAMETER(uint32, bDisplaySpaceLuminance)
		SHADER_PARAMETER(uint32, ScreenType)
		SHADER_PARAMETER(uint32, Visualization)
		SHADER_PARAMETER(uint32, bPreserveVignetteAfterKey)
		SHADER_PARAMETER(uint32, bInvertAlpha)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCompositePassDenoiseShader, "/Plugin/Composite/Private/CompositeDenoise.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCompositePassColorKeyerShader, "/Plugin/Composite/Private/CompositeColorKeyer.usf", "MainPS", SF_Pixel);

static FScreenPassTexture AddColorKeyerDenoisePass(FRDGBuilder& GraphBuilder, const FScreenPassTexture& Input, ERHIFeatureLevel::Type FeatureLevel)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeDenoise, "Composite.Denoise");

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
	const FIntPoint TextureSize = Input.ViewRect.Size();

	const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(TextureSize, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
	FScreenPassRenderTarget Output = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("CompositeTextureDenoised")), FIntRect(FIntPoint::ZeroValue, TextureSize), ERenderTargetLoadAction::ENoAction);

	FCompositePassDenoiseShader::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePassDenoiseShader::FParameters>();
	// Always use point sampling: denoising needs exact texel values for its spatial filter.
	PassParameters->Input = GetScreenPassTextureInput(Input, TStaticSamplerState<SF_Point>::GetRHI());
	PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(Output.Texture);

	TShaderMapRef<FCompositePassDenoiseShader> ComputeShader(GlobalShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Composite.Denoise (%dx%d)", TextureSize.X, TextureSize.Y),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(TextureSize, FCompositePassDenoiseShader::ThreadGroupSize)
	);

	return Output;
}

namespace UE::Composite::Private
{
	using namespace CompositeCore;

	FPassTexture FCompositePassColorKeyerProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeColorKeyer, "Composite.ColorKeyer");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
		FScreenPassTexture Input = Inputs[0].Texture;
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		if (DenoiseMethod != ECompositeDenoiseMethod::None)
		{
			Input = AddColorKeyerDenoisePass(GraphBuilder, Input, InView.GetFeatureLevel());
		}

		// If the override output is provided, it means that this is the last pass in post processing.
		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("ColorKeyerCompositePass"));
		}

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
		const FIntPoint OutputSize = Output.ViewRect.Size();

		FRHISamplerState* InputSampler = Metadata.GetSamplerState();

		FCompositePassColorKeyerShader::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePassColorKeyerShader::FParameters>();
		PassParameters->Input = GetScreenPassTextureInput(Input, InputSampler);
		PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));
		PassParameters->SvPositionToInputTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Input), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
		PassParameters->KeyColor = FVector3f(KeyColor);
		PassParameters->Params0 = Params0;
		PassParameters->Params1 = Params1;
		PassParameters->KeyerMode = static_cast<uint32>(KeyerMode);
		PassParameters->bDisplaySpaceLuminance = static_cast<uint32>(bDisplaySpaceLuminance);
		PassParameters->ScreenType = static_cast<uint32>(ScreenType);
		PassParameters->Visualization = static_cast<uint32>(Visualization);
		PassParameters->bPreserveVignetteAfterKey = static_cast<uint32>(bPreserveVignetteAfterKey);
		PassParameters->bInvertAlpha = static_cast<uint32>(bInvertAlpha);
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		FRDGTextureRef CleanPlateResource = nullptr;
		PassParameters->CleanPlate.Texture = GSystemTextures.GetBlackDummy(GraphBuilder);
		// Clean plate is a separate reference texture not driven by the input metadata; bilinear is always appropriate.
		PassParameters->CleanPlate.Sampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

		TStrongObjectPtr<UTexture> CleanPlateTexturePtr = CleanPlateWeakPtr.Pin();
		if (CleanPlateTexturePtr.IsValid())
		{
			const FTextureResource* TextureResource = CleanPlateTexturePtr->GetResource();
			if (TextureResource != nullptr)
			{
				FRHITexture* TextureRHI = TextureResource->GetTexture2DRHI();
				if (TextureRHI != nullptr)
				{
					CleanPlateResource = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(TextureRHI, TEXT("CompositeCleanPlateTexture")));
					GraphBuilder.UseInternalAccessMode(CleanPlateResource);

					PassParameters->CleanPlate.Texture = CleanPlateResource;
				}
			}
		}

		FCompositePassColorKeyerShader::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCompositePassColorKeyerShader::FUseCleanPlate>(CleanPlateResource != nullptr);
		PermutationVector.Set<FCompositePassColorKeyerShader::FDespillOnly>(bDespillOnly);
		PermutationVector.Set<FCompositePassColorKeyerShader::FUseChromaShift>(!FMath::IsNearlyZero(Params1.W));

		TShaderMapRef<FCompositePassColorKeyerShader> PixelShader(GlobalShaderMap, PermutationVector);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			GlobalShaderMap,
			RDG_EVENT_NAME("Composite.ColorKeyer (%dx%d)", OutputSize.X, OutputSize.Y),
			PixelShader,
			PassParameters,
			Output.ViewRect
		);

		if (CleanPlateResource != nullptr)
		{
			GraphBuilder.UseExternalAccessMode(CleanPlateResource, ERHIAccess::SRVMask);
		}

		return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}
}

UCompositePassColorKeyer::UCompositePassColorKeyer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ScreenType(ECompositeColorKeyerScreenType::Green)
	, KeyColor(FLinearColor::Green)
	, CleanPlate(nullptr)
	, RedWeight(0.5f)
	, GreenWeight(0.5f)
	, BlueWeight(0.5f)
	, AlphaThreshold(0.0f, 1.0f)
	, DespillStrength(0.0f)
	, DevignetteStrength(0.0f)
	, bPreserveVignetteAfterKey(true)
	, DenoiseMethod(ECompositeDenoiseMethod::None)
	, ChromaShift(0.0f)
	, Visualization(ECompositeColorKeyerVisualization::Key)
	, bDespillOnly(false)
	, bInvertAlpha(false)
{ 
}

UCompositePassColorKeyer::~UCompositePassColorKeyer() = default;

FCompositeCorePassProxy* UCompositePassColorKeyer::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	FCompositePassColorKeyerProxy* Proxy = InFrameAllocator.Create<FCompositePassColorKeyerProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->KeyColor = KeyColor;
	Proxy->Params0 = FVector4f(RedWeight, GreenWeight, BlueWeight, DespillStrength);
	Proxy->Params1 = FVector4f(AlphaThreshold.X, AlphaThreshold.Y, DevignetteStrength, ChromaShift);
	Proxy->ScreenType = ScreenType;
	Proxy->bPreserveVignetteAfterKey = bPreserveVignetteAfterKey;
	Proxy->DenoiseMethod = DenoiseMethod;
	Proxy->bDespillOnly = bDespillOnly;
	Proxy->bInvertAlpha = bInvertAlpha;
	Proxy->Visualization = Visualization;

	if (KeyerSource == ECompositeKeyerSource::CleanPlate)
	{
		Proxy->CleanPlateWeakPtr = CleanPlate;
	}

	return Proxy;
}

void UCompositePassColorKeyer::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Set the initial value of KeyerSource to old behavior that assumed clean plate texture should be used if set
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CompositePassKeyerSource)
	{
		if (CleanPlate != nullptr)
		{
			KeyerSource = ECompositeKeyerSource::CleanPlate;
		}
	}
#endif
}

#if WITH_EDITOR
void UCompositePassColorKeyer::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (!PropertyThatWillChange)
	{
		return;
	}

	const FName PropertyName = PropertyThatWillChange->GetFName();
	const bool bIsBooleanProperty = CastField<const FBoolProperty>(PropertyThatWillChange) != nullptr;

	if (PropertyName != GET_MEMBER_NAME_CHECKED(ThisClass, bIsEnabled)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(ThisClass, ScreenType)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(ThisClass, KeyColor)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(ThisClass, CleanPlate)
		&& !bIsBooleanProperty
		)
	{
		/**
		* For properties not included in the list above, we want to keep the editor ticking so that
		* result from sliding their (SNumericEntryBox) values can be observed in real-time.
		* 
		* Unlike component transform customizations which prevent throttling explicitly
		* (see .PreventThrottling(true)) we don't appear to have a metadata property for this.
		* 
		* As an alternative solution we directly disable throttling from the slate manager,
		* and re-enable it once the value is set.
		*/
		FSlateThrottleManager::Get().DisableThrottle(true);
		bEditorThrottleDisabled = true;
	}
}

void UCompositePassColorKeyer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (bEditorThrottleDisabled && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		FSlateThrottleManager::Get().DisableThrottle(false);
		bEditorThrottleDisabled = false;
	}

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, ScreenType))
	{
		switch (ScreenType)
		{
		case ECompositeColorKeyerScreenType::Red:
			KeyColor = FLinearColor::Red;
			break;
		case ECompositeColorKeyerScreenType::Green:
			KeyColor = FLinearColor::Green;
			break;
		case ECompositeColorKeyerScreenType::Blue:
			KeyColor = FLinearColor::Blue;
			break;

		default:
			checkNoEntry();
			break;
		}
	}
}
#endif

UCompositePassLumaKeyer::UCompositePassLumaKeyer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LumaRange(0.0f, 1.0f)
	, Visualization(ECompositeColorKeyerVisualization::Key)
	, bInvertAlpha(false)
	, bDisplaySpaceLuminance(true)
	, DenoiseMethod(ECompositeDenoiseMethod::None)
{
}

UCompositePassLumaKeyer::~UCompositePassLumaKeyer() = default;

FCompositeCorePassProxy* UCompositePassLumaKeyer::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	FCompositePassColorKeyerProxy* Proxy = InFrameAllocator.Create<FCompositePassColorKeyerProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->KeyerMode = ECompositeColorKeyerMode::Luma;
	Proxy->DenoiseMethod = DenoiseMethod;
	Proxy->bInvertAlpha = bInvertAlpha;
	Proxy->Visualization = Visualization;
	// Pack LumaRange into Params1.xy (the same slots chroma uses for AlphaThreshold).
	Proxy->Params1 = FVector4f(LumaRange.X, LumaRange.Y, 0.0f, 0.0f);
	Proxy->bDisplaySpaceLuminance = bDisplaySpaceLuminance;

	return Proxy;
}

