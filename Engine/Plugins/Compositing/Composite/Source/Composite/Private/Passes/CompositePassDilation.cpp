// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassDilation.h"

#include "Passes/CompositeCorePassProxy.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"

DECLARE_GPU_STAT_NAMED(FCompositeDilation, TEXT("Composite.Dilation"));

class FCompositePassDilationShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassDilationShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassDilationShader, FGlobalShader);

	// DILATION_SIZE drives the LDS tile halo and the neighbor kernel:
	//   1 -> 8-neighbor (3x3 minus center), 1 pixel per dispatch
	//   2 -> 24-neighbor (5x5 minus center), 2 pixels per dispatch
	// The C++ dispatch loop uses DILATION_SIZE=2 greedily, then DILATION_SIZE=1 for any
	// remaining pixel - same reach as N*DILATION_SIZE=1 but up to half the dispatch count.
	class FDilationSize : SHADER_PERMUTATION_RANGE_INT("DILATION_SIZE", 1, 2);
	class FErode : SHADER_PERMUTATION_BOOL("MORPHOLOGY_ERODE");
	// 0 = ALPHA_ONLY (verbatim legacy path when threshold on), 1 = GENERAL (per-channel mask).
	class FDilationChannelsMode : SHADER_PERMUTATION_INT("DILATION_CHANNELS_MODE", 2);
	// 1 = thresholded morphology, 0 = true min/max morphology (graded output).
	class FDilationUseThreshold : SHADER_PERMUTATION_BOOL("DILATION_USE_THRESHOLD");
	using FPermutationDomain = TShaderPermutationDomain<FDilationSize, FErode, FDilationChannelsMode, FDilationUseThreshold>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWOutputTexture)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		SHADER_PARAMETER(uint32, bOpacifyOutput)
		SHADER_PARAMETER(float, AlphaThreshold)
		SHADER_PARAMETER(uint32, ChannelMask)
		SHADER_PARAMETER(uint32, bCarryRGBWithAlpha)
	END_SHADER_PARAMETER_STRUCT()

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), ThreadGroupSize);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// ALPHA_ONLY is the legacy thresholded path (also used by the holdout system); the user
		// pass routes all min/max dispatches through GENERAL. Skip ALPHA_ONLY + USE_THRESHOLD=0
		// so no dead variant is compiled.
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDilationChannelsMode>() == 0 && !PermutationVector.Get<FDilationUseThreshold>())
		{
			return false;
		}
		return FGlobalShader::ShouldCompilePermutation(Parameters);
	}

	static const uint32 ThreadGroupSize = 16;
};

IMPLEMENT_GLOBAL_SHADER(FCompositePassDilationShader, "/Plugin/CompositeCore/Private/CompositeCoreDilate.usf", "MainCS", SF_Compute);

namespace UE::Composite::Private
{

FPassTexture FCompositePassDilationProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeDilation, "Composite.Dilation");

	check(ValidateInputs(Inputs));
	check(AbsSize >= 1);

	const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
	const FScreenPassTexture& InputTexture = Inputs[0].Texture;

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	if (Output.IsValid())
	{
		if (!ensureMsgf(!!(Output.Texture->Desc.Flags & TexCreate_UAV), TEXT("CompositePassDilation: OverrideOutput texture '%s' was not created with TexCreate_UAV; ignoring override and allocating an internal output instead."), Output.Texture->Name))
		{
			Output = {};
		}
		else if (!ensureMsgf(Output.ViewRect.Size() == InputTexture.ViewRect.Size(), TEXT("CompositePassDilation: OverrideOutput viewport (%dx%d) does not match input viewport (%dx%d); ignoring override and allocating an internal output instead."), Output.ViewRect.Size().X, Output.ViewRect.Size().Y, InputTexture.ViewRect.Size().X, InputTexture.ViewRect.Size().Y))
		{
			Output = {};
		}
	}
	if (!Output.IsValid())
	{
		Output = CreateOutputRenderTarget(GraphBuilder, InView, InputTexture.ViewRect, InputTexture.Texture->Desc, TEXT("CompositeDilationOutput"));
	}

	/** Input and Output viewports must be the same size: the dilation kernel operates at a fixed
	 *  resolution throughout the ping-pong chain, and the final iteration writes to FinalComputeTexture
	 *  (copied to Output afterwards). */
	const FIntPoint TextureSize = InputTexture.ViewRect.Size();
	checkf(Output.ViewRect.Size() == TextureSize,
		TEXT("CompositePassDilation: Input viewport (%dx%d) and Output viewport (%dx%d) must be the same size."),
		TextureSize.X, TextureSize.Y, Output.ViewRect.Size().X, Output.ViewRect.Size().Y);

	const FRDGTextureDesc PingPongDesc = FRDGTextureDesc::Create2D(TextureSize, Output.Texture->Desc.Format, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
	FRDGTextureRef PingPong[2] = {
		GraphBuilder.CreateTexture(PingPongDesc, TEXT("CompositeDilationPingPong0")),
		GraphBuilder.CreateTexture(PingPongDesc, TEXT("CompositeDilationPingPong1"))
	};

	/** The final compute step writes here instead of directly to Output.Texture.
	 *  Output.Texture is RenderTargetable; D3D12 state tracking for UAV writes onto
	 *  RenderTargetable textures can race with the RHI thread (r.RHIThread.Enable 1),
	 *  corrupting the subsequent graphics SRV read in the merge pass. Using a
	 *  UAV-only intermediate (same pattern as GeometryMaskPostProcess_Blur and
	 *  PostProcessUpscale) gives RDG a clean UAV -> CopySrc -> CopyDest -> SRVGraphics chain
	 *  and avoids the issue entirely. */
	const FRDGTextureDesc FinalComputeDesc = FRDGTextureDesc::Create2D(TextureSize, Output.Texture->Desc.Format, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
	FRDGTextureRef FinalComputeTexture = GraphBuilder.CreateTexture(FinalComputeDesc, TEXT("CompositeDilationFinalCS"));

	// Pick the channel permutation. ALPHA_ONLY is only used for the thresholded alpha-only legacy
	// path (shared with the holdout dispatch). Everything else - anything with an R/G/B bit, or
	// min/max mode regardless of channel selection - routes through GENERAL.
	const bool bGeneralPermutation = !bUseThreshold || ((ChannelMask & 0x7u) != 0u);

	// Greedy dispatch: consume DILATION_SIZE=2 steps (2 pixels each) until fewer than 2 pixels
	// remain, then finish with DILATION_SIZE=1. Same total pixel reach as AbsSize×DILATION_SIZE=1
	// but up to half the number of GPU dispatches.
	// Pixel tracks pixels consumed so far (stepping by 2); Iteration counts actual dispatches.
	int32 Iteration = 0;
	for (int32 Pixel = 0; Pixel < AbsSize; Pixel += 2, ++Iteration)
	{
		const int32 StepSize = FMath::Min(AbsSize - Pixel, 2);

		// Determine read source: first iteration reads from input, subsequent alternate ping-pong.
		FScreenPassTexture ReadTexture;
		if (Iteration == 0)
		{
			ReadTexture = InputTexture;
		}
		else
		{
			const int32 ReadIndex = (Iteration - 1) % 2;
			ReadTexture = FScreenPassTexture(PingPong[ReadIndex], FIntRect(FIntPoint::ZeroValue, TextureSize));
		}

		/** Final iteration writes to the UAV-only intermediate; intermediate iterations write to ping-pong. */
		const bool bFinalIteration = (Pixel + StepSize >= AbsSize);
		FRDGTextureRef WriteTexture;
		FScreenPassTextureViewportParameters OutputViewportParams;
		if (bFinalIteration)
		{
			WriteTexture = FinalComputeTexture;
		}
		else
		{
			const int32 WriteIndex = Iteration % 2;
			WriteTexture = PingPong[WriteIndex];
		}
		OutputViewportParams = GetScreenPassTextureViewportParameters(
			FScreenPassTextureViewport(FScreenPassTexture(WriteTexture, FIntRect(FIntPoint::ZeroValue, TextureSize))));

		FCompositePassDilationShader::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCompositePassDilationShader::FDilationSize>(StepSize);
		PermutationVector.Set<FCompositePassDilationShader::FErode>(bErode);
		PermutationVector.Set<FCompositePassDilationShader::FDilationChannelsMode>(bGeneralPermutation ? 1 : 0);
		PermutationVector.Set<FCompositePassDilationShader::FDilationUseThreshold>(bUseThreshold);
		TShaderMapRef<FCompositePassDilationShader> ComputeShader(GlobalShaderMap, PermutationVector);

		FCompositePassDilationShader::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePassDilationShader::FParameters>();
		PassParameters->Input = GetScreenPassTextureInput(ReadTexture, TStaticSamplerState<SF_Point>::GetRHI());
		PassParameters->Output = OutputViewportParams;
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(WriteTexture);
		PassParameters->bInvertedAlpha = static_cast<uint32>(Metadata.bInvertedAlpha);
		PassParameters->bOpacifyOutput = 0;
		PassParameters->AlphaThreshold = Threshold;
		PassParameters->ChannelMask = ChannelMask;
		PassParameters->bCarryRGBWithAlpha = bCarryRGBWithAlpha ? 1u : 0u;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Composite.Dilation %s (%dx%d) [pix:%d/%d step:%d]", bErode ? TEXT("Erode") : TEXT("Dilate"), TextureSize.X, TextureSize.Y, Pixel, AbsSize, StepSize),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(TextureSize, FCompositePassDilationShader::ThreadGroupSize)
		);
	}

	/** Copy the final CS result from the UAV-only intermediate into the RT-capable output.
	 *  This bridges the compute UAV state on the source to copy-dest on the destination,
	 *  after which the downstream merge pass transitions cleanly to SRVGraphics. This mirrors
	 *  the engine pattern in PostProcessUpscale.cpp and GeometryMaskPostProcess_Blur.cpp. */
	AddCopyTexturePass(GraphBuilder, FinalComputeTexture, Output.Texture);

	return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
}

}  // namespace UE::Composite::Private

UCompositePassDilation::UCompositePassDilation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UCompositePassDilation::~UCompositePassDilation() = default;

bool UCompositePassDilation::GetIsActive() const
{
	return Super::GetIsActive() && Size != 0 && Channels != 0;
}

FCompositeCorePassProxy* UCompositePassDilation::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	FCompositePassDilationProxy* Proxy = InFrameAllocator.Create<FCompositePassDilationProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->AbsSize            = FMath::Max(1, FMath::Abs(Size));
	Proxy->bErode             = (Size < 0);
	Proxy->ChannelMask        = static_cast<uint32>(Channels);
	Proxy->bUseThreshold      = bUseThreshold;
	Proxy->Threshold          = Threshold;
	Proxy->bCarryRGBWithAlpha = bCarryRGBWithAlpha;

	return Proxy;
}
