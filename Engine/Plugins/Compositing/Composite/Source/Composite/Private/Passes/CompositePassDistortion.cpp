// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassDistortion.h"

#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "CameraCalibrationSubsystem.h"
#include "CameraCalibrationTypes.h"
#include "CompositeActor.h"
#include "CompositeModule.h"
#include "LensDistortionSceneViewExtension.h"
#include "ScreenPass.h"
#include "SystemTextures.h"
#include "Passes/CompositeCorePassProxy.h"
#include "PixelShaderUtils.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "SceneView.h"

class FCompositePassDistortionShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassDistortionShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassDistortionShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)
		SHADER_PARAMETER(uint32, DistortionUV)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositePassDistortionShader, "/Plugin/Composite/Private/CompositeDistortion.usf", "MainPS", SF_Pixel);

DECLARE_GPU_STAT_NAMED(FCompositeDistortion, TEXT("Composite.Distortion"));

class FDistortionCompositePassProxy : public FCompositeCorePassProxy
{
public:
	IMPLEMENT_COMPOSITE_PASS(FDistortionCompositePassProxy);

	enum class EDistortionUV
	{
		None = 0,
		Distorted = 1,
		Undistorted = 2
	};

	FDistortionCompositePassProxy(UE::CompositeCore::FPassInputDeclArray InPassDeclaredInputs)
		: FCompositeCorePassProxy(MoveTemp(InPassDeclaredInputs))
	{

	}

	UE::CompositeCore::FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const UE::CompositeCore::FPassInputArray& Inputs, const UE::CompositeCore::FPassContext& PassContext) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeDistortion, "Composite.Distortion");

		check(ValidateInputs(Inputs));

		UE::CompositeCore::FResourceMetadata Metadata = Inputs[0].Metadata;
		const FScreenPassTexture& Input = Inputs[0].Texture;
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		// If the override output is provided, it means that this is the last pass in post processing.
		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("DistortionCompositePass"));
		}

		FLensDistortionLUT ViewDistortionLUT = LensDistortion::GetLUTUnsafe(InView);
		bool bValidLensDistortion = ViewDistortionLUT.IsEnabled();

		if (!bValidLensDistortion)
		{
			// Explicitely render view distortion LUT if existing view one doesn't yet exist, i.e. in a pre-processing context.
			if (InView.Family->EngineShowFlags.LensDistortion && FPaniniProjectionConfig::IsEnabledByCVars())
			{
				const FPaniniProjectionConfig PaniniProjection = FPaniniProjectionConfig::ReadCVars();

				checkSlow(InView.bIsViewInfo);
				ViewDistortionLUT = PaniniProjection.GenerateLUTPassesUnsafe(GraphBuilder, InView);
				bValidLensDistortion = ViewDistortionLUT.IsEnabled();
			}
			else if (LensDistortionSVE)
			{
				const TOptional<EDistortionRenderingMode> RenderingMode = LensDistortionSVE->GetRenderingMode_AnyThread(InView.ViewActor.ActorUniqueId);
				if (RenderingMode.IsSet() && RenderingMode.GetValue() != EDistortionRenderingMode::TemporalSuperResolution)
				{
					UE_CALL_ONCE([]
					{
						UE_LOG(LogComposite, Warning, TEXT("Composite distortion pass requires the Lens Component's Distortion Rendering Mode to be TemporalSuperResolution."));
					});
				}

				bValidLensDistortion = LensDistortionSVE->RenderViewDistortionLUT(GraphBuilder, InView.ViewActor.ActorUniqueId, ViewDistortionLUT);
			}
		}

		if (bValidLensDistortion)
		{
			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
			FScreenPassTextureViewport Viewport = FScreenPassTextureViewport(Output);

			FCompositePassDistortionShader::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePassDistortionShader::FParameters>();
			// Always use bilinear: distortion resamples through a displacement LUT, mapping texels to fractional coordinates.
			PassParameters->Input = GetScreenPassTextureInput(Input, TStaticSamplerState<SF_Bilinear>::GetRHI());
			PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));
			if (ViewDistortionLUT.IsEnabled())
			{
				PassParameters->DistortingDisplacementTexture = ViewDistortionLUT.DistortingDisplacementTexture;
				PassParameters->UndistortingDisplacementTexture = ViewDistortionLUT.UndistortingDisplacementTexture;
			}
			else
			{
				PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
				PassParameters->DistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
			}
			PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
			PassParameters->DistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
			// Sample input UVs with the opposite of the desired output distortion
			switch (Distortion)
			{
			case ECompositeDistortion::Distort:
				PassParameters->DistortionUV = static_cast<uint32>(EDistortionUV::Undistorted);
				break;
			case ECompositeDistortion::Undistort:
				PassParameters->DistortionUV = static_cast<uint32>(EDistortionUV::Distorted);
				break;
			default:
				checkNoEntry();
				break;
			}

			PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

			TShaderMapRef<FCompositePassDistortionShader> PixelShader(GlobalShaderMap);
			
			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				GlobalShaderMap,
				RDG_EVENT_NAME("Composite.Distortion (%dx%d)", Viewport.Extent.X, Viewport.Extent.Y),
				PixelShader,
				PassParameters,
				Output.ViewRect
			);

			Metadata.bDistorted = (Distortion == ECompositeDistortion::Distort);
		}
		else
		{
			AddDrawTexturePass(GraphBuilder, InView, Input, Output);
		}

		return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}

	TSharedPtr<FLensDistortionSceneViewExtension, ESPMode::ThreadSafe> LensDistortionSVE;
	ECompositeDistortion Distortion;
};

UCompositePassDistortion::UCompositePassDistortion(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Distortion = ECompositeDistortion::Undistort;
}

UCompositePassDistortion::~UCompositePassDistortion() = default;

void UCompositePassDistortion::PostInitProperties()
{
	Super::PostInitProperties();

	UCameraCalibrationSubsystem* Subsystem = GEngine ? GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>() : nullptr;
	if (Subsystem)
	{
		LensDistortionSVE = Subsystem->GetSceneViewExtension();
	}
}

bool UCompositePassDistortion::GetIsActive() const
{
	bool bIsDistortionActive = false;

	if (Super::GetIsActive())
	{
		static IConsoleVariable* PaniniCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LensDistortion.Panini.D"));
		if (LensDistortionSVE && LensDistortionSVE->HasDistortionState_AnyThread())
		{
			bIsDistortionActive = true;
		}
		else if (PaniniCVar->GetFloat() > 0.01f)
		{
			bIsDistortionActive = true;
		}
	}

	return bIsDistortionActive;
}

FCompositeCorePassProxy* UCompositePassDistortion::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	FDistortionCompositePassProxy* Proxy = InFrameAllocator.Create<FDistortionCompositePassProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->Distortion = Distortion;
	Proxy->LensDistortionSVE = LensDistortionSVE;

	return Proxy;
}

