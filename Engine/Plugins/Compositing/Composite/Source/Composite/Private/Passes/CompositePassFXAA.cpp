// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassFXAA.h"

#include "Passes/CompositeCorePassFXAAProxy.h"

UCompositePassFXAA::UCompositePassFXAA(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Quality(5)
{
}

UCompositePassFXAA::~UCompositePassFXAA() = default;

FCompositeCorePassProxy* UCompositePassFXAA::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::CompositeCore;

	FFXAAPassProxy* Proxy = InFrameAllocator.Create<FFXAAPassProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->QualityOverride = Quality;
	Proxy->bDisplayTransform = bDisplayTransform;

	return Proxy;
}

