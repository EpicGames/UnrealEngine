// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassMasking.h"
#include "Passes/CompositeCorePassProxy.h"

#include "CompositeActor.h"
#include "CompositeRenderTargetPool.h"
#include "Components/CompositeSceneCapture2DComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"

DECLARE_GPU_STAT_NAMED(FCompositeMasking, TEXT("Composite.Masking"));

class FCompositePassMaskingShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassMaskingShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassMaskingShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FInputPassTextureParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		SHADER_PARAMETER(uint32, SourceEncoding)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FInputPassTextureParameters, Input)
		SHADER_PARAMETER_STRUCT(FInputPassTextureParameters, Mask)
		SHADER_PARAMETER(uint32, MaskSourceChannel)
		SHADER_PARAMETER(uint32, bInputIsPremultiplied)
		SHADER_PARAMETER(uint32, bPremultipliedInDisplaySpace)
		SHADER_PARAMETER(uint32, bInvert)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		SHADER_PARAMETER(FScreenTransform, SvPositionToMaskTextureUV)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositePassMaskingShader, "/Plugin/Composite/Private/CompositeMasking.usf", "MainPS", SF_Pixel);


namespace UE::Composite::Private
{
	using namespace CompositeCore;

	static FCompositePassMaskingShader::FInputPassTextureParameters GetPassTextureParameters(const UE::CompositeCore::FPassInput& Input)
	{
		FCompositePassMaskingShader::FInputPassTextureParameters Parameters;
		Parameters.Texture = Input.Texture.Texture;
		Parameters.Sampler = Input.Metadata.GetSamplerState();
		Parameters.bInvertedAlpha = static_cast<uint32>(Input.Metadata.bInvertedAlpha);
		Parameters.SourceEncoding = static_cast<uint32>(Input.Metadata.Encoding);
		return Parameters;
	}

	FPassTexture FMaskingPassProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeMasking, "Composite.Masking");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
		const FScreenPassTexture& InputTexture = Inputs[0].Texture;
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		// If the override output is provided, it means that this is the last pass in post processing.
		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, InputTexture.Texture->Desc, TEXT("MaskingCompositePass"));
		}

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
		FCompositePassMaskingShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePassMaskingShader::FParameters>();
		Parameters->Input = GetPassTextureParameters(Inputs[0]);
		Parameters->Mask = GetPassTextureParameters(Inputs[1]);
		Parameters->MaskSourceChannel = MaskSourceChannel;
		Parameters->bInputIsPremultiplied = static_cast<uint32>(bInputIsPremultiplied);
		Parameters->bPremultipliedInDisplaySpace = static_cast<uint32>(bPremultipliedInDisplaySpace);
		Parameters->bInvert = static_cast<uint32>(bInvert);
		Parameters->SvPositionToInputTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Inputs[0].Texture), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
		Parameters->SvPositionToMaskTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Inputs[1].Texture), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
		Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FCompositePassMaskingShader> PixelShader(GlobalShaderMap);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			GlobalShaderMap,
			RDG_EVENT_NAME("Composite.Masking (%dx%d)", Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			Parameters,
			Output.ViewRect
		);

		return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}
}

UCompositePassMasking::UCompositePassMasking(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MaskingMode{ ECompositeMaskingMode::Texture }
	, MaskTexture{}
	, MaskSource{ ECompositeColorChannel::Red }
	, RenderTargetResolution{ FCompositeRenderTargetPool::DefaultSize }
	, bVisibleInSceneCaptureOnly{ true }
	, bInputIsPremultiplied{ true }
	, bPremultipliedInDisplaySpace{ false }
	, bInvert{ false }
{
}

UCompositePassMasking::~UCompositePassMasking() = default;

ETickableTickType UCompositePassMasking::GetTickableTickType() const
{
	return HasAnyFlags(RF_ClassDefaultObject) ? ETickableTickType::Never : ETickableTickType::Conditional;
}

bool UCompositePassMasking::IsTickable() const
{
	if (HasAnyFlags(RF_BeginDestroyed) || MaskingMode != ECompositeMaskingMode::Geometry)
	{
		return false;
	}

	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	return IsValid(CompositeActor) && !CompositeActor->HasAnyFlags(RF_ClassDefaultObject);
}

UWorld* UCompositePassMasking::GetTickableGameObjectWorld() const
{
	return GetWorld();
}

TStatId UCompositePassMasking::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UCompositePassMasking, STATGROUP_Tickables);
}

void UCompositePassMasking::Tick(float DeltaTime)
{
	// Per-primitive bVisible state set by UpdatePrimitiveVisibilityState isn't replayed by the
	// per-frame ShowOnlyComponents refresh below, so re-apply it when a stale spawnable resolves.
	if (SpawnableBindings.TickResolveStale(MaskActors, GetTickableGameObjectWorld(), GetUniqueID())
		&& MaskingMode == ECompositeMaskingMode::Geometry)
	{
		UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);
	}

	UCompositeSceneCapture2DComponent* SceneCaptureComponent = FindOrCreateSceneCaptureComponent();
	if (!IsValid(SceneCaptureComponent))
	{
		return;
	}

	// Refresh ShowOnlyComponents every frame to handle primitive recreation.
	SceneCaptureComponent->ShowOnlyComponents.Reset(MaskActors.Num());
	for (const TSoftObjectPtr<AActor>& SoftActor : MaskActors)
	{
		if (const AActor* Actor = SoftActor.Get())
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
				{
					SceneCaptureComponent->ShowOnlyComponents.Add(PrimitiveComponent);
				}
			}
		}
	}
}

UCompositeSceneCapture2DComponent* UCompositePassMasking::FindOrCreateSceneCaptureComponent() const
{
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	if (!IsValid(CompositeActor))
	{
		return nullptr;
	}

	UCompositeSceneCapture2DComponent* CaptureComponent = CompositeActor->FindOrCreateSceneCapture<UCompositeSceneCapture2DComponent>(this, 0, FName("MaskingGeometryCapture"));
	if (IsValid(CaptureComponent) && !CaptureComponent->bRenderInMainRenderer)
	{
		// Configure as custom render pass inlined in the main renderer (shared scene setup, reduced overhead).
		CaptureComponent->CaptureSource = ESceneCaptureSource::SCS_SceneColorHDR;
		CaptureComponent->bRenderInMainRenderer = true;
		CaptureComponent->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
	}

	return CaptureComponent;
}

void UCompositePassMasking::UpdatePrimitiveVisibilityState(bool bInVisibleInSceneCaptureOnly) const
{
	for (const TSoftObjectPtr<AActor>& SoftActor : MaskActors)
	{
		if (const AActor* Actor = SoftActor.Get())
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
				{
					if (PrimitiveComponent->bVisibleInSceneCaptureOnly != bInVisibleInSceneCaptureOnly)
					{
						PrimitiveComponent->Modify();
						PrimitiveComponent->SetVisibleInSceneCaptureOnly(bInVisibleInSceneCaptureOnly);
					}
				}
			}
		}
	}
}

void UCompositePassMasking::RestorePrimitiveVisibilityState() const
{
	// Restores to CDO default rather than caching original per-component state.
	// Primitives may be recreated or modified externally at any time, making cached state unreliable.
	UPrimitiveComponent* CDO = UPrimitiveComponent::StaticClass()->GetDefaultObject<UPrimitiveComponent>();
	UpdatePrimitiveVisibilityState(CDO->bVisibleInSceneCaptureOnly);
}

void UCompositePassMasking::BeginDestroy()
{
	if (MaskingMode == ECompositeMaskingMode::Geometry)
	{
		if (ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>())
		{
			CompositeActor->DestroySceneCaptures(this);
		}

		RestorePrimitiveVisibilityState();
	}

	Super::BeginDestroy();
}

void UCompositePassMasking::SetMaskingMode(ECompositeMaskingMode InMaskingMode)
{
	if (MaskingMode == InMaskingMode)
	{
		return;
	}

	// Clean up previous geometry mode state.
	if (MaskingMode == ECompositeMaskingMode::Geometry)
	{
		if (ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>())
		{
			CompositeActor->DestroySceneCaptures(this);
		}

		RestorePrimitiveVisibilityState();
	}

	MaskingMode = InMaskingMode;

	// Initialize new geometry mode state.
	if (MaskingMode == ECompositeMaskingMode::Geometry)
	{
		UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);
	}
}

const TArray<TSoftObjectPtr<AActor>> UCompositePassMasking::GetMaskActors() const
{
	return MaskActors;
}

void UCompositePassMasking::SetMaskActors(TArray<TSoftObjectPtr<AActor>> InMaskActors)
{
	SpawnableBindings.CachePreEditState(MaskActors);
	SetMaskActorsInternal(MoveTemp(InMaskActors));
	SpawnableBindings.SyncOnPropertyChange(MaskActors, GetTickableGameObjectWorld());
}

void UCompositePassMasking::SetMaskActorsInternal(TArray<TSoftObjectPtr<AActor>> InMaskActors)
{
	if (MaskingMode == ECompositeMaskingMode::Geometry)
	{
		RestorePrimitiveVisibilityState();
	}

	MaskActors = MoveTemp(InMaskActors);

	if (MaskingMode == ECompositeMaskingMode::Geometry)
	{
		UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);
	}
}

bool UCompositePassMasking::IsVisibleInSceneCaptureOnly() const
{
	return bVisibleInSceneCaptureOnly;
}

void UCompositePassMasking::SetVisibleInSceneCaptureOnly(bool bInVisible)
{
	bVisibleInSceneCaptureOnly = bInVisible;

	if (MaskingMode == ECompositeMaskingMode::Geometry)
	{
		UpdatePrimitiveVisibilityState(bVisibleInSceneCaptureOnly);
	}
}

void UCompositePassMasking::SetRenderTargetResolution(FIntPoint InRenderTargetResolution)
{
	RenderTargetResolution.X = FMath::Max(InRenderTargetResolution.X, 1);
	RenderTargetResolution.Y = FMath::Max(InRenderTargetResolution.Y, 1);
}

UCompositePassUltimatteMasking::UCompositePassUltimatteMasking(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MaskTexture{}
	, bInvert{ true }
{
}

bool UCompositePassUltimatteMasking::GetIsActive() const
{
	return GetIsEnabled() && IsValid(MaskTexture);
}

FCompositeCorePassProxy* UCompositePassUltimatteMasking::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;

	if (!ensureMsgf(IsValid(MaskTexture), TEXT("Texture is expected to be valid for active pass.")))
	{
		return nullptr;
	}

	FResourceMetadata MaskMetadata = { .DebugName = TEXT("MaskTexture") };
	MaskMetadata.Filter = (MaskTexture->Filter == TF_Nearest) ? SF_Point : SF_Bilinear;

	const ResourceId TextureId = InContext.FindOrCreateExternalTexture(MaskTexture, MaskMetadata);

	FPassInputDecl MaskInputDecl;
	MaskInputDecl.Set<FPassExternalResourceDesc>({ TextureId });

	FPassInputDeclArray Inputs = { InputDecl, MaskInputDecl };
	FMaskingPassProxy* Proxy = InFrameAllocator.Create<FMaskingPassProxy>(Inputs);
	Proxy->MaskSourceChannel = static_cast<uint32>(ECompositeColorChannel::Red);
	Proxy->bInputIsPremultiplied = true;
	Proxy->bPremultipliedInDisplaySpace = true;
	Proxy->bInvert = bInvert;

	return Proxy;
}

#if WITH_EDITOR
void UCompositePassMasking::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropertyName = PropertyAboutToChange->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, MaskingMode))
	{
		PreEditMaskingMode = MaskingMode;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, MaskActors))
	{
		SpawnableBindings.CachePreEditState(MaskActors);

		if (MaskingMode == ECompositeMaskingMode::Geometry)
		{
			RestorePrimitiveVisibilityState();
		}
	}
}

void UCompositePassMasking::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, MaskingMode))
	{
		/**
		 * The editor has already written the new value into MaskingMode.
		 * Restore the old value so SetMaskingMode sees the actual transition.
		 */
		ECompositeMaskingMode NewMode = MaskingMode;
		MaskingMode = PreEditMaskingMode;
		SetMaskingMode(NewMode);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, MaskActors))
	{
		SetMaskActorsInternal(MoveTemp(MaskActors));
		SpawnableBindings.SyncOnPropertyChange(MaskActors, GetTickableGameObjectWorld());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bVisibleInSceneCaptureOnly))
	{
		SetVisibleInSceneCaptureOnly(bVisibleInSceneCaptureOnly);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, RenderTargetResolution))
	{
		SetRenderTargetResolution(RenderTargetResolution);
	}
}

void UCompositePassMasking::PostEditUndo()
{
	Super::PostEditUndo();

	// MaskActors and SpawnableBindings.Bindings are restored by the transaction system, but the
	// per-primitive bVisible flags that Restore/UpdatePrimitiveVisibilityState mutate are not part
	// of the transaction. Other Composite layers (Plate, SceneCapture, etc.) self-heal because they
	// reapply visibility/holdout state unconditionally in Tick; this pass does not, so we explicitly
	// re-run the internal setter to bring primitive visibility back in sync with the restored array.
	SetMaskActorsInternal(MoveTemp(MaskActors));
}
#endif

bool UCompositePassMasking::GetIsActive() const
{
	if (!GetIsEnabled())
	{
		return false;
	}

	switch (MaskingMode)
	{
	case ECompositeMaskingMode::Texture:
		return IsValid(MaskTexture);

	case ECompositeMaskingMode::Geometry:
	{
		if (!IsValid(GetTypedOuter<ACompositeActor>()))
		{
			return false;
		}
		for (const TSoftObjectPtr<AActor>& SoftActor : MaskActors)
		{
			if (SoftActor.Get())
			{
				return true;
			}
		}
		return false;
	}

	default:
		return false;
	}
}

FCompositeCorePassProxy* UCompositePassMasking::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;

	FPassInputDecl MaskInputDecl;

	if (MaskingMode == ECompositeMaskingMode::Texture)
	{
		if (!ensureMsgf(IsValid(MaskTexture), TEXT("Texture is expected to be valid for active pass.")))
		{
			return nullptr;
		}

		FResourceMetadata MaskMetadata = { .DebugName = TEXT("MaskTexture") };
		MaskMetadata.Filter = (MaskTexture->Filter == TF_Nearest) ? SF_Point : SF_Bilinear;

		ResourceId TextureId = InContext.FindOrCreateExternalTexture(MaskTexture, MaskMetadata);
		MaskInputDecl.Set<FPassExternalResourceDesc>({ TextureId });
	}
	else // Geometry
	{
		UCompositeSceneCapture2DComponent* CaptureComponent = FindOrCreateSceneCaptureComponent();
		if (!ensureMsgf(IsValid(CaptureComponent), TEXT("Scene capture component is expected to be valid for active geometry mask pass.")))
		{
			return nullptr;
		}

		FCompositeRenderTargetPool::Get().ConditionalAcquireTarget(CaptureComponent, CaptureComponent->TextureTarget, RenderTargetResolution);

		FResourceMetadata MaskMetadata = {};
		MaskMetadata.bInvertedAlpha = true;
		MaskMetadata.DebugName = TEXT("MaskingGeometryCaptureTex");

		ResourceId TextureId = InContext.FindOrCreateExternalTexture(CaptureComponent->TextureTarget, MaskMetadata);
		MaskInputDecl.Set<FPassExternalResourceDesc>({ TextureId });
	}

	FPassInputDeclArray Inputs = { InputDecl, MaskInputDecl };
	FMaskingPassProxy* Proxy = InFrameAllocator.Create<FMaskingPassProxy>(Inputs);
	constexpr uint32 AlphaChannelIndex = static_cast<uint32>(ECompositeColorChannel::Alpha);
	Proxy->MaskSourceChannel = (MaskingMode == ECompositeMaskingMode::Geometry) ? AlphaChannelIndex : static_cast<uint32>(MaskSource);
	Proxy->bInputIsPremultiplied = bInputIsPremultiplied;
	Proxy->bPremultipliedInDisplaySpace = bPremultipliedInDisplaySpace;
	Proxy->bInvert = bInvert;

	return Proxy;
}
