// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassMaterial.h"

#include "Materials/MaterialInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "SystemTextures.h"
#include "PostProcess/PostProcessMaterialInputs.h"

DECLARE_GPU_STAT_NAMED(FCompositeMaterial, TEXT("Composite.Material"));

class FCompositePassMaterialProxy : public FCompositeCorePassProxy
{
public:
	IMPLEMENT_COMPOSITE_PASS(FCompositePassMaterialProxy);

	using FCompositeCorePassProxy::FCompositeCorePassProxy;

	/* Post-process material weak pointer. */
	TWeakObjectPtr<UMaterialInterface> Material;

	/** Whether this proxy runs as a standalone pass (outside the normal post-process pipeline). */
	bool bStandalone = false;

	UE::CompositeCore::FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const UE::CompositeCore::FPassInputArray& Inputs, const UE::CompositeCore::FPassContext& PassContext) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeMaterial, "Composite.Material");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;

		FSceneTextureShaderParameters SceneTextures = PassContext.SceneTextures;
		if (bStandalone)
		{
			// Create dummy scene textures for preprocessing — the full scene hasn't rendered yet.
			// We build these manually using GSystemTextures rather than calling
			// CreateSceneTextureShaderParameters, which depends on FRDGSystemTextures that may
			// not be initialized yet (and creating them would conflict with the renderer later).
			FSceneTextureUniformParameters* DummyParams = GraphBuilder.AllocParameters<FSceneTextureUniformParameters>();
			const FRDGTextureRef BlackDummy = GSystemTextures.GetBlackDummy(GraphBuilder);
			const FRDGTextureRef DepthDummy = GSystemTextures.GetDepthDummy(GraphBuilder);
			DummyParams->PointClampSampler = TStaticSamplerState<SF_Point>::GetRHI();
			DummyParams->SceneColorTexture = BlackDummy;
			DummyParams->SceneDepthTexture = DepthDummy;
			DummyParams->ScenePartialDepthTexture = DepthDummy;
			DummyParams->GBufferATexture = BlackDummy;
			DummyParams->GBufferBTexture = BlackDummy;
			DummyParams->GBufferCTexture = BlackDummy;
			DummyParams->GBufferDTexture = BlackDummy;
			DummyParams->GBufferETexture = BlackDummy;
			DummyParams->GBufferFTexture = BlackDummy;
			DummyParams->GBufferVelocityTexture = BlackDummy;
			DummyParams->GBufferSGGXTexture = BlackDummy;
			DummyParams->ScreenSpaceAOTexture = BlackDummy;
			DummyParams->CustomDepthTexture = DepthDummy;
			DummyParams->CustomStencilTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(GSystemTextures.GetStencilDummy(GraphBuilder)));
			SceneTextures.SceneTextures = GraphBuilder.CreateUniformBuffer(DummyParams);

			// Ensure the view has a valid uniform buffer. During early preprocessing (e.g. scene captures),
			// the ViewUniformBuffer may not be initialized yet. Create a minimal one with essential fields
			// so that post-process material UV computation works correctly.
			if (!InView.ViewUniformBuffer.IsValid())
			{
				const FIntPoint Extent = Inputs[0].Texture.Texture ? Inputs[0].Texture.Texture->Desc.Extent : FIntPoint(1, 1);
				const float Width = static_cast<float>(Extent.X);
				const float Height = static_cast<float>(Extent.Y);

				FViewUniformShaderParameters MinimalViewParameters;
				MinimalViewParameters.ViewSizeAndInvSize = FVector4f(Width, Height, 1.0f / Width, 1.0f / Height);
				MinimalViewParameters.BufferSizeAndInvSize = FVector4f(Width, Height, 1.0f / Width, 1.0f / Height);
				MinimalViewParameters.PreExposure = 1.0f;
				MinimalViewParameters.OneOverPreExposure = 1.0f;

				const_cast<FSceneView&>(InView).ViewUniformBuffer =
					TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(MinimalViewParameters, UniformBuffer_SingleFrame);
			}
		}

		FPostProcessMaterialInputs PostInputs = Inputs.ToPostProcessInputs(GraphBuilder, SceneTextures);
		PostInputs.OverrideOutput = Inputs.OverrideOutput;
		PostInputs.bStandalonePass = bStandalone;

		if (!bStandalone)
		{
			// Specify the scene color output format in case a render target needs to be created and our input is render target incompatible.
			PostInputs.OutputFormat = GetSceneColorFormatChecked();
		}

		constexpr bool bEvenIfPendingKill = false;
		constexpr bool bThreadsafeTest = true;
		if (Material.IsValid(bEvenIfPendingKill, bThreadsafeTest))
		{
			FScreenPassTexture Output = AddPostProcessMaterialPass(GraphBuilder, InView, PostInputs, Material.GetEvenIfUnreachable());

			return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
		}

		return UE::CompositeCore::FPassTexture{ PostInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder), Metadata };
	}
};


UCompositePassMaterial::UCompositePassMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

UCompositePassMaterial::~UCompositePassMaterial() = default;

bool UCompositePassMaterial::GetIsActive() const
{
	return Super::GetIsActive() && IsValid(PostProcessMaterial) && PostProcessMaterial->IsPostProcessMaterial();
}

FCompositeCorePassProxy* UCompositePassMaterial::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	FCompositePassMaterialProxy* Proxy = InFrameAllocator.Create<FCompositePassMaterialProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->Material = PostProcessMaterial;
	Proxy->bStandalone = InContext.bIsPreprocessing;

	// Conservatively mark post-processing materials as potentially using scene textures,
	// unless we are collecting passes for preprocessing where scene textures are not available.
	if (!InContext.bIsPreprocessing)
	{
		InContext.bNeedsSceneTextures = true;
	}

	return Proxy;
}

