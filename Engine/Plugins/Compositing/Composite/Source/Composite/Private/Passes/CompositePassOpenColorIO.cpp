// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassOpenColorIO.h"

#include "OpenColorIORendering.h"
#include "ScreenPass.h"
#include "PostProcess/PostProcessMaterialInputs.h"

DECLARE_GPU_STAT_NAMED(FCompositeOpenColorIO, TEXT("Composite.OpenColorIO"));

class FOpenColorIOCompositePassProxy
	: public FCompositeCorePassProxy
	, public TSharedFromThis<FOpenColorIOCompositePassProxy, ESPMode::ThreadSafe>
{
public:
	IMPLEMENT_COMPOSITE_PASS(FOpenColorIOCompositePassProxy);

	FOpenColorIOCompositePassProxy(UE::CompositeCore::FPassInputDeclArray InPassDeclaredInputs, FOpenColorIORenderPassResources&& InPassResources)
		: FCompositeCorePassProxy(MoveTemp(InPassDeclaredInputs))
		, PassResources(InPassResources)
	{

	}

	UE::CompositeCore::FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const UE::CompositeCore::FPassInputArray& Inputs, const UE::CompositeCore::FPassContext& PassContext) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeOpenColorIO, "Composite.OpenColorIO");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
		const FScreenPassTexture& Input = Inputs[0].Texture;
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		// If the override output is provided, it means that this is the last pass in post processing.
		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("OcioCompositePass"));
		}

		EOpenColorIOTransformAlpha TransformAlpha = EOpenColorIOTransformAlpha::None;
		if (bInputIsPremultiplied)
		{
			TransformAlpha = Metadata.bInvertedAlpha
				? EOpenColorIOTransformAlpha::InvertUnpremultiply
				: EOpenColorIOTransformAlpha::Unpremultiply;
		}

		FOpenColorIORendering::AddPass_RenderThread(
			GraphBuilder,
			InView,
			Input,
			Output,
			PassResources,
			TransformAlpha
		);

		return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}

	FOpenColorIORenderPassResources PassResources;
	bool bInputIsPremultiplied = true;
};

UCompositePassOpenColorIO::UCompositePassOpenColorIO(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ColorConversionSettings{}
	, bInputIsPremultiplied{true}
{
}

UCompositePassOpenColorIO::~UCompositePassOpenColorIO() = default;

FCompositeCorePassProxy* UCompositePassOpenColorIO::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	FOpenColorIOCompositePassProxy* Proxy = InFrameAllocator.Create<FOpenColorIOCompositePassProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl },
		FOpenColorIORendering::GetRenderPassResources(ColorConversionSettings, GMaxRHIFeatureLevel)
	);
	Proxy->bInputIsPremultiplied = bInputIsPremultiplied;

	return Proxy;
}

