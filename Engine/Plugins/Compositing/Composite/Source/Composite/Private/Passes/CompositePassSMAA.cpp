// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassSMAA.h"
#include "Passes/CompositeCorePassFXAAProxy.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "PostProcess/SubpixelMorphologicalAA.h"

DECLARE_GPU_STAT_NAMED(FCompositeSMAA, TEXT("Composite.SMAA"));

namespace UE
{
	namespace Composite
	{
		using namespace CompositeCore;

		class FSMAAPassProxy : public FCompositeCorePassProxy
		{
		public:
			IMPLEMENT_COMPOSITE_PASS(FSMAAPassProxy);

			using FCompositeCorePassProxy::FCompositeCorePassProxy;

			FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeSMAA, "Composite.SMAA");

				check(ValidateInputs(Inputs));

				FScreenPassTexture Input = Inputs[0].Texture;
				FResourceMetadata Metadata = Inputs[0].Metadata;
				const bool bLinearSourceColors = (Metadata.Encoding == EEncoding::Linear);
				const bool bApplyDisplayTransform = bLinearSourceColors && bDisplayTransform;

				if (bApplyDisplayTransform)
				{
					// We tonemap & encode the result so that SMAA can operate on perceptual colors
					constexpr bool bIsForward = true;
					Input = CompositeCore::Private::AddDisplayTransformPass(GraphBuilder, InView, Input, {}, bIsForward);
				}

				static IConsoleVariable* SMAAQualityCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SMAA.Quality"));
				check(SMAAQualityCVar);
				int32 Quality = SMAAQualityCVar ? SMAAQualityCVar->GetInt() : static_cast<int32>(ESMAAQuality::MAX);
				Quality = QualityOverride.Get(Quality);

				FSMAAInputs SMAAInputs{};
				SMAAInputs.SceneColor = Input;
				SMAAInputs.SceneColorBeforeTonemap = Inputs[0].Texture;
				SMAAInputs.Quality = static_cast<ESMAAQuality>(FMath::Clamp(Quality, 0, static_cast<int32>(ESMAAQuality::MAX) - 1));
				if (!bApplyDisplayTransform)
				{
					SMAAInputs.OverrideOutput = Inputs.OverrideOutput.IsValid()
						? Inputs.OverrideOutput
						: CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("CompositePassSMAA"));
				}

				FScreenPassTexture Output = AddSMAAPasses(GraphBuilder, InView, SMAAInputs);

				if (bApplyDisplayTransform)
				{
					// We decode and invert the tonemapping to obtain linear colors again.
					constexpr bool bIsForward = false;
					Output = CompositeCore::Private::AddDisplayTransformPass(GraphBuilder, InView, Output, Inputs.OverrideOutput, bIsForward);
				}

				return FPassTexture{ MoveTemp(Output), Metadata };
			}

			/** Optional r.SMAA.Quality setting override.  */
			TOptional<int32> QualityOverride = {};

			/** When true, applies a display transform (tonemap + gamma) before AA and inverts it after. */
			bool bDisplayTransform = true;
		};
	}
}

UCompositePassSMAA::UCompositePassSMAA(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Quality(5)
{
}

UCompositePassSMAA::~UCompositePassSMAA() = default;

FCompositeCorePassProxy* UCompositePassSMAA::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite;

	UE_CALL_ONCE([&]{
		if (ensure(GEngine))
		{
			GEngine->LoadSMAATextures();
		}
	});

	FSMAAPassProxy* Proxy = InFrameAllocator.Create<FSMAAPassProxy>(UE::CompositeCore::FPassInputDeclArray{ InputDecl });
	Proxy->QualityOverride = Quality;
	Proxy->bDisplayTransform = bDisplayTransform;

	return Proxy;
}

