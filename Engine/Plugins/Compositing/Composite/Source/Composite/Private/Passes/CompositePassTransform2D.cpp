// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassTransform2D.h"

#include "Camera/CameraComponent.h"
#include "CompositeActor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "Layers/CompositeLayerPlate.h"
#include "MediaTexture.h"
#include "Passes/CompositeCorePassProxy.h"
#include "PixelShaderUtils.h"
#include "ScreenPass.h"
#include "SceneView.h"

DECLARE_GPU_STAT_NAMED(FCompositeTransform2D, TEXT("Composite.Transform2D"));

class FCompositePassTransform2DShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassTransform2DShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassTransform2DShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FScreenTransform, SvPositionToOutputViewportUV)
		SHADER_PARAMETER(FScreenTransform, OutputViewportUVToInputTextureUV)
		SHADER_PARAMETER(FVector2f, Pivot)
		SHADER_PARAMETER(FVector2f, ScaleUV)
		SHADER_PARAMETER(float, RotationRadians)
		SHADER_PARAMETER(FVector2f, Translation)
		SHADER_PARAMETER(float, AspectRatio)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositePassTransform2DShader, "/Plugin/Composite/Private/CompositeTransform2D.usf", "MainPS", SF_Pixel);


namespace UE::Composite::Private
{
	using namespace CompositeCore;

	FPassTexture FTransform2DPassProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeTransform2D, "Composite.Transform2D");

		check(ValidateInputs(Inputs));

		const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
		const FScreenPassTexture& Input = Inputs[0].Texture;
		FScreenPassRenderTarget Output = Inputs.OverrideOutput;

		// No override output: create an intermediate render target.
		if (!Output.IsValid())
		{
			Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("Transform2DCompositePass"));
		}

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
		FCompositePassTransform2DShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePassTransform2DShader::FParameters>();
		FRHISamplerState* InputSampler = Metadata.GetSamplerState();
		Parameters->Input = GetScreenPassTextureInput(Input, InputSampler);
		Parameters->SvPositionToOutputViewportUV = FScreenTransform::ChangeTextureBasisFromTo(
			FScreenPassTextureViewport(Output),
			FScreenTransform::ETextureBasis::TexelPosition,
			FScreenTransform::ETextureBasis::ViewportUV);
		Parameters->OutputViewportUVToInputTextureUV = FScreenTransform::ChangeTextureBasisFromTo(
			FScreenPassTextureViewport(Input),
			FScreenTransform::ETextureBasis::ViewportUV,
			FScreenTransform::ETextureBasis::TextureUV);
		Parameters->ScaleUV = ScaleUV;
		Parameters->Translation = Translation;
		Parameters->RotationRadians = RotationRadians;
		Parameters->Pivot = Pivot;
		Parameters->AspectRatio = AspectRatio;
		Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FCompositePassTransform2DShader> PixelShader(GlobalShaderMap);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			GlobalShaderMap,
			RDG_EVENT_NAME("Composite.Transform2D (%dx%d)", Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			Parameters,
			Output.ViewRect
		);

		return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
	}
}

UCompositePassTransform2D::UCompositePassTransform2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Pivot(0.5f, 0.5f)
	, ScaleMode(ECompositePassScaleMode::Automatic)
	, ContainerAspectRatio(16.0f, 9.0f)
	, ContentAspectRatio(16.0f, 9.0f)
	, bScaleSingleAxis(true)
	, UnpadPixels(FIntPoint::ZeroValue)
	, ManualScale(1.0f, 1.0f)
	, bRemoveOverscan(false)
	, RotationAngle(0.0f)
	, Translation(FVector2f::ZeroVector)
{
}

UCompositePassTransform2D::~UCompositePassTransform2D() = default;

FCompositeCorePassProxy* UCompositePassTransform2D::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	// Use cached value from GetIsActive() if available, otherwise compute fresh.
	const FVector2f ScaleUV = CachedScaleUV.IsSet() ? CachedScaleUV.GetValue() : CalculateScale();
	CachedScaleUV.Reset();

	FTransform2DPassProxy* Proxy = InFrameAllocator.Create<FTransform2DPassProxy>(FPassInputDeclArray{ InputDecl });
	Proxy->ScaleUV = ScaleUV;
	Proxy->Translation = Translation;
	Proxy->RotationRadians = -FMath::DegreesToRadians(RotationAngle);
	Proxy->Pivot = Pivot;

	// Compute aspect ratio for rotation correction.
	const FIntVector2 TextureSize = GetPlateTextureSize();
	Proxy->AspectRatio = (TextureSize.X > 0 && TextureSize.Y > 0)
		? static_cast<float>(TextureSize.X) / TextureSize.Y
		: 1.0f;

	return Proxy;
}

bool UCompositePassTransform2D::GetIsActive() const
{
	CachedScaleUV = CalculateScale();

	const bool bHasScale = !CachedScaleUV.GetValue().Equals(FVector2f::One(), UE_SMALL_NUMBER);
	const bool bHasRotation = !FMath::IsNearlyZero(RotationAngle, UE_SMALL_NUMBER);
	const bool bHasTranslation = !Translation.IsNearlyZero(UE_SMALL_NUMBER);

	return GetIsEnabled()
		&& (ScaleMode != ECompositePassScaleMode::None || bRemoveOverscan || bHasRotation || bHasTranslation)
		&& (bHasScale || bHasRotation || bHasTranslation);
}

FVector2f UCompositePassTransform2D::CalculateScale() const
{
	ACompositeActor* CompositeActor = GetTypedOuter<ACompositeActor>();
	UCameraComponent* CameraComponent = nullptr;

	if (CompositeActor)
	{
		CameraComponent = CompositeActor->GetCameraComponent();
	}

	FVector2f ScaleUV = FVector2f::One();

	auto ApplyAspectRatioScaleFn = [this, &ScaleUV](float ContainerAR, float ContentAR)
		{
			if (ContainerAR > ContentAR)
			{
				const float Factor = (ContainerAR != 0.0f) ? ContentAR / ContainerAR : 0.0f;

				ScaleUV *= FVector2f(Factor, bScaleSingleAxis ? 1.0f : Factor);
			}
			else if (ContainerAR < ContentAR)
			{
				const float Factor = (ContentAR != 0.0f) ? ContainerAR / ContentAR : 0.0f;

				ScaleUV *= FVector2f(bScaleSingleAxis ? 1.0f : Factor, Factor);
			}
		};

	// Fetch plate texture dimensions (used by Automatic and Unpad modes).
	const FIntVector2 TextureSize = (ScaleMode == ECompositePassScaleMode::Automatic || ScaleMode == ECompositePassScaleMode::Unpad)
		? GetPlateTextureSize()
		: FIntVector2::ZeroValue;

	switch (ScaleMode)
	{
	case ECompositePassScaleMode::None:
		break;
	case ECompositePassScaleMode::Automatic:
	{
		if (TextureSize.X > 0 && TextureSize.Y > 0)
		{
			const float ContainerAR = FMath::SafeDivide(TextureSize.X, static_cast<float>(TextureSize.Y));
			const float ContentAR = CameraComponent ? CameraComponent->AspectRatio : ContainerAR;

			ApplyAspectRatioScaleFn(ContainerAR, ContentAR);
		}
	}
	break;
	case ECompositePassScaleMode::AspectRatio:
	{
		const float ContainerAR = FMath::SafeDivide(ContainerAspectRatio.X, ContainerAspectRatio.Y);
		const float ContentAR = FMath::SafeDivide(ContentAspectRatio.X, ContentAspectRatio.Y);

		ApplyAspectRatioScaleFn(ContainerAR, ContentAR);
	}
	break;
	case ECompositePassScaleMode::Unpad:
	{
		if (TextureSize.X > 2 * UnpadPixels.X && TextureSize.X > 0)
		{
			ScaleUV.X *= static_cast<float>(TextureSize.X - 2 * UnpadPixels.X) / TextureSize.X;
		}
		if (TextureSize.Y > 2 * UnpadPixels.Y && TextureSize.Y > 0)
		{
			ScaleUV.Y *= static_cast<float>(TextureSize.Y - 2 * UnpadPixels.Y) / TextureSize.Y;
		}
	}
	break;
	case ECompositePassScaleMode::Manual:
		ScaleUV.X = FMath::SafeDivide(ScaleUV.X, ManualScale.X);
		ScaleUV.Y = FMath::SafeDivide(ScaleUV.Y, ManualScale.Y);
		break;

	default:
		checkNoEntry();
		break;
	};

	if (bRemoveOverscan && CameraComponent && CameraComponent->bCropOverscan)
	{
		ScaleUV *= 1.0f + CameraComponent->Overscan;
	}

	return ScaleUV;
}

FIntVector2 UCompositePassTransform2D::GetPlateTextureSize() const
{
	if (const UCompositeLayerPlate* PlateLayer = GetTypedOuter<UCompositeLayerPlate>())
	{
		if (IsValid(PlateLayer->Texture))
		{
			if (const UMediaTexture* MediaTexture = Cast<UMediaTexture>(PlateLayer->Texture))
			{
				return FIntVector2(MediaTexture->GetWidth(), MediaTexture->GetHeight());
			}
			else if (const UTexture2D* Texture2D = Cast<UTexture2D>(PlateLayer->Texture))
			{
				return FIntVector2(Texture2D->GetSizeX(), Texture2D->GetSizeY());
			}
			else if (const UTexture2DDynamic* Texture2DDynamic = Cast<UTexture2DDynamic>(PlateLayer->Texture))
			{
				return FIntVector2(Texture2DDynamic->SizeX, Texture2DDynamic->SizeY);
			}
			else if (const UTextureRenderTarget2D* RenderTarget2D = Cast<UTextureRenderTarget2D>(PlateLayer->Texture))
			{
				return FIntVector2(RenderTarget2D->SizeX, RenderTarget2D->SizeY);
			}
		}
	}

	return FIntVector2::ZeroValue;
}
