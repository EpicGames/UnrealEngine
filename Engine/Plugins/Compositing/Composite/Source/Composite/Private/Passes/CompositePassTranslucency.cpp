// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositePassTranslucency.h"

#include "Engine/Texture.h"
#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"

DECLARE_GPU_STAT_NAMED(FCompositeTranslucency, TEXT("Composite.Translucency"));

class FCompositePassTranslucencyShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositePassTranslucencyShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositePassTranslucencyShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		SHADER_PARAMETER(uint32, PremultOp)
		SHADER_PARAMETER(float, Fade)
		SHADER_PARAMETER(uint32, OverrideOutputAlpha)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositePassTranslucencyShader, "/Plugin/Composite/Private/CompositeTranslucency.usf", "MainPS", SF_Pixel);


namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			FPassTexture FTranslucencyPassProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeTranslucency, "Composite.Translucency");

				check(ValidateInputs(Inputs));

				const UE::CompositeCore::FResourceMetadata& Metadata = Inputs[0].Metadata;
				const FScreenPassTexture& InputTexture = Inputs[0].Texture;
				FScreenPassRenderTarget Output = Inputs.OverrideOutput;

				if (!Output.IsValid())
				{
					Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, InputTexture.Texture->Desc, TEXT("TranslucencyCompositePass"));
				}

				FRHISamplerState* InputSampler = Metadata.GetSamplerState();
				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
				FCompositePassTranslucencyShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositePassTranslucencyShader::FParameters>();
				Parameters->Input = GetScreenPassTextureInput(InputTexture, InputSampler);
				Parameters->SvPositionToInputTextureUV = (
					FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
					FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(InputTexture), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
				Parameters->PremultOp = static_cast<uint32>(PremultOp);
				Parameters->Fade = Fade;
				Parameters->OverrideOutputAlpha = static_cast<uint32>(OverrideOutputAlpha);
				Parameters->bInvertedAlpha = static_cast<uint32>(Metadata.bInvertedAlpha);
				Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

				TShaderMapRef<FCompositePassTranslucencyShader> PixelShader(GlobalShaderMap);
				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					GlobalShaderMap,
					RDG_EVENT_NAME("Composite.Translucency (%dx%d)", Output.ViewRect.Width(), Output.ViewRect.Height()),
					PixelShader,
					Parameters,
					Output.ViewRect
				);

				return UE::CompositeCore::FPassTexture{ MoveTemp(Output), Metadata };
			}
		}
	}
}

UCompositePassTranslucency::UCompositePassTranslucency(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PremultOp{ ECompositeAlphaPremultiplication::None }
	, Fade{ 1.0f }
	, OverrideOutputAlpha{ ECompositeAlphaOverride::None }
{
}

UCompositePassTranslucency::~UCompositePassTranslucency() = default;

FCompositeCorePassProxy* UCompositePassTranslucency::GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const
{
	using namespace UE::Composite::Private;

	FTranslucencyPassProxy* Proxy = InFrameAllocator.Create<FTranslucencyPassProxy>(FPassInputDeclArray{ InputDecl });
	Proxy->PremultOp = PremultOp;
	Proxy->Fade = Fade;
	Proxy->OverrideOutputAlpha = OverrideOutputAlpha;

	return Proxy;
}
