// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "ScreenPass.h"

#include "Passes/CompositePassBlur.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

// -- Upload/download parameter structs (shared by both tests) --

BEGIN_SHADER_PARAMETER_STRUCT(FBlurTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FBlurTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

// ============================================================================
// Test 1: uniform-field
// ============================================================================

namespace Blur  // unique name prevents symbol collision in unity builds
{

// A uniform-color texture yields output == input.
// This holds regardless of kernel radius or separable-pass order.
static FLinearColor BlurUniformCPU(FLinearColor In)
{
	return In;
}

struct FBlurScenario
{
	const TCHAR* Label;
	FLinearColor  Input;
	FIntPoint     Radius;
	bool          bAlphaOnly;
};

static const FBlurScenario GScenarios[] =
{
	// Uniform inputs: blurring any constant field returns the same constant.
	{ TEXT("UniformSmallRadius"),      { 0.5f, 0.3f, 0.7f, 0.8f }, { 4,  4  }, false },
	{ TEXT("UniformLargeRadius"),      { 0.2f, 0.6f, 0.4f, 1.0f }, { 16, 16 }, false },
	{ TEXT("AlphaOnlyUniform"),        { 0.8f, 0.4f, 0.2f, 0.6f }, { 8,  8  }, true  },
	{ TEXT("UniformAsymmetricRadius"), { 0.3f, 0.7f, 0.5f, 0.9f }, { 8,  2  }, false },
};

}  // namespace Blur

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeBlurPassTest, "Composite.UnitTests.BlurPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeBlurPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace Blur;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FCompositePassBlurProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FBlurScenario& S = GScenarios[i];
		FCompositePassBlurProxy Proxy(GetDefaultInputDeclArray());
		Proxy.Radius     = S.Radius;
		Proxy.bAlphaOnly = S.bAlphaOnly;
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FBlurScenario& S = GScenarios[i];

				const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("BlurTest.Input"));

				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("BlurTest.Output"));

				{
					FBlurTestUploadParameters* Params = GraphBuilder.AllocParameters<FBlurTestUploadParameters>();
					Params->InputTexture = InputTexture;

					const FLinearColor PixelValue = S.Input;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("BlurTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[PixelValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = PixelValue;
								}
							}
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
					In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					In.Metadata.Filter = SF_Point;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FBlurTestDownloadParameters* Params = GraphBuilder.AllocParameters<FBlurTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("BlurTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
								static_cast<const uint8*>(Data) + (GTestViewSize.Y / 2) * Stride);
							FMemory::Memcpy(ResultPtr, &RowPtr[GTestViewSize.X / 2], sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	// Blur uses bilinear multi-tap sampling across two separable passes, accumulating
	// more floating-point rounding than single-operation passes. 5e-4 is still tight
	// enough to catch shader errors while tolerating hardware summation noise.
	constexpr float Tolerance = 5e-4f;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FBlurScenario& S = GScenarios[i];
		const FLinearColor Expected = BlurUniformCPU(S.Input);
		const FLinearColor& Actual  = ResultRDG[i];

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, Expected.R, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.G, Expected.G, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.B, Expected.B, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, Expected.A, Tolerance);
	}

	return true;
}

// ============================================================================
// Test 2: step-function gradient — validates blur actually spreads
// ============================================================================
//
// Input: left half (cols 0-31) = black (0,0,0,0), right half (cols 32-63) = white (1,1,1,1).
// After blurring with radius 4, we assert three properties without recreating the kernel:
//   1. Far-field: col 0 stays 0, col 63 stays 1  (blur does not reach 31 pixels away).
//   2. Boundary: col 31 and col 32 are strictly blurred — neither 0 nor 1.
//   3. Symmetry: value(col 31) + value(col 32) == 1  (gray at the step edge).

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeBlurStepPassTest, "Composite.UnitTests.BlurPassStep",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeBlurStepPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;

	// Read back 4 columns from a single blurred step-function texture.
	// Index: 0=far-field black (col 0), 1=boundary black (col 31),
	//        2=boundary white (col 32), 3=far-field white (col 63).
	static constexpr int32 ReadCols[4] = { 0, 31, 32, 63 };
	static constexpr int32 BlurRadius  = 4;

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(4);

	FCompositePassBlurProxy BlurProxy(GetDefaultInputDeclArray());
	BlurProxy.Radius     = FIntPoint(BlurRadius, BlurRadius);
	BlurProxy.bAlphaOnly = false;

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, Proxy = MoveTemp(BlurProxy)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);
			const int32 CenterRow = GTestViewSize.Y / 2;

			const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
				GTestViewSize, PF_A32B32G32R32F,
				FClearValueBinding::Black, ETextureCreateFlags::None);
			FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("BlurStepTest.Input"));

			const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
				GTestViewSize, PF_A32B32G32R32F,
				FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
			FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("BlurStepTest.Output"));

			// Upload: left half (cols 0-31) = black, right half (cols 32-63) = white.
			{
				FBlurTestUploadParameters* Params = GraphBuilder.AllocParameters<FBlurTestUploadParameters>();
				Params->InputTexture = InputTexture;

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("BlurStepTest.Upload"), Params,
					ERDGPassFlags::Readback,
					[Params](FRHICommandListImmediate& RHICmdList)
					{
						uint32 Stride = 0;
						void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
						for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
						{
							FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
							for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
							{
								RowPtr[Col] = (Col < GTestViewSize.X / 2)
									? FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)
									: FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
							}
						}
						RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
					});
			}

			FPassInputArray Inputs;
			{
				FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
				In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
				In.Metadata.Filter = SF_Bilinear;
			}
			Inputs.OverrideOutput = FScreenPassRenderTarget{
				FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

			FPassContext PassContext;
			PassContext.OutputViewRect = PixelRect;

			Proxy.Add(GraphBuilder, View, Inputs, PassContext);

			for (int32 i = 0; i < 4; ++i)
			{
				FBlurTestDownloadParameters* Params = GraphBuilder.AllocParameters<FBlurTestDownloadParameters>();
				Params->OutputTexture = OutputTexture;

				FLinearColor* ResultPtr = &ResultRDG[i];
				const int32 Col = ReadCols[i];
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("BlurStepTest.Download[%d]", i), Params,
					ERDGPassFlags::Readback,
					[ResultPtr, Params, CenterRow, Col](FRHICommandListImmediate& RHICmdList)
					{
						uint32 Stride = 0;
						const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
						const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
							static_cast<const uint8*>(Data) + CenterRow * Stride);
						FMemory::Memcpy(ResultPtr, &RowPtr[Col], sizeof(FLinearColor));
						RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
					});
			}
		});

	// Tolerance accounts for FP16 intermediate and bilinear hardware precision.
	constexpr float Tol = 2e-3f;

	const FLinearColor& BlackFarField = ResultRDG[0]; // col  0 — 31 pixels from boundary
	const FLinearColor& BlackBoundary = ResultRDG[1]; // col 31 — first black pixel at edge
	const FLinearColor& WhiteBoundary = ResultRDG[2]; // col 32 — first white pixel at edge
	const FLinearColor& WhiteFarField = ResultRDG[3]; // col 63 — 31 pixels from boundary

	// Far-field pixels must be unaffected (blur radius 4 cannot reach 31 pixels away).
	UTEST_TRUE("BlackFarField is black", BlackFarField.R < Tol);
	UTEST_TRUE("WhiteFarField is white", WhiteFarField.R > 1.0f - Tol);

	// Boundary pixels must be blurred: strictly between 0 and 1.
	UTEST_TRUE("BlackBoundary is non-zero", BlackBoundary.R > Tol);
	UTEST_TRUE("BlackBoundary is non-one",  BlackBoundary.R < 1.0f - Tol);
	UTEST_TRUE("WhiteBoundary is non-zero", WhiteBoundary.R > Tol);
	UTEST_TRUE("WhiteBoundary is non-one",  WhiteBoundary.R < 1.0f - Tol);

	// Symmetry: col 31 and col 32 are mirror images of the step → they sum to 1.
	UTEST_EQUAL_TOLERANCE("BoundarySum", BlackBoundary.R + WhiteBoundary.R, 1.0f, Tol);

	return true;
}

// ============================================================================
// Test 3: alpha-only flag — blur must spread alpha but leave RGB unchanged
// ============================================================================
//
// Input: same step-function (left half = 0000, right half = 1111).
// With bAlphaOnly=true and radius 4, boundary pixels must have:
//   - RGB unchanged (step values — pass-through, not blurred).
//   - Alpha blurred (strictly between 0 and 1).
// This distinguishes "alpha-only blur" from "full blur" at the shader level.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeBlurAlphaOnlyPassTest, "Composite.UnitTests.BlurPassAlphaOnly",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeBlurAlphaOnlyPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;

	static constexpr int32 ReadCols[4] = { 0, 31, 32, 63 };
	static constexpr int32 BlurRadius  = 4;

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(4);

	FCompositePassBlurProxy BlurProxy(GetDefaultInputDeclArray());
	BlurProxy.Radius     = FIntPoint(BlurRadius, BlurRadius);
	BlurProxy.bAlphaOnly = true;

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, Proxy = MoveTemp(BlurProxy)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);
			const int32 CenterRow = GTestViewSize.Y / 2;

			const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
				GTestViewSize, PF_A32B32G32R32F,
				FClearValueBinding::Black, ETextureCreateFlags::None);
			FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("BlurAlphaOnlyTest.Input"));

			const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
				GTestViewSize, PF_A32B32G32R32F,
				FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
			FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("BlurAlphaOnlyTest.Output"));

			// Upload: left half (cols 0-31) = black (0,0,0,0), right half (cols 32-63) = white (1,1,1,1).
			{
				FBlurTestUploadParameters* Params = GraphBuilder.AllocParameters<FBlurTestUploadParameters>();
				Params->InputTexture = InputTexture;

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("BlurAlphaOnlyTest.Upload"), Params,
					ERDGPassFlags::Readback,
					[Params](FRHICommandListImmediate& RHICmdList)
					{
						uint32 Stride = 0;
						void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
						for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
						{
							FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
							for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
							{
								RowPtr[Col] = (Col < GTestViewSize.X / 2)
									? FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)
									: FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
							}
						}
						RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
					});
			}

			FPassInputArray Inputs;
			{
				FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
				In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
				In.Metadata.Filter = SF_Bilinear;
			}
			Inputs.OverrideOutput = FScreenPassRenderTarget{
				FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

			FPassContext PassContext;
			PassContext.OutputViewRect = PixelRect;

			Proxy.Add(GraphBuilder, View, Inputs, PassContext);

			for (int32 i = 0; i < 4; ++i)
			{
				FBlurTestDownloadParameters* Params = GraphBuilder.AllocParameters<FBlurTestDownloadParameters>();
				Params->OutputTexture = OutputTexture;

				FLinearColor* ResultPtr = &ResultRDG[i];
				const int32 Col = ReadCols[i];
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("BlurAlphaOnlyTest.Download[%d]", i), Params,
					ERDGPassFlags::Readback,
					[ResultPtr, Params, CenterRow, Col](FRHICommandListImmediate& RHICmdList)
					{
						uint32 Stride = 0;
						const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
						const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
							static_cast<const uint8*>(Data) + CenterRow * Stride);
						FMemory::Memcpy(ResultPtr, &RowPtr[Col], sizeof(FLinearColor));
						RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
					});
			}
		});

	// Tolerance accounts for FP16 intermediate and bilinear hardware precision.
	constexpr float Tol = 2e-3f;

	const FLinearColor& BlackFarField = ResultRDG[0]; // col  0 — 31 pixels from boundary
	const FLinearColor& BlackBoundary = ResultRDG[1]; // col 31 — first black pixel at edge
	const FLinearColor& WhiteBoundary = ResultRDG[2]; // col 32 — first white pixel at edge
	const FLinearColor& WhiteFarField = ResultRDG[3]; // col 63 — 31 pixels from boundary

	// Far-field: neither RGB nor alpha is affected at radius 4 / 31-pixel distance.
	UTEST_TRUE("BlackFarField.R is black", BlackFarField.R < Tol);
	UTEST_TRUE("BlackFarField.A is zero",  BlackFarField.A < Tol);
	UTEST_TRUE("WhiteFarField.R is white", WhiteFarField.R > 1.0f - Tol);
	UTEST_TRUE("WhiteFarField.A is one",   WhiteFarField.A > 1.0f - Tol);

	// Boundary RGB: pass-through — must NOT be blurred.
	UTEST_TRUE("BlackBoundary.R is not blurred", BlackBoundary.R < Tol);
	UTEST_TRUE("WhiteBoundary.R is not blurred", WhiteBoundary.R > 1.0f - Tol);

	// Boundary alpha: must BE blurred (strictly between 0 and 1).
	UTEST_TRUE("BlackBoundary.A is non-zero",  BlackBoundary.A > Tol);
	UTEST_TRUE("BlackBoundary.A is non-one",   BlackBoundary.A < 1.0f - Tol);
	UTEST_TRUE("WhiteBoundary.A is non-zero",  WhiteBoundary.A > Tol);
	UTEST_TRUE("WhiteBoundary.A is non-one",   WhiteBoundary.A < 1.0f - Tol);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
