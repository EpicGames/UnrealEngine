// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "ScreenPass.h"

#include "Passes/CompositePassColorGrading.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

// -- Upload/download parameter structs --

BEGIN_SHADER_PARAMETER_STRUCT(FColorGradingTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FColorGradingTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace ColorGrading  // unique name prevents symbol collision in unity builds
{

// -- CPU references --

// Default FColorGradingSettings is identity: output == input.
static FLinearColor ColorGradingIdentityCPU(FLinearColor In)
{
	return In;
}

// Black input (0,0,0) is fully in the shadow range (luma=0, shadow weight=1).
// Global Offset.w adds uniformly to all channels in shadow CC, so output = (Offset, Offset, Offset, A).
static FLinearColor ColorGradingShadowOffsetCPU(FLinearColor In, float Offset)
{
	return FLinearColor(In.R + Offset, In.G + Offset, In.B + Offset, In.A);
}

// -- Test scenarios --

struct FColorGradingScenario
{
	const TCHAR* Label;
	FLinearColor  Input;
	bool          bInputIsPremultiplied;
	float         GlobalOffsetW; // Global.Offset.w — 0 means no offset
};

static const FColorGradingScenario GScenarios[] =
{
	// Identity: default settings leave input unchanged regardless of range blending weights.
	{ TEXT("IdentityMidtone"),    { 0.5f, 0.5f, 0.5f, 1.0f }, false, 0.0f },
	{ TEXT("IdentityColorful"),   { 0.6f, 0.3f, 0.1f, 1.0f }, false, 0.0f },
	{ TEXT("IdentityHighlight"),  { 0.8f, 0.8f, 0.8f, 1.0f }, false, 0.0f },

	// Unpremult + identity grading: unpremult and premult cancel → output == input (alpha > 0).
	{ TEXT("UnpremultRoundtrip"), { 0.4f, 0.3f, 0.2f, 0.5f }, true,  0.0f },

	// Black input lies entirely in the shadow range (luma=0, shadow weight=1).
	// Offset.w=0.2 adds 0.2 to all channels via the shadow CC term.
	{ TEXT("ShadowOffset"),       { 0.0f, 0.0f, 0.0f, 1.0f }, false, 0.2f },
};

// CPU reference dispatcher.
static FLinearColor ColorGradingCPU(const FColorGradingScenario& S)
{
	if (S.GlobalOffsetW != 0.0f)
	{
		return ColorGradingShadowOffsetCPU(S.Input, S.GlobalOffsetW);
	}
	return ColorGradingIdentityCPU(S.Input);
}

}  // namespace ColorGrading

// -- Test --

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeColorGradingPassTest, "Composite.UnitTests.ColorGradingPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeColorGradingPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace ColorGrading;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FColorGradingCompositePassProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FColorGradingScenario& S = GScenarios[i];
		FColorGradingCompositePassProxy Proxy(GetDefaultInputDeclArray());
		Proxy.bInputIsPremultiplied = S.bInputIsPremultiplied;
		if (S.GlobalOffsetW != 0.0f)
		{
			Proxy.ColorGradingSettings.Global.Offset = FVector4(0.0, 0.0, 0.0, static_cast<double>(S.GlobalOffsetW));
		}
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FColorGradingScenario& S = GScenarios[i];

				const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("ColorGradingTest.Input"));

				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("ColorGradingTest.Output"));

				{
					FColorGradingTestUploadParameters* Params = GraphBuilder.AllocParameters<FColorGradingTestUploadParameters>();
					Params->InputTexture = InputTexture;

					const FLinearColor PixelValue = S.Input;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("ColorGradingTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[PixelValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = PixelValue;
								}
							}
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
					In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					In.Metadata.Filter = SF_Point;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FColorGradingTestDownloadParameters* Params = GraphBuilder.AllocParameters<FColorGradingTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("ColorGradingTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
								static_cast<const uint8*>(Data) + (GTestViewSize.Y / 2) * Stride);
							FMemory::Memcpy(ResultPtr, &RowPtr[GTestViewSize.X / 2], sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	constexpr float Tolerance = 1e-4f;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FColorGradingScenario& S = GScenarios[i];
		const FLinearColor Expected = ColorGradingCPU(S);
		const FLinearColor& Actual  = ResultRDG[i];

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, Expected.R, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.G, Expected.G, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.B, Expected.B, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, Expected.A, Tolerance);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
