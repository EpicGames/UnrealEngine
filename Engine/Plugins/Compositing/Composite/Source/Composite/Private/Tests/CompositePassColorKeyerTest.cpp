// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "ScreenPass.h"

#include "Passes/CompositePassColorKeyer.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

// -- Upload/download parameter structs --

BEGIN_SHADER_PARAMETER_STRUCT(FColorKeyerTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FColorKeyerTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace ColorKeyer  // unique name prevents symbol collision in unity builds
{

struct FColorKeyerScenario
{
	const TCHAR* Label;
	FLinearColor  Input;
	FLinearColor  KeyColor;
	float         RedWeight;
	float         BlueWeight;
	float         AlphaMin;
	float         AlphaMax;
	bool          bInvertAlpha;
	FLinearColor  Expected;  // premultiplied output
};

static const FColorKeyerScenario GScenarios[] =
{
	// Pixel identical to the key color → fully keyed out. Alpha=0, RGB premultiplied to zero.
	{ TEXT("GreenOnGreen"),   { 0.0f, 1.0f, 0.0f, 1.0f }, { 0.0f, 1.0f, 0.0f, 1.0f }, 0.5f, 0.5f, 0.0f, 1.0f, false, { 0.0f, 0.0f, 0.0f, 0.0f } },
	// Pixel has no key-channel content → fully preserved. Alpha=1, RGB premultiplied unchanged.
	{ TEXT("RedOnGreen"),     { 1.0f, 0.0f, 0.0f, 1.0f }, { 0.0f, 1.0f, 0.0f, 1.0f }, 0.5f, 0.5f, 0.0f, 1.0f, false, { 1.0f, 0.0f, 0.0f, 1.0f } },
	// Inverted alpha: fully-keyed pixel gets alpha=1. Fill is zero, so premultiplied RGB stays zero.
	{ TEXT("GreenInverted"),  { 0.0f, 1.0f, 0.0f, 1.0f }, { 0.0f, 1.0f, 0.0f, 1.0f }, 0.5f, 0.5f, 0.0f, 1.0f, true,  { 0.0f, 0.0f, 0.0f, 1.0f } },
};

}  // namespace ColorKeyer

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeColorKeyerPassTest, "Composite.UnitTests.ColorKeyerPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeColorKeyerPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace ColorKeyer;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FCompositePassColorKeyerProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FColorKeyerScenario& S = GScenarios[i];
		FCompositePassColorKeyerProxy Proxy(GetDefaultInputDeclArray());
		Proxy.ScreenType              = ECompositeColorKeyerScreenType::Green;
		Proxy.KeyColor                = S.KeyColor;
		Proxy.Params0                 = FVector4f(S.RedWeight, 0.5f, S.BlueWeight, 0.0f);
		Proxy.Params1                 = FVector4f(S.AlphaMin, S.AlphaMax, 0.0f, 0.0f);
		Proxy.Visualization           = ECompositeColorKeyerVisualization::Key;
		Proxy.bPreserveVignetteAfterKey = false;
		Proxy.DenoiseMethod           = ECompositeDenoiseMethod::None;
		Proxy.bDespillOnly            = false;
		Proxy.bInvertAlpha            = S.bInvertAlpha;
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FColorKeyerScenario& S = GScenarios[i];

				const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("ColorKeyerTest.Input"));

				// Output needs RenderTargetable + UAV flags.
				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black,
					ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("ColorKeyerTest.Output"));

				{
					FColorKeyerTestUploadParameters* Params = GraphBuilder.AllocParameters<FColorKeyerTestUploadParameters>();
					Params->InputTexture = InputTexture;

					const FLinearColor PixelValue = S.Input;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("ColorKeyerTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[PixelValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = PixelValue;
								}
							}
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
					In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					In.Metadata.Filter = SF_Point;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FColorKeyerTestDownloadParameters* Params = GraphBuilder.AllocParameters<FColorKeyerTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("ColorKeyerTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
								static_cast<const uint8*>(Data) + (GTestViewSize.Y / 2) * Stride);
							FMemory::Memcpy(ResultPtr, &RowPtr[GTestViewSize.X / 2], sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	constexpr float Tolerance = 1e-4f;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FColorKeyerScenario& S = GScenarios[i];
		const FLinearColor& Expected = S.Expected;
		const FLinearColor& Actual = ResultRDG[i];

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, Expected.R, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.G, Expected.G, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.B, Expected.B, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, Expected.A, Tolerance);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

// -------------------------------------------------------------------------------------------------

#if WITH_DEV_AUTOMATION_TESTS

namespace LumaKeyer  // unique name prevents symbol collision in unity builds
{

struct FLumaKeyerScenario
{
	const TCHAR* Label;
	FLinearColor  Input;
	FVector2f     LumaRange;
	bool          bInvertAlpha;
	bool          bDisplaySpaceLuminance;
	FLinearColor  Expected;  // premultiplied output
};

static const FLumaKeyerScenario GScenarios[] =
{
	// Pure black: luminance=0 → alpha=0 → fully transparent.
	{ TEXT("BlackFullRange"),         { 0.0f, 0.0f, 0.0f, 1.0f }, { 0.0f, 1.0f }, false, false, { 0.0f,    0.0f,    0.0f,    0.0f    } },
	// Pure white: luminance=1 → alpha=1 → fully opaque. Premultiplied RGB unchanged.
	{ TEXT("WhiteFullRange"),         { 1.0f, 1.0f, 1.0f, 1.0f }, { 0.0f, 1.0f }, false, false, { 1.0f,    1.0f,    1.0f,    1.0f    } },
	// Mid-grey (linear): luminance=0.5 → alpha=0.5 → premultiplied RGB halved.
	{ TEXT("GreyMidpoint"),           { 0.5f, 0.5f, 0.5f, 1.0f }, { 0.0f, 1.0f }, false, false, { 0.25f,   0.25f,   0.25f,   0.5f    } },
	// Dark grey below [0.4,0.6] range → alpha clamps to 0.
	{ TEXT("DarkBelowThreshold"),     { 0.2f, 0.2f, 0.2f, 1.0f }, { 0.4f, 0.6f }, false, false, { 0.0f,    0.0f,    0.0f,    0.0f    } },
	// Bright grey above [0.4,0.6] range → alpha clamps to 1. Premultiplied RGB unchanged.
	{ TEXT("BrightAboveThreshold"),   { 0.8f, 0.8f, 0.8f, 1.0f }, { 0.4f, 0.6f }, false, false, { 0.8f,    0.8f,    0.8f,    1.0f    } },
	// White + inverted alpha: raw alpha=1 → inverted to 0 → premultiplied output is zero.
	{ TEXT("WhiteInverted"),          { 1.0f, 1.0f, 1.0f, 1.0f }, { 0.0f, 1.0f }, true,  false, { 0.0f,    0.0f,    0.0f,    0.0f    } },
	// Mid-grey with display-space luminance: LinearToSrgb(0.5)≈0.73536 → luma≈0.73536 → alpha≈0.73536.
	// Premultiplied: RGB = 0.5 * 0.73536 ≈ 0.36768. Verifies the LinearToSrgb path in the shader.
	{ TEXT("GreyMidpointDisplaySpace"),{ 0.5f, 0.5f, 0.5f, 1.0f }, { 0.0f, 1.0f }, false, true,  { 0.36768f, 0.36768f, 0.36768f, 0.73536f } },
};

}  // namespace LumaKeyer

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeLumaKeyerPassTest, "Composite.UnitTests.LumaKeyerPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeLumaKeyerPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace LumaKeyer;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FCompositePassColorKeyerProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FLumaKeyerScenario& S = GScenarios[i];
		FCompositePassColorKeyerProxy Proxy(GetDefaultInputDeclArray());
		Proxy.KeyerMode              = ECompositeColorKeyerMode::Luma;
		// Pack LumaRange into Params1.xy — same packing GetProxy() uses for luma mode.
		Proxy.Params1                = FVector4f(S.LumaRange.X, S.LumaRange.Y, 0.0f, 0.0f);
		Proxy.Visualization          = ECompositeColorKeyerVisualization::Key;
		Proxy.bInvertAlpha           = S.bInvertAlpha;
		Proxy.bDisplaySpaceLuminance = S.bDisplaySpaceLuminance;
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FLumaKeyerScenario& S = GScenarios[i];

				const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("LumaKeyerTest.Input"));

				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black,
					ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("LumaKeyerTest.Output"));

				{
					FColorKeyerTestUploadParameters* Params = GraphBuilder.AllocParameters<FColorKeyerTestUploadParameters>();
					Params->InputTexture = InputTexture;

					const FLinearColor PixelValue = S.Input;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("LumaKeyerTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[PixelValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = PixelValue;
								}
							}
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
					In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					In.Metadata.Filter = SF_Point;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FColorKeyerTestDownloadParameters* Params = GraphBuilder.AllocParameters<FColorKeyerTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("LumaKeyerTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
								static_cast<const uint8*>(Data) + (GTestViewSize.Y / 2) * Stride);
							FMemory::Memcpy(ResultPtr, &RowPtr[GTestViewSize.X / 2], sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	constexpr float Tolerance = 1e-4f;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FLumaKeyerScenario& S = GScenarios[i];
		const FLinearColor& Expected = S.Expected;
		const FLinearColor& Actual   = ResultRDG[i];

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, Expected.R, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.G, Expected.G, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.B, Expected.B, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, Expected.A, Tolerance);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
