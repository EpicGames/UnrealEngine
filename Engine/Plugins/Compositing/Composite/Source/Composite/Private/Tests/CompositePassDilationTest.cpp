// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIGPUReadback.h"
#include "ScreenPass.h"
#include "UObject/Package.h"

#include "Passes/CompositePassDilation.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_SHADER_PARAMETER_STRUCT(FDilationTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

namespace Dilation  // unique name prevents symbol collision in unity builds
{
using namespace UE::Composite::Private;
using namespace UE::CompositeCore;

constexpr uint32 kAlphaMask = static_cast<uint32>(ECompositeDilationChannel::Alpha);
constexpr uint32 kRgbMask   = 0x1u | 0x2u | 0x4u;

// CPU-side pixel-data upload helper shared by the strip/ramp/rgb variants below.
// The Fill functor receives (Row, Col) and returns the color that pixel should have.
template <typename FillFn>
void UploadPixels(FRDGBuilder& GraphBuilder, FRDGTextureRef Tex, FillFn&& Fill, const TCHAR* PassName)
{
	TArray<FLinearColor> PixelData;
	PixelData.SetNumUninitialized(GTestViewSize.X * GTestViewSize.Y);
	for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
	{
		for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
		{
			PixelData[Row * GTestViewSize.X + Col] = Fill(Row, Col);
		}
	}

	FDilationTestUploadParameters* Params = GraphBuilder.AllocParameters<FDilationTestUploadParameters>();
	Params->InputTexture = Tex;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DilationTest.Upload(%s)", PassName), Params,
		ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
		[Params, PixelData = MoveTemp(PixelData)](FRHICommandListImmediate& RHICmdList)
		{
			const uint32 SrcPitch = GTestViewSize.X * sizeof(FLinearColor);
			RHICmdList.UpdateTexture2D(
				Params->InputTexture->GetRHI(),
				0,
				FUpdateTextureRegion2D(0, 0, 0, 0, GTestViewSize.X, GTestViewSize.Y),
				SrcPitch,
				reinterpret_cast<const uint8*>(PixelData.GetData())
			);
		});
}

// Fill a texture with vertical strips: left half = opaque red, right half = transparent black.
// bInvertedAlpha=true  (UE convention): opaque=A=0, transparent=A=1.
// bInvertedAlpha=false (standard):      opaque=A=1, transparent=A=0.
void UploadVerticalStrip(FRDGBuilder& GraphBuilder, FRDGTextureRef Tex, bool bInvertedAlpha, const TCHAR* PassName)
{
	const int32 HalfWidth        = GTestViewSize.X / 2;
	const float OpaqueAlpha      = bInvertedAlpha ? 0.0f : 1.0f;
	const float TransparentAlpha = 1.0f - OpaqueAlpha;

	UploadPixels(GraphBuilder, Tex,
		[HalfWidth, OpaqueAlpha, TransparentAlpha](int32, int32 Col) -> FLinearColor
		{
			return (Col < HalfWidth)
				? FLinearColor(1.0f, 0.0f, 0.0f, OpaqueAlpha)
				: FLinearColor(0.0f, 0.0f, 0.0f, TransparentAlpha);
		}, PassName);
}

// Standard alpha (A=1 opaque). Cols 0..29 fully opaque, cols 30-32 form a soft ramp,
// cols 33+ fully transparent. Designed to differentiate threshold-based morphology
// (only the A=0 region expands) from true min/max (the whole ramp shifts outward).
//   col 30: A = 0.75, col 31: A = 0.5, col 32: A = 0.25
void UploadSoftAlphaRamp(FRDGBuilder& GraphBuilder, FRDGTextureRef Tex, const TCHAR* PassName)
{
	UploadPixels(GraphBuilder, Tex,
		[](int32, int32 Col) -> FLinearColor
		{
			float A = 1.0f;
			if      (Col == 30) A = 0.75f;
			else if (Col == 31) A = 0.5f;
			else if (Col == 32) A = 0.25f;
			else if (Col >= 33) A = 0.0f;
			return FLinearColor(1.0f, 0.0f, 0.0f, A);
		}, PassName);
}

// Standard alpha. Left half = bright red (R=1,G=0,B=0,A=0.5); right half = black (R=0,G=0,B=0,A=0.5).
// Constant A=0.5 makes it easy to assert that alpha is untouched when the mask excludes it.
void UploadVerticalStripRGB(FRDGBuilder& GraphBuilder, FRDGTextureRef Tex, const TCHAR* PassName)
{
	const int32 HalfWidth = GTestViewSize.X / 2;
	UploadPixels(GraphBuilder, Tex,
		[HalfWidth](int32, int32 Col) -> FLinearColor
		{
			return (Col < HalfWidth)
				? FLinearColor(1.0f, 0.0f, 0.0f, 0.5f)
				: FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);
		}, PassName);
}

// Create textures, upload via the given uploader, run the proxy, and schedule a GPU readback.
// Readback must be locked AFTER GraphBuilder.Execute() + BlockUntilGPUIdle() - use the
// PostWork callback of DispatchAndWait.
using FUploader = TFunctionRef<void(FRDGBuilder&, FRDGTextureRef, const TCHAR*)>;
void RunAndSample(
	FRDGBuilder& GraphBuilder, const FSceneView& View,
	FCompositePassDilationProxy& Proxy, bool bInvertedAlpha,
	FRHIGPUTextureReadback& Readback, FUploader Upload, const TCHAR* Tag)
{
	const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

	const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
		GTestViewSize, PF_A32B32G32R32F, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource);
	const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
		GTestViewSize, PF_A32B32G32R32F, FClearValueBinding::Black,
		ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV);

	FRDGTextureRef InputTex  = GraphBuilder.CreateTexture(InputDesc,  *FString::Printf(TEXT("%s.Input"),  Tag));
	FRDGTextureRef OutputTex = GraphBuilder.CreateTexture(OutputDesc, *FString::Printf(TEXT("%s.Output"), Tag));
	Upload(GraphBuilder, InputTex, Tag);

	FPassInputArray Inputs;
	FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
	In.Texture              = FScreenPassTexture{ InputTex, PixelRect };
	In.Metadata.Filter      = SF_Point;
	In.Metadata.bInvertedAlpha = bInvertedAlpha;
	Inputs.OverrideOutput = FScreenPassRenderTarget{
		FScreenPassTexture{ OutputTex, PixelRect }, ERenderTargetLoadAction::ENoAction };

	FPassContext PassContext;
	PassContext.OutputViewRect = PixelRect;
	Proxy.Add(GraphBuilder, View, Inputs, PassContext);

	// Schedule the GPU→staging copy. The actual readback data is only valid after
	// GraphBuilder.Execute() + BlockUntilGPUIdle() - call Readback.Lock() in PostWork.
	AddEnqueueCopyPass(GraphBuilder, &Readback, OutputTex);
}

// Extract pixels at the given columns from the center row of a readback.
// Must be called after BlockUntilGPUIdle() (i.e., from DispatchAndWait's PostWork callback).
void ExtractColumns(FRHIGPUTextureReadback& Readback, const TArray<int32>& Cols, FLinearColor* Out)
{
	const int32 CenterRow = GTestViewSize.Y / 2;
	int32 RowPitchInPixels = 0;
	const FLinearColor* Pixels = static_cast<const FLinearColor*>(Readback.Lock(RowPitchInPixels));
	checkf(Pixels != nullptr, TEXT("GPU readback Lock() returned null - staging texture may not have been enqueued or a GPU error occurred"));
	for (int32 i = 0; i < Cols.Num(); ++i)
	{
		Out[i] = Pixels[CenterRow * RowPitchInPixels + Cols[i]];
	}
	Readback.Unlock();
}

// Configure a proxy with the default alpha-only thresholded path (today's behavior).
void ConfigureAlphaThresholdedProxy(FCompositePassDilationProxy& Proxy, int32 AbsSize, bool bErode)
{
	Proxy.AbsSize       = AbsSize;
	Proxy.bErode        = bErode;
	Proxy.ChannelMask   = kAlphaMask;
	Proxy.bUseThreshold = true;
}

}  // namespace Dilation

// ============================================================================
// Test 1: Size=1 - dilation expands by 1 pixel, erosion shrinks by 1 pixel
// ============================================================================
//
// Input: cols 0..31 = opaque red (A=0), cols 32..63 = transparent black (A=1).
//
// Dilation (Size=1): only the first transparent column (32) becomes opaque.
//   col 30: A=0 (opaque, far from boundary)
//   col 31: A=0 (boundary opaque - unchanged)
//   col 32: A=0 (first transparent, dilated by left neighbor)
//   col 33: A=1 (still transparent - onestep beyond dilation reach)
//
// Erosion (Size=1): only the last opaque column (31) becomes transparent.
//   col 30: A=0 (opaque, one step in from boundary)
//   col 31: A=1 (boundary opaque - eroded by right transparent neighbor)
//   col 32: A=1 (transparent - unchanged)
//   col 33: A=1 (transparent - unchanged)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassTest, "Composite.UnitTests.DilationPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeDilationPassTest::RunTest(const FString& Parameter)
{
	using namespace Dilation;

	TArray<FLinearColor> Results;
	Results.SetNumZeroed(8);

	FCompositePassDilationProxy DilateProxy(GetDefaultInputDeclArray());
	ConfigureAlphaThresholdedProxy(DilateProxy, /*AbsSize*/ 1, /*bErode*/ false);

	FCompositePassDilationProxy ErodeProxy(GetDefaultInputDeclArray());
	ConfigureAlphaThresholdedProxy(ErodeProxy,  /*AbsSize*/ 1, /*bErode*/ true);

	FRHIGPUTextureReadback DilateRB(TEXT("DilateS1")), ErodeRB(TEXT("ErodeS1"));

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&, DProxy = MoveTemp(DilateProxy), EProxy = MoveTemp(ErodeProxy)]
		(FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			auto StripInv = [](FRDGBuilder& GB, FRDGTextureRef T, const TCHAR* Tag) { UploadVerticalStrip(GB, T, /*bInvertedAlpha*/ true, Tag); };
			RunAndSample(GraphBuilder, View, DProxy, true, DilateRB, StripInv, TEXT("DilateS1"));
			RunAndSample(GraphBuilder, View, EProxy, true, ErodeRB,  StripInv, TEXT("ErodeS1"));
		},
		[&](FRHICommandListImmediate&)
		{
			ExtractColumns(DilateRB, {30, 31, 32, 33}, &Results[0]);
			ExtractColumns(ErodeRB,  {30, 31, 32, 33}, &Results[4]);
		});

	constexpr float Tol = 1e-3f;

	// Dilation Size=1
	UTEST_TRUE("Dilate S1 col30: stays opaque",       Results[0].A < Tol);
	UTEST_TRUE("Dilate S1 col31: stays opaque",       Results[1].A < Tol);
	UTEST_TRUE("Dilate S1 col32: becomes opaque",     Results[2].A < Tol);
	UTEST_TRUE("Dilate S1 col33: stays transparent",  Results[3].A > 1.0f - Tol);

	// Erosion Size=1
	UTEST_TRUE("Erode S1 col30: stays opaque",        Results[4].A < Tol);
	UTEST_TRUE("Erode S1 col31: becomes transparent", Results[5].A > 1.0f - Tol);
	UTEST_TRUE("Erode S1 col32: stays transparent",   Results[6].A > 1.0f - Tol);
	UTEST_TRUE("Erode S1 col33: stays transparent",   Results[7].A > 1.0f - Tol);

	return true;
}

// ============================================================================
// Test 2: Size=4 - dilation expands by 4 pixels, erosion shrinks by 4 pixels
// ============================================================================
//
// Same input layout. Size=4 is dispatched as 2 × DILATION_SIZE=2 passes (greedy dispatch),
// covering 4 pixels total across 2 GPU iterations (Input → PingPong[0] → Output).
//
// Dilation (Size=4): transparent columns 32–35 become opaque.
//   col 30: A=0 (unchanged)
//   col 32: A=0 (within reach of iteration 1 - 2-pixel step)
//   col 35: A=0 (within reach of iteration 2 - stresses the full 2-pass chain)
//   col 36: A=1 (still transparent - onebeyond reach)
//
// Erosion (Size=4): opaque columns 31–28 become transparent.
//   col 27: A=0 (unchanged - first pixel beyond erosion reach, survives)
//   col 28: A=1 (within reach of iteration 2 - stresses the full 2-pass chain)
//   col 31: A=1 (within reach of iteration 1 - 2-pixel step)
//   col 32: A=1 (unchanged)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassSize4Test, "Composite.UnitTests.DilationPassSize4",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeDilationPassSize4Test::RunTest(const FString& Parameter)
{
	using namespace Dilation;

	TArray<FLinearColor> Results;
	Results.SetNumZeroed(8);

	FCompositePassDilationProxy DilateProxy(GetDefaultInputDeclArray());
	ConfigureAlphaThresholdedProxy(DilateProxy, /*AbsSize*/ 4, /*bErode*/ false);

	FCompositePassDilationProxy ErodeProxy(GetDefaultInputDeclArray());
	ConfigureAlphaThresholdedProxy(ErodeProxy,  /*AbsSize*/ 4, /*bErode*/ true);

	FRHIGPUTextureReadback DilateRB(TEXT("DilateS4")), ErodeRB(TEXT("ErodeS4"));

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&, DProxy = MoveTemp(DilateProxy), EProxy = MoveTemp(ErodeProxy)]
		(FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			auto StripInv = [](FRDGBuilder& GB, FRDGTextureRef T, const TCHAR* Tag) { UploadVerticalStrip(GB, T, /*bInvertedAlpha*/ true, Tag); };
			RunAndSample(GraphBuilder, View, DProxy, true, DilateRB, StripInv, TEXT("DilateS4"));
			RunAndSample(GraphBuilder, View, EProxy, true, ErodeRB,  StripInv, TEXT("ErodeS4"));
		},
		[&](FRHICommandListImmediate&)
		{
			ExtractColumns(DilateRB, {30, 32, 35, 36}, &Results[0]);
			ExtractColumns(ErodeRB,  {27, 28, 31, 32}, &Results[4]);
		});

	constexpr float Tol = 1e-3f;

	// Dilation Size=4
	UTEST_TRUE("Dilate S4 col30: stays opaque",       Results[0].A < Tol);
	UTEST_TRUE("Dilate S4 col32: becomes opaque",     Results[1].A < Tol);
	UTEST_TRUE("Dilate S4 col35: becomes opaque",     Results[2].A < Tol);
	UTEST_TRUE("Dilate S4 col36: stays transparent",  Results[3].A > 1.0f - Tol);

	// Erosion Size=4
	UTEST_TRUE("Erode S4 col27: stays opaque",        Results[4].A < Tol);
	UTEST_TRUE("Erode S4 col28: becomes transparent", Results[5].A > 1.0f - Tol);
	UTEST_TRUE("Erode S4 col31: becomes transparent", Results[6].A > 1.0f - Tol);
	UTEST_TRUE("Erode S4 col32: stays transparent",   Results[7].A > 1.0f - Tol);

	return true;
}

// ============================================================================
// Test 3: Standard alpha (bInvertedAlpha=false) - Size=1 dilation and erosion
// ============================================================================
//
// Same boundary geometry but standard alpha convention: A=1=opaque, A=0=transparent.
// Assertions are the mirror of Test 1 - opaque pixels have A near 1, transparent near 0.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassStdAlphaTest, "Composite.UnitTests.DilationPassStdAlpha",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeDilationPassStdAlphaTest::RunTest(const FString& Parameter)
{
	using namespace Dilation;

	TArray<FLinearColor> Results;
	Results.SetNumZeroed(8);

	FCompositePassDilationProxy DilateProxy(GetDefaultInputDeclArray());
	ConfigureAlphaThresholdedProxy(DilateProxy, /*AbsSize*/ 1, /*bErode*/ false);

	FCompositePassDilationProxy ErodeProxy(GetDefaultInputDeclArray());
	ConfigureAlphaThresholdedProxy(ErodeProxy,  /*AbsSize*/ 1, /*bErode*/ true);

	FRHIGPUTextureReadback DilateRB(TEXT("DilateStd")), ErodeRB(TEXT("ErodeStd"));

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&, DProxy = MoveTemp(DilateProxy), EProxy = MoveTemp(ErodeProxy)]
		(FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			auto StripStd = [](FRDGBuilder& GB, FRDGTextureRef T, const TCHAR* Tag) { UploadVerticalStrip(GB, T, /*bInvertedAlpha*/ false, Tag); };
			RunAndSample(GraphBuilder, View, DProxy, false, DilateRB, StripStd, TEXT("DilateStd"));
			RunAndSample(GraphBuilder, View, EProxy, false, ErodeRB,  StripStd, TEXT("ErodeStd"));
		},
		[&](FRHICommandListImmediate&)
		{
			ExtractColumns(DilateRB, {30, 31, 32, 33}, &Results[0]);
			ExtractColumns(ErodeRB,  {30, 31, 32, 33}, &Results[4]);
		});

	constexpr float Tol = 1e-3f;

	// Standard alpha: opaque = A > 1-Tol, transparent = A < Tol
	// Dilation Size=1
	UTEST_TRUE("Dilate StdAlpha col30: stays opaque",       Results[0].A > 1.0f - Tol);
	UTEST_TRUE("Dilate StdAlpha col31: stays opaque",       Results[1].A > 1.0f - Tol);
	UTEST_TRUE("Dilate StdAlpha col32: becomes opaque",     Results[2].A > 1.0f - Tol);
	UTEST_TRUE("Dilate StdAlpha col33: stays transparent",  Results[3].A < Tol);

	// Erosion Size=1
	UTEST_TRUE("Erode StdAlpha col30: stays opaque",        Results[4].A > 1.0f - Tol);
	UTEST_TRUE("Erode StdAlpha col31: becomes transparent", Results[5].A < Tol);
	UTEST_TRUE("Erode StdAlpha col32: stays transparent",   Results[6].A < Tol);
	UTEST_TRUE("Erode StdAlpha col33: stays transparent",   Results[7].A < Tol);

	return true;
}

// ============================================================================
// Test 4: Min/max morphology on alpha (bUseThreshold=false) over a hard-edged matte
// ============================================================================
//
// Same binary input as Test 1. With Channels=Alpha only + threshold off, the proxy routes
// through the GENERAL min/max path (ALPHA_ONLY permutation is reserved for the thresholded
// legacy / holdout dispatch). Threshold and min/max must agree on alpha for hard-edged mattes.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassMinMaxAlphaTest, "Composite.UnitTests.DilationPassMinMaxAlpha",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeDilationPassMinMaxAlphaTest::RunTest(const FString& Parameter)
{
	using namespace Dilation;

	TArray<FLinearColor> Results;
	Results.SetNumZeroed(8);

	FCompositePassDilationProxy DilateProxy(GetDefaultInputDeclArray());
	DilateProxy.AbsSize       = 1;
	DilateProxy.bErode        = false;
	DilateProxy.ChannelMask   = kAlphaMask;
	DilateProxy.bUseThreshold = false;

	FCompositePassDilationProxy ErodeProxy(GetDefaultInputDeclArray());
	ErodeProxy.AbsSize       = 1;
	ErodeProxy.bErode        = true;
	ErodeProxy.ChannelMask   = kAlphaMask;
	ErodeProxy.bUseThreshold = false;

	FRHIGPUTextureReadback DilateRB(TEXT("MinMaxDilate")), ErodeRB(TEXT("MinMaxErode"));

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&, DProxy = MoveTemp(DilateProxy), EProxy = MoveTemp(ErodeProxy)]
		(FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			auto StripInv = [](FRDGBuilder& GB, FRDGTextureRef T, const TCHAR* Tag) { UploadVerticalStrip(GB, T, /*bInvertedAlpha*/ true, Tag); };
			RunAndSample(GraphBuilder, View, DProxy, true, DilateRB, StripInv, TEXT("MinMaxDilate"));
			RunAndSample(GraphBuilder, View, EProxy, true, ErodeRB,  StripInv, TEXT("MinMaxErode"));
		},
		[&](FRHICommandListImmediate&)
		{
			ExtractColumns(DilateRB, {30, 31, 32, 33}, &Results[0]);
			ExtractColumns(ErodeRB,  {30, 31, 32, 33}, &Results[4]);
		});

	constexpr float Tol = 1e-3f;

	// Same expectations as Test 1 - threshold and min/max agree on binary mattes.
	UTEST_TRUE("MinMaxDilate col30: stays opaque",       Results[0].A < Tol);
	UTEST_TRUE("MinMaxDilate col31: stays opaque",       Results[1].A < Tol);
	UTEST_TRUE("MinMaxDilate col32: becomes opaque",     Results[2].A < Tol);
	UTEST_TRUE("MinMaxDilate col33: stays transparent",  Results[3].A > 1.0f - Tol);

	UTEST_TRUE("MinMaxErode col30: stays opaque",        Results[4].A < Tol);
	UTEST_TRUE("MinMaxErode col31: becomes transparent", Results[5].A > 1.0f - Tol);
	UTEST_TRUE("MinMaxErode col32: stays transparent",   Results[6].A > 1.0f - Tol);
	UTEST_TRUE("MinMaxErode col33: stays transparent",   Results[7].A > 1.0f - Tol);

	return true;
}

// ============================================================================
// Test 5: Min/max morphology on a soft matte - graded output differentiates it from threshold
// ============================================================================
//
// Input ramp in standard alpha:
//   cols 0..29: A=1.0
//   col 30: A=0.75,  col 31: A=0.5,  col 32: A=0.25
//   cols 33..63: A=0.0
//
// With min/max dilate, each pixel reads the max of its kernel (±1 neighbors). The ramp
// shifts outward by one pixel:
//   col 30 ← max(A=1, 0.75, 0.5) = 1
//   col 31 ← max(0.75, 0.5, 0.25) = 0.75
//   col 32 ← max(0.5, 0.25, 0) = 0.5
//   col 33 ← max(0.25, 0, 0) = 0.25
// Threshold-based dilate would leave the whole ramp pinned (only A=0 pixels can change).

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassSoftMatteTest, "Composite.UnitTests.DilationPassSoftMatte",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeDilationPassSoftMatteTest::RunTest(const FString& Parameter)
{
	using namespace Dilation;

	TArray<FLinearColor> Results;
	Results.SetNumZeroed(5);

	FCompositePassDilationProxy Proxy(GetDefaultInputDeclArray());
	Proxy.AbsSize       = 1;
	Proxy.bErode        = false;
	Proxy.ChannelMask   = kAlphaMask;
	Proxy.bUseThreshold = false;

	FRHIGPUTextureReadback ReadBack(TEXT("SoftMatte"));

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&, P = MoveTemp(Proxy)]
		(FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			RunAndSample(GraphBuilder, View, P, /*bInvertedAlpha*/ false, ReadBack, UploadSoftAlphaRamp, TEXT("SoftMatte"));
		},
		[&](FRHICommandListImmediate&)
		{
			ExtractColumns(ReadBack, {30, 31, 32, 33, 34}, Results.GetData());
		});

	constexpr float Tol = 1e-3f;

	// Standard alpha; values shift outward by 1 pixel.
	UTEST_TRUE("SoftMatte col30: A≈1.0",  FMath::IsNearlyEqual(Results[0].A, 1.00f, Tol));
	UTEST_TRUE("SoftMatte col31: A≈0.75", FMath::IsNearlyEqual(Results[1].A, 0.75f, Tol));
	UTEST_TRUE("SoftMatte col32: A≈0.50", FMath::IsNearlyEqual(Results[2].A, 0.50f, Tol));
	UTEST_TRUE("SoftMatte col33: A≈0.25", FMath::IsNearlyEqual(Results[3].A, 0.25f, Tol));
	UTEST_TRUE("SoftMatte col34: A≈0.0",  Results[4].A < Tol);

	return true;
}

// ============================================================================
// Test 6: Per-channel RGB dilate, alpha untouched
// ============================================================================
//
// Input (standard alpha, A=0.5 everywhere):
//   cols 0..31: R=1, G=0, B=0
//   cols 32..63: R=0, G=0, B=0
//
// ChannelMask = R|G|B, bUseThreshold=false, Size=1 dilate. Col 32 reads the kernel max
// per channel - R from its left neighbor (col 31, R=1) makes it R=1. Col 33 is too far
// from the boundary (kernel ±1) and stays black. Alpha must stay 0.5 on every sampled
// pixel because the A bit is not in the mask.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassRGBTest, "Composite.UnitTests.DilationPassRGB",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeDilationPassRGBTest::RunTest(const FString& Parameter)
{
	using namespace Dilation;

	TArray<FLinearColor> Results;
	Results.SetNumZeroed(4);

	FCompositePassDilationProxy Proxy(GetDefaultInputDeclArray());
	Proxy.AbsSize       = 1;
	Proxy.bErode        = false;
	Proxy.ChannelMask   = kRgbMask;
	Proxy.bUseThreshold = false;

	FRHIGPUTextureReadback ReadBack(TEXT("RGBDilate"));

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&, P = MoveTemp(Proxy)]
		(FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			RunAndSample(GraphBuilder, View, P, /*bInvertedAlpha*/ false, ReadBack, UploadVerticalStripRGB, TEXT("RGBDilate"));
		},
		[&](FRHICommandListImmediate&)
		{
			ExtractColumns(ReadBack, {31, 32, 33, 34}, Results.GetData());
		});

	constexpr float Tol = 1e-3f;

	UTEST_TRUE("RGBDilate col31: stays red (R≈1)",       FMath::IsNearlyEqual(Results[0].R, 1.0f, Tol));
	UTEST_TRUE("RGBDilate col32: becomes red (R≈1)",     FMath::IsNearlyEqual(Results[1].R, 1.0f, Tol));
	UTEST_TRUE("RGBDilate col33: still black (R≈0)",     Results[2].R < Tol);
	UTEST_TRUE("RGBDilate col34: still black (R≈0)",     Results[3].R < Tol);

	// Alpha must be untouched on all sampled pixels (A bit not in the mask).
	UTEST_TRUE("RGBDilate col31: alpha untouched",       FMath::IsNearlyEqual(Results[0].A, 0.5f, Tol));
	UTEST_TRUE("RGBDilate col32: alpha untouched",       FMath::IsNearlyEqual(Results[1].A, 0.5f, Tol));
	UTEST_TRUE("RGBDilate col33: alpha untouched",       FMath::IsNearlyEqual(Results[2].A, 0.5f, Tol));

	return true;
}

// ============================================================================
// Test 7: Identity - Size=0 or Channels=0 makes the pass inactive
// ============================================================================
//
// Game-thread only; no GPU work.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeDilationPassIdentityTest, "Composite.UnitTests.DilationPassIdentity",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FCompositeDilationPassIdentityTest::RunTest(const FString& Parameter)
{
	UCompositePassDilation* Pass = NewObject<UCompositePassDilation>(GetTransientPackage());
	UTEST_NOT_NULL("Constructed pass", Pass);

	// Defaults (Size=1, Channels=Alpha, enabled) are active.
	UTEST_TRUE("Defaults: active", Pass->GetIsActive());

	Pass->Size = 0;
	UTEST_FALSE("Size==0: inactive", Pass->GetIsActive());

	Pass->Size = 3;
	Pass->Channels = 0;
	UTEST_FALSE("Channels==0: inactive", Pass->GetIsActive());

	Pass->Channels = static_cast<int32>(ECompositeDilationChannel::Alpha);
	UTEST_TRUE("Restored: active", Pass->GetIsActive());

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
