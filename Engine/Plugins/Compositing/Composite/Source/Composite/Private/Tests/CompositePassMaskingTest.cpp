// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/Color.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "ScreenPass.h"

#include "Passes/CompositePassMasking.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

// -- Upload/download parameter structs --

BEGIN_SHADER_PARAMETER_STRUCT(FMaskingTestMainUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMaskingTestMaskUploadParameters, )
	RDG_TEXTURE_ACCESS(MaskTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMaskingTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace Masking  // unique name prevents symbol collision in unity builds
{

// -- CPU reference --

// Default metadata: bInvertedAlpha=false (InvertAlpha = identity), Encoding=Linear (Decode/Encode = identity).
// Masking shader: AlphaMask = saturate(Mask[Channel]); optional invert; Output.A = Input.A * AlphaMask.
// When bInputIsPremultiplied is true, RGB is divided by A before masking and multiplied back by the new A after.
static FLinearColor MaskingCPU(FLinearColor Main, FLinearColor Mask, uint32 Channel, bool bInputIsPremultiplied, bool bInvert)
{
	if (bInputIsPremultiplied)
	{
		const float InvA = (Main.A > 0.0f) ? 1.0f / Main.A : 0.0f;
		Main.R *= InvA;
		Main.G *= InvA;
		Main.B *= InvA;
	}

	const float MaskChannels[4] = { Mask.R, Mask.G, Mask.B, Mask.A };
	float AlphaMask = FMath::Clamp(MaskChannels[Channel], 0.0f, 1.0f);
	if (bInvert)
	{
		AlphaMask = 1.0f - AlphaMask;
	}

	Main.A *= AlphaMask;

	if (bInputIsPremultiplied)
	{
		Main.R *= Main.A;
		Main.G *= Main.A;
		Main.B *= Main.A;
	}

	return Main;
}

// -- Test scenarios --

struct FMaskingScenario
{
	const TCHAR* Label;
	FLinearColor  MainInput;
	FLinearColor  MaskInput;
	uint32        Channel;    // 0=R, 1=G, 2=B, 3=A
	bool          bInputIsPremultiplied;
	bool          bInvert;
};

static const FMaskingScenario GScenarios[] =
{
	// Mask from red channel: AlphaMask = 1.0 → Output.A = Main.A * 1.0.
	{ TEXT("MaskFromRed"),   { 0.5f, 0.3f, 0.7f, 0.8f }, { 1.0f, 0.0f, 0.0f, 1.0f }, 0, false, false },
	// Mask from green channel: AlphaMask = 0.6 → Output.A = Main.A * 0.6 = 0.48.
	{ TEXT("MaskFromGreen"), { 0.5f, 0.3f, 0.7f, 0.8f }, { 0.0f, 0.6f, 0.0f, 1.0f }, 1, false, false },
	// Mask from alpha channel: AlphaMask = 0.4 → Output.A = Main.A * 0.4 = 0.32.
	{ TEXT("MaskFromAlpha"), { 0.5f, 0.3f, 0.7f, 0.8f }, { 0.0f, 0.0f, 0.0f, 0.4f }, 3, false, false },
	// Inverted mask: AlphaMask = 1.0 - 0.75 = 0.25 → Output.A = Main.A * 0.25 = 0.2.
	{ TEXT("InvertedMask"),  { 0.5f, 0.3f, 0.7f, 0.8f }, { 0.75f, 0.0f, 0.0f, 1.0f }, 0, false, true  },
	// Unpremult: divide by alpha, apply 0.5 mask, multiply back → RGB changes.
	{ TEXT("Unpremult"),     { 0.4f, 0.6f, 0.2f, 0.8f }, { 0.5f, 0.0f, 0.0f, 1.0f }, 0, true,  false },
};

}  // namespace Masking

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeMaskingPassTest, "Composite.UnitTests.MaskingPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeMaskingPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace Masking;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FMaskingPassProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FMaskingScenario& S = GScenarios[i];
		FPassInputDeclArray InputDecls;
		InputDecls.AddDefaulted_GetRef().Set<FPassInternalResourceDesc>(FPassInternalResourceDesc{});
		InputDecls.AddDefaulted_GetRef().Set<FPassInternalResourceDesc>(FPassInternalResourceDesc{});
		FMaskingPassProxy Proxy(MoveTemp(InputDecls));
		Proxy.MaskSourceChannel = S.Channel;
		Proxy.bInputIsPremultiplied        = S.bInputIsPremultiplied;
		Proxy.bInvert           = S.bInvert;
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FMaskingScenario& S = GScenarios[i];

				const FRDGTextureDesc MainDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(MainDesc, TEXT("MaskingTest.Input"));

				const FRDGTextureDesc MaskDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef MaskTexture = GraphBuilder.CreateTexture(MaskDesc, TEXT("MaskingTest.Mask"));

				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("MaskingTest.Output"));

				{
					FMaskingTestMainUploadParameters* Params = GraphBuilder.AllocParameters<FMaskingTestMainUploadParameters>();
					Params->InputTexture = InputTexture;

					const FLinearColor PixelValue = S.MainInput;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("MaskingTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[PixelValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = PixelValue;
								}
							}
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				{
					FMaskingTestMaskUploadParameters* Params = GraphBuilder.AllocParameters<FMaskingTestMaskUploadParameters>();
					Params->MaskTexture = MaskTexture;

					const FLinearColor MaskValue = S.MaskInput;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("MaskingTest.MaskUpload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[MaskValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->MaskTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = MaskValue;
								}
							}
							RHICmdList.UnlockTexture2D(Params->MaskTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& InMain = Inputs.GetInputs().AddDefaulted_GetRef();
					InMain.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					InMain.Metadata.Filter = SF_Point;
				}
				{
					FPassInput& InMask = Inputs.GetInputs().AddDefaulted_GetRef();
					InMask.Texture         = FScreenPassTexture{ MaskTexture, PixelRect };
					InMask.Metadata.Filter = SF_Point;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FMaskingTestDownloadParameters* Params = GraphBuilder.AllocParameters<FMaskingTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("MaskingTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
								static_cast<const uint8*>(Data) + (GTestViewSize.Y / 2) * Stride);
							FMemory::Memcpy(ResultPtr, &RowPtr[GTestViewSize.X / 2], sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	constexpr float Tolerance = 1e-4f;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FMaskingScenario& S = GScenarios[i];
		const FLinearColor Expected = MaskingCPU(S.MainInput, S.MaskInput, S.Channel, S.bInputIsPremultiplied, S.bInvert);
		const FLinearColor& Actual  = ResultRDG[i];

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, Expected.R, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.G, Expected.G, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.B, Expected.B, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, Expected.A, Tolerance);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
