// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "ScreenPass.h"

#include "Passes/CompositePassTransform2D.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

// -- Upload/download parameter structs --

BEGIN_SHADER_PARAMETER_STRUCT(FTransform2DTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FTransform2DTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace Transform2D  // unique name prevents symbol collision in unity builds
{

// -- CPU reference --

// Input: horizontal gradient (Red = col/(W-1), G=B=0, A=1).
// Computes the 2D affine transform matching the shader (output UV -> input UV), then bilinear-samples the gradient.
static float Transform2DExpectedR(FVector2f ScaleUV, FVector2f Translation, float RotationDegrees,
	FVector2f Pivot, float AspectRatio, int32 SampleCol, int32 SampleRow, int32 W, int32 H)
{
	const float UOut = (SampleCol + 0.5f) / W;
	const float VOut = (SampleRow + 0.5f) / H;

	const float RotRad = -FMath::DegreesToRadians(RotationDegrees);

	// Y-up user convention -> Y-down UV space: flip Translation.Y and Pivot.Y.
	const float UVPivotX = Pivot.X;
	const float UVPivotY = 1.0f - Pivot.Y;

	float U = UOut - Translation.X;
	float V = VOut + Translation.Y;
	U -= UVPivotX;
	V -= UVPivotY;

	V /= AspectRatio;
	const float c = FMath::Cos(RotRad);
	const float s = FMath::Sin(RotRad);
	const float RotU = U * c - V * s;
	const float RotV = U * s + V * c;
	U = RotU;
	V = RotV;
	U *= ScaleUV.X;
	V *= ScaleUV.Y;
	V *= AspectRatio;
	U += UVPivotX;
	V += UVPivotY;

	// Bilinear on a horizontal linear gradient: red = clamp(u*W - 0.5, 0, W-1) / (W-1).
	return FMath::Clamp(U * W - 0.5f, 0.0f, float(W - 1)) / (W - 1);
}

// -- Test scenarios --

struct FTransform2DScenario
{
	const TCHAR* Label;
	FVector2f    ScaleUV;
	FVector2f    Translation;
	float        RotationDegrees;
	FVector2f    Pivot;
	float        AspectRatio;
};

// Input: horizontal gradient (Red = col/(W-1), G=B=0, A=1).
// Sampled at column W/4, row H/2 - offset from center so transforms shift the sample point.
static const FTransform2DScenario GScenarios[] =
{
	// Scale-only scenarios (backwards compatible with original CenteredScale tests)
	{ TEXT("Identity"),        FVector2f(1.0f,  1.0f),  FVector2f(0.0f, 0.0f), 0.0f,  FVector2f(0.5f, 0.5f), 1.0f },
	{ TEXT("ZoomIn"),          FVector2f(0.5f,  0.5f),  FVector2f(0.0f, 0.0f), 0.0f,  FVector2f(0.5f, 0.5f), 1.0f },
	{ TEXT("ZoomOut"),         FVector2f(2.0f,  2.0f),  FVector2f(0.0f, 0.0f), 0.0f,  FVector2f(0.5f, 0.5f), 1.0f },
	{ TEXT("HorizontalOnly"), FVector2f(0.75f, 1.0f),  FVector2f(0.0f, 0.0f), 0.0f,  FVector2f(0.5f, 0.5f), 1.0f },
	// Translation
	{ TEXT("TranslateX"),      FVector2f(1.0f, 1.0f),  FVector2f(0.1f, 0.0f), 0.0f,  FVector2f(0.5f, 0.5f), 1.0f },
	// Rotation (square aspect ratio for predictable results)
	{ TEXT("Rotate90"),        FVector2f(1.0f, 1.0f),  FVector2f(0.0f, 0.0f), 90.0f, FVector2f(0.5f, 0.5f), 1.0f },
	// Custom pivot with scale
	{ TEXT("CustomPivot"),     FVector2f(0.5f, 0.5f),  FVector2f(0.0f, 0.0f), 0.0f,  FVector2f(0.25f, 0.5f), 1.0f },
	// Rotation with non-square aspect ratio
	{ TEXT("Rotate45_Wide"),   FVector2f(1.0f, 1.0f),  FVector2f(0.0f, 0.0f), 45.0f, FVector2f(0.5f, 0.5f), 16.0f / 9.0f },
	// Combined transform
	{ TEXT("Combined"),        FVector2f(0.5f, 0.5f),  FVector2f(0.1f, 0.0f), 45.0f, FVector2f(0.5f, 0.5f), 1.0f },
};

}  // namespace Transform2D

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeTransform2DPassTest, "Composite.UnitTests.Transform2DPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeTransform2DPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace Transform2D;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FTransform2DPassProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FTransform2DScenario& S = GScenarios[i];
		FTransform2DPassProxy Proxy(GetDefaultInputDeclArray());
		Proxy.ScaleUV = S.ScaleUV;
		Proxy.Translation = S.Translation;
		Proxy.RotationRadians = -FMath::DegreesToRadians(S.RotationDegrees);
		Proxy.Pivot = S.Pivot;
		Proxy.AspectRatio = S.AspectRatio;
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			const FIntRect PixelRect(0, 0, GTestViewSize.X, GTestViewSize.Y);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("Transform2DTest.Input"));

				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					GTestViewSize, PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("Transform2DTest.Output"));

				{
					FTransform2DTestUploadParameters* Params = GraphBuilder.AllocParameters<FTransform2DTestUploadParameters>();
					Params->InputTexture = InputTexture;

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("Transform2DTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							for (int32 Row = 0; Row < GTestViewSize.Y; ++Row)
							{
								FLinearColor* RowPtr = reinterpret_cast<FLinearColor*>(static_cast<uint8*>(Data) + Row * Stride);
								for (int32 Col = 0; Col < GTestViewSize.X; ++Col)
								{
									RowPtr[Col] = FLinearColor(Col / (GTestViewSize.X - 1.0f), 0.0f, 0.0f, 1.0f);
								}
							}
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
					In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					In.Metadata.Filter = SF_Bilinear;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FTransform2DTestDownloadParameters* Params = GraphBuilder.AllocParameters<FTransform2DTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("Transform2DTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							const FLinearColor* RowPtr = reinterpret_cast<const FLinearColor*>(
								static_cast<const uint8*>(Data) + (GTestViewSize.Y / 2) * Stride);
							FMemory::Memcpy(ResultPtr, &RowPtr[GTestViewSize.X / 4], sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	constexpr float Tolerance = 1e-3f;
	const int32 SampleCol = GTestViewSize.X / 4;
	const int32 SampleRow = GTestViewSize.Y / 2;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FTransform2DScenario& S  = GScenarios[i];
		const FLinearColor&         Actual = ResultRDG[i];

		const float ExpectedR = Transform2DExpectedR(
			S.ScaleUV, S.Translation, S.RotationDegrees, S.Pivot, S.AspectRatio,
			SampleCol, SampleRow, GTestViewSize.X, GTestViewSize.Y);

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, ExpectedR, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, 1.0f, Tolerance);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
