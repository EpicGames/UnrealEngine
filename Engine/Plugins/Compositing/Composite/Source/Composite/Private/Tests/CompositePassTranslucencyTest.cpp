// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "ScreenPass.h"

#include "Passes/CompositePassTranslucency.h"
#include "Tests/CompositeTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::Composite::Tests
{

#if WITH_DEV_AUTOMATION_TESTS

// -- Upload/download parameter structs --

BEGIN_SHADER_PARAMETER_STRUCT(FTranslucencyTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FTranslucencyTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace Translucency  // unique name prevents symbol collision in unity builds
{

// -- CPU reference --

static FLinearColor TranslucencyCPU(
	FLinearColor In,
	ECompositeAlphaPremultiplication PremultOp,
	float Fade,
	ECompositeAlphaOverride Override)
{
	if (PremultOp == ECompositeAlphaPremultiplication::Premultiply)
	{
		In.R *= In.A;
		In.G *= In.A;
		In.B *= In.A;
		In.R *= Fade; In.G *= Fade; In.B *= Fade; In.A *= Fade;
	}
	else if (PremultOp == ECompositeAlphaPremultiplication::Unpremultiply)
	{
		const float Rcp = In.A > 0.f ? 1.f / In.A : 0.f;
		In.R *= Rcp;
		In.G *= Rcp;
		In.B *= Rcp;
		In.A *= Fade;
	}
	else
	{
		In.R *= Fade; In.G *= Fade; In.B *= Fade; In.A *= Fade;
	}

	if (Override == ECompositeAlphaOverride::Zero)  { In.A = 0.f; }
	if (Override == ECompositeAlphaOverride::One)   { In.A = 1.f; }

	return In;
}

// -- Test scenarios --

struct FTranslucencyScenario
{
	const TCHAR*                       Label;
	FLinearColor                       Input;
	ECompositeAlphaPremultiplication   PremultOp;
	float                              Fade;
	ECompositeAlphaOverride            Override;
};

static const FTranslucencyScenario GScenarios[] =
{
	{ TEXT("None+Fade"),         { 0.8f, 0.6f, 0.4f, 1.0f }, ECompositeAlphaPremultiplication::None,          0.5f, ECompositeAlphaOverride::None },
	{ TEXT("Premultiply"),       { 0.8f, 0.6f, 0.4f, 0.5f }, ECompositeAlphaPremultiplication::Premultiply,   1.0f, ECompositeAlphaOverride::None },
	{ TEXT("Unpremultiply"),     { 0.4f, 0.3f, 0.2f, 0.5f }, ECompositeAlphaPremultiplication::Unpremultiply, 0.8f, ECompositeAlphaOverride::None },
	{ TEXT("OverrideAlphaZero"), { 0.8f, 0.6f, 0.4f, 0.7f }, ECompositeAlphaPremultiplication::None,          1.0f, ECompositeAlphaOverride::Zero },
	{ TEXT("OverrideAlphaOne"),  { 0.8f, 0.6f, 0.4f, 0.0f }, ECompositeAlphaPremultiplication::None,          1.0f, ECompositeAlphaOverride::One  },
	{ TEXT("UnpremultiplyAlpha0"),{ 0.4f, 0.3f, 0.2f, 0.0f }, ECompositeAlphaPremultiplication::Unpremultiply,1.0f, ECompositeAlphaOverride::None },
};

}  // namespace Translucency

// -- Test --

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeTranslucencyPassTest, "Composite.UnitTests.TranslucencyPass",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

bool FCompositeTranslucencyPassTest::RunTest(const FString& Parameter)
{
	using namespace UE::Composite::Private;
	using namespace UE::CompositeCore;
	using namespace Translucency;

	const int32 NumScenarios = UE_ARRAY_COUNT(GScenarios);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(NumScenarios);

	TArray<FTranslucencyPassProxy> Proxies;
	Proxies.Reserve(NumScenarios);
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FTranslucencyScenario& S = GScenarios[i];
		FTranslucencyPassProxy Proxy(GetDefaultInputDeclArray());
		Proxy.PremultOp           = S.PremultOp;
		Proxy.Fade                = S.Fade;
		Proxy.OverrideOutputAlpha = S.Override;
		Proxies.Emplace(MoveTemp(Proxy));
	}

	FCompositeTestView TestView;
	DispatchAndWait(TestView.Get(),
		[&ResultRDG, NumScenarios, Proxies = MoveTemp(Proxies)](FRDGBuilder& GraphBuilder, const FSceneView& View) mutable
		{
			// 1×1 textures are sufficient: the translucency pass operates per-pixel uniformly.
			const FIntRect PixelRect(0, 0, 1, 1);

			for (int32 i = 0; i < NumScenarios; ++i)
			{
				const FTranslucencyScenario& S = GScenarios[i];

				const FRDGTextureDesc InputDesc = FRDGTextureDesc::Create2D(
					FIntPoint(1, 1), PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::None);
				FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputDesc, TEXT("TranslucencyTest.Input"));

				const FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
					FIntPoint(1, 1), PF_A32B32G32R32F,
					FClearValueBinding::Black, ETextureCreateFlags::RenderTargetable);
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("TranslucencyTest.Output"));

				{
					FTranslucencyTestUploadParameters* Params = GraphBuilder.AllocParameters<FTranslucencyTestUploadParameters>();
					Params->InputTexture = InputTexture;

					const FLinearColor PixelValue = S.Input;
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("TranslucencyTest.Upload[%d]", i), Params,
						ERDGPassFlags::Readback,
						[PixelValue, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							void* Data = RHICmdList.LockTexture2D(Params->InputTexture->GetRHI(), 0, RLM_WriteOnly, Stride, false);
							FMemory::Memcpy(Data, &PixelValue, sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->InputTexture->GetRHI(), 0, false);
						});
				}

				FPassInputArray Inputs;
				{
					FPassInput& In = Inputs.GetInputs().AddDefaulted_GetRef();
					In.Texture         = FScreenPassTexture{ InputTexture, PixelRect };
					In.Metadata.Filter = SF_Point;
				}
				Inputs.OverrideOutput = FScreenPassRenderTarget{
					FScreenPassTexture{ OutputTexture, PixelRect }, ERenderTargetLoadAction::ENoAction };

				FPassContext PassContext;
				PassContext.OutputViewRect = PixelRect;

				Proxies[i].Add(GraphBuilder, View, Inputs, PassContext);

				{
					FTranslucencyTestDownloadParameters* Params = GraphBuilder.AllocParameters<FTranslucencyTestDownloadParameters>();
					Params->OutputTexture = OutputTexture;

					FLinearColor* ResultPtr = &ResultRDG[i];
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("TranslucencyTest.Download[%d]", i), Params,
						ERDGPassFlags::Readback,
						[ResultPtr, Params](FRHICommandListImmediate& RHICmdList)
						{
							uint32 Stride = 0;
							const void* Data = RHICmdList.LockTexture2D(Params->OutputTexture->GetRHI(), 0, RLM_ReadOnly, Stride, false);
							FMemory::Memcpy(ResultPtr, Data, sizeof(FLinearColor));
							RHICmdList.UnlockTexture2D(Params->OutputTexture->GetRHI(), 0, false);
						});
				}
			}
		});

	constexpr float Tolerance = 1e-4f;
	for (int32 i = 0; i < NumScenarios; ++i)
	{
		const FTranslucencyScenario& S = GScenarios[i];
		const FLinearColor Expected = TranslucencyCPU(S.Input, S.PremultOp, S.Fade, S.Override);
		const FLinearColor& Actual  = ResultRDG[i];

		UTEST_EQUAL_TOLERANCE(S.Label, Actual.R, Expected.R, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.G, Expected.G, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.B, Expected.B, Tolerance);
		UTEST_EQUAL_TOLERANCE(S.Label, Actual.A, Expected.A, Tolerance);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

} // namespace UE::Composite::Tests
