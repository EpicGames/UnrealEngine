// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"
#include "Components/ActorTestSpawner.h"
#include "Components/CompositeShadowReflectionCatcherComponent.h"
#include "CompositeActor.h"
#include "Engine/StaticMeshActor.h"
#include "Layers/CompositeLayerShadowReflection.h"

namespace
{
	int32 CountShadowReflectionCatchers(const ACompositeActor* InActor)
	{
		if (!InActor)
		{
			return 0;
		}

		TArray<UCompositeShadowReflectionCatcherComponent*> Catchers;
		InActor->GetComponents<UCompositeShadowReflectionCatcherComponent>(Catchers);
		return Catchers.Num();
	}
}

TEST_CLASS_WITH_FLAGS(
	FCompositeShadowReflectionLifecycleTests,
	"Composite.UnitTests.ShadowReflectionLifecycle",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	TUniquePtr<FActorTestSpawner> Spawner;
	ACompositeActor* CompositeActor = nullptr;
	AStaticMeshActor* HiddenActor = nullptr;
	UCompositeLayerShadowReflection* Layer = nullptr;

	BEFORE_EACH()
	{
		Spawner = MakeUnique<FActorTestSpawner>();

		CompositeActor = &Spawner->SpawnActor<ACompositeActor>();
		HiddenActor = &Spawner->SpawnActor<AStaticMeshActor>();

		Layer = NewObject<UCompositeLayerShadowReflection>(CompositeActor);
		CompositeActor->SetCompositeLayers({ Layer });
		Layer->SetIsEnabled(true);
	}

	AFTER_EACH()
	{
		Layer = nullptr;
		HiddenActor = nullptr;
		CompositeActor = nullptr;
		Spawner.Reset();
	}

	TEST_METHOD(CreatesExactlyTwoCatchersWhenLayerEnabled)
	{
		Layer->SetActors({ HiddenActor });

		ASSERT_THAT(AreEqual(2, CountShadowReflectionCatchers(CompositeActor)));
	}

	TEST_METHOD(DoesNotAccumulateOnDisableEnableCycle)
	{
		Layer->SetActors({ HiddenActor });
		ASSERT_THAT(AreEqual(2, CountShadowReflectionCatchers(CompositeActor)));

		Layer->SetIsEnabled(false);
		Layer->SetIsEnabled(true);
		Layer->SetActors({ HiddenActor });

		ASSERT_THAT(AreEqual(2, CountShadowReflectionCatchers(CompositeActor)));
	}

	TEST_METHOD(RepeatedSetActorsDoesNotDuplicate)
	{
		for (int32 Iteration = 0; Iteration < 5; ++Iteration)
		{
			Layer->SetActors({ HiddenActor });
		}

		ASSERT_THAT(AreEqual(2, CountShadowReflectionCatchers(CompositeActor)));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
