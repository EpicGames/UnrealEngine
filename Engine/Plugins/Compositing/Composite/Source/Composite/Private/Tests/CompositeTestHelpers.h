// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "SceneView.h"
#include "UnrealClient.h"
#include "Templates/Function.h"

namespace UE::Composite::Tests
{

// Shared viewport size used by all unit tests (wide enough to keep center pixels
// clear of blur boundaries; small enough for fast texture allocation and readback).
inline const FIntPoint GTestViewSize(64, 32);

// Owns all game-thread resources needed to create a minimal FSceneView for
// render-thread dispatch: a dummy render target, a view family, and the view itself.
// Members are declared in construction order so that DummyRT is alive when ViewFamily
// is constructed (ViewFamily stores a pointer to the render target).
struct FCompositeTestView
{
private:
	struct FDummyRenderTarget final : public FRenderTarget
	{
		virtual FIntPoint GetSizeXY() const override { return GTestViewSize; }
	};

	FDummyRenderTarget     DummyRT;
	FSceneViewFamily       ViewFamily;
	TUniquePtr<FSceneView> TestView;

public:
	FCompositeTestView()
		: ViewFamily(FSceneViewFamily::ConstructionValues(&DummyRT, nullptr, FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime()))
	{
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily         = &ViewFamily;
		ViewInitOptions.ViewOrigin         = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix   = FMatrix::Identity;
		ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint::ZeroValue, GTestViewSize));
		TestView = MakeUnique<FSceneView>(ViewInitOptions);
	}

	FSceneView* Get() const { return TestView.Get(); }
};

// Enqueues a batch of RDG passes on the render thread and blocks the game thread until
// they complete. Work receives an open FRDGBuilder and the view; GraphBuilder.Execute()
// and SubmitAndBlockUntilGPUIdle() are called internally so callers only need to add passes.
// PostWork (optional) runs after Execute() and BlockUntilGPUIdle() — use it to lock
// GPU readbacks (FRHIGPUTextureReadback::Lock) that were enqueued during Work.
inline void DispatchAndWait(
	FSceneView* ViewPtr,
	TUniqueFunction<void(FRDGBuilder&, const FSceneView&)> Work,
	TUniqueFunction<void(FRHICommandListImmediate&)> PostWork = {})
{
	FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

	ENQUEUE_RENDER_COMMAND(CompositeTestDispatch)(
		[ViewPtr, Signal, Work = MoveTemp(Work), PostWork = MoveTemp(PostWork)](FRHICommandListImmediate& RHICmdList) mutable
		{
			FRDGBuilder GraphBuilder(RHICmdList);
			Work(GraphBuilder, *ViewPtr);
			GraphBuilder.Execute();
			RHICmdList.SubmitAndBlockUntilGPUIdle();
			if (PostWork) { PostWork(RHICmdList); }
			Signal->Trigger();
		});

	Signal->Wait();
	FGenericPlatformProcess::ReturnSynchEventToPool(Signal);
}

} // namespace UE::Composite::Tests
