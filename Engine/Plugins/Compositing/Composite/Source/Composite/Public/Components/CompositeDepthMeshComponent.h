// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Components/StaticMeshComponent.h"

#include "CompositeDepthMeshComponent.generated.h"

#define UE_API COMPOSITE_API

UCLASS(MinimalAPI, ClassGroup = Composite, editinlinenew, HideCategories = (Transform, Tags, Cooking, Physics, LOD, AssetUserData, Navigation), PrioritizeCategories = ("Composite"), meta = (BlueprintSpawnableComponent))
class UCompositeDepthMeshComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	//~ Begin UActorComponent interface
	UE_API virtual void OnComponentCreated() override;
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	//~ End UActorComponent interface

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
#endif

	/* Planar grid mesh resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Composite")
	FIntVector2 GridResolution;

	/* Depth texture sent to material. */
	UPROPERTY(EditAnywhere, Category = "Composite", meta = (AllowedClasses = "/Script/Engine.Texture2D, /Script/Engine.Texture2DDynamic, /Script/Engine.TextureRenderTarget2D, /Script/MediaAssets.MediaTexture"))
	TObjectPtr<UTexture> DepthTexture;

	/* Scale factor sent to material, to convert depth units into Unreal units. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Composite")
	float ScaleFactor = 1.0f;

protected:
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

private:
	/* Convert material to MID and update its parameters. */
	void UpdateMaterial();

#if WITH_EDITOR
	/* Generate our static grid mesh based on the resolution parameters. */
	void GenerateGridMesh_Editor();
#endif
};

#undef UE_API
