// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Components/StaticMeshComponent.h"

#include "CompositeMeshComponent.generated.h"

#define UE_API COMPOSITE_API

/** Material selection type for convenience. */
UENUM(BlueprintType)
enum class ECompositeMeshMaterialType : uint8
{
	/** Default lit material. Perfect for catching shadows and/or reflections, but poor alpha edge quality due to masked alpha. */
	DefaultLitMasked,
	
	/** Default unlit alpha composite (translucent pre-multiplied alpha) material. Perfect for . */
	DefaultUnlitAlphaComposite,

	/** Custom user material */
	Custom
};

UCLASS(MinimalAPI, Blueprintable, ClassGroup = Composite, EditInlineNew)
class UCompositeMeshComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	//~ Begin UActorComponent interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	//~ End UActorComponent interface

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

	/** Get material type. */
	UFUNCTION(BlueprintGetter)
	ECompositeMeshMaterialType GetMaterialType() const { return MaterialType; }

	/** Set material type. */
	UFUNCTION(BlueprintSetter)
	void SetMaterialType(ECompositeMeshMaterialType InMaterialType);

protected:
	//~ Begin UActorComponent interface
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	//~ End UActorComponent interface

private:
	/** Material type selection for easier access to default plugin materials. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetMaterialType, BlueprintSetter = SetMaterialType, Category = "Materials", meta = (AllowPrivateAccess))
	ECompositeMeshMaterialType MaterialType;

	/** Default masked material. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DefaultLitMaskedMaterial;

	/** Default alpha composite material. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DefaultUnlitAlphaCompositeMaterial;
};

#undef UE_API

