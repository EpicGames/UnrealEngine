// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Components/SceneCaptureComponent2D.h"

#include "CompositeSceneCapture2DComponent.generated.h"

#define UE_API COMPOSITE_API

UCLASS(MinimalAPI, ClassGroup = Composite, EditInlineNew, meta = (BlueprintSpawnableComponent))
class UCompositeSceneCapture2DComponent : public USceneCaptureComponent2D
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject Interface

	//~ Begin USceneCaptureComponent Interface
	UE_API virtual const AActor* GetViewOwner() const override;
	//~ End USceneCaptureComponent Interface
};

#undef UE_API

