// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CompositeSceneCapture2DComponent.h"

#include "CompositeShadowReflectionCatcherComponent.generated.h"

#define UE_API COMPOSITE_API

UCLASS(MinimalAPI, ClassGroup = Composite, EditInlineNew, meta = (BlueprintSpawnableComponent))
class UCompositeShadowReflectionCatcherComponent : public UCompositeSceneCapture2DComponent
{
	GENERATED_UCLASS_BODY()

};

#undef UE_API

