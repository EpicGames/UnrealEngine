// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Components/SceneComponent.h"

#include "CompositeViewProjectionComponent.generated.h"

#define UE_API COMPOSITE_API

class UCameraComponent;
class UMaterialParameterCollection;
class UWorld;

/** Component responsible for continuously updating the specified material parameter collection with a camera view projection matrix (to be used for texture projection in materials). */
UCLASS(MinimalAPI, EditInlineNew, HideCategories = (Activation, Transform, Lighting, Rendering, Tags, Cooking, Physics, LOD, AssetUserData, Navigation), ClassGroup = Composite, meta = (BlueprintSpawnableComponent, DisplayName = "Camera View Projection"))
class UCompositeViewProjectionComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	//~ Begin UActorComponent interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent interface

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

public:
	/** Whether or not the component activates the view-projection matrix Material Parameter Collection update. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CameraProjection")
	bool bIsEnabled = true;

	/** The Material Parameter Collection in which the view-projection matrix should be stored. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CameraProjection")
	TObjectPtr<UMaterialParameterCollection> MaterialParameterCollection;

	/** Parameter name of the first element of the view projection matrix in the Material Parameter Collection set above.  Requires space for 4 vectors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CameraProjection")
	FName ViewProjectionMatrixParameter;

	/** Parameter name of the first element of the inverse view projection matrix in the Material Parameter Collection set above.  Requires space for 4 vectors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CameraProjection", AdvancedDisplay)
	FName ClipToWorldMatrixParameter;

	/** Parameter name of the first element of the screen to clip matrix in the Material Parameter Collection set above.  Requires space for 4 vectors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CameraProjection", AdvancedDisplay)
	FName ScreenToClipMatrixParameter;

public:
	/** Force update the Material Parameter Collection this frame. */
	UFUNCTION(BlueprintCallable, Category = "CameraProjection")
	UE_API void ForceUpdate();

	/** Camera component getter. */
	UFUNCTION(BlueprintGetter)
	UE_API UCameraComponent* GetCameraComponent() const;

	/** Camera component setter. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCameraComponent(UCameraComponent* InComponent);

	/** Camera actor getter. */
	UFUNCTION(BlueprintGetter)
	UE_API const TSoftObjectPtr<AActor>& GetCameraActor();

	/** Camera actor setter. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCameraActor(const TSoftObjectPtr<AActor>& InActor);

private:
	/** The camera whose view-projection matrix is used. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetCameraActor, BlueprintSetter = SetCameraActor, Category = "CameraProjection", meta = (AllowPrivateAccess))
	TSoftObjectPtr<AActor> CameraActor;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "TargetCameraComponent has been deprecated, use CameraActor instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "TargetCameraComponent has been deprecated, use CameraActor instead."))
	FComponentReference TargetCameraComponent_DEPRECATED;
#endif

	/** Initialize CameraActor from the outer CompositeActor's camera, or default to the owning actor. */
	void InitDefaultCamera();
	
	/** Update the Material Parameter Collection with the camera view transform information. */
	void UpdateProjection(UWorld* InWorld) const;

	/** Cached view projection matrix, to avoid needless MPC updates. */
	mutable FMatrix LastViewProjectionMatrix = FMatrix::Identity;

	/** Post-viewport update callback. */
	FDelegateHandle OnWorldPreSendAllEndOfFrameUpdatesHandle;

	friend class FCompositeViewProjectionComponentCustomization;
};

#undef UE_API

