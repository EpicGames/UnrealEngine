// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"
#include "GameplayTagContainer.h"

#include "CompositeSpawnableBinding.h"
#include "Passes/CompositePassBase.h"
#include "Layers/CompositeLayerBase.h"

#include "CompositeActor.generated.h"

#define UE_API COMPOSITE_API

class ACameraActor;
class UCameraComponent;
class UCompositeShadowReflectionCatcherComponent;
class UPrimitiveComponent;
class UCompositeSceneCapture2DComponent;

USTRUCT()
struct FSceneCaptureComponentArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UCompositeSceneCapture2DComponent>> Components;
};

/* Main render color space & encoding output mode. */
UENUM(BlueprintType)
enum class ECompositeMainRenderOutputMode : uint8
{
	Default UMETA(DisplayName = "Default", ToolTip = "Default output from post-processing's tonemap, usually sRGB display output with tone curve, equivalent to Final Color (LDR) in RGB."),
	FinalColorHDR UMETA(DisplayName = "HDR Linear", ToolTip = "Linear output from post-processing's tonemap, equivalent to Final Color (HDR) in Linear Working Color Space."),
	FinalToneCurveHDR UMETA(DisplayName = "HDR Linear with Tone Curve", ToolTip = "Linear with tone curve output from post-processing's tonemap, equivalent to Final Color (with tone curve) in Linear sRGB gamut."),
};

/**
* Constrain composite render to certain view modes for multi-viewport workflows where unlit or wireframe viewports will not conflict with the composite.
*/
UENUM(BlueprintType)
enum class ECompositeAllowedViewModes : uint8
{
	Default UMETA(ToolTip = "Compositing is allowed on viewports with Lit, Path Tracing or Unknown view modes."),
	MediaProfileUnknown UMETA(DisplayName = "Media Profile (Unknown)", ToolTip = "Compositing is only allowed with the Unknown view mode, the default when media profile does its own rendering."),
	AllViewModes UMETA(ToolTip = "Compositing is allowed with all view modes."),
};

/** Actor used to control properties of the composite pipeline. */
UCLASS(MinimalAPI)
class ACompositeActor : public AActor
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API ACompositeActor(const FObjectInitializer& ObjectInitializer);

	/** Destructor. */
	UE_API ~ACompositeActor();

	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	//~ Begin AActor Interface
	UE_API virtual void PreRegisterAllComponents() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void UnregisterAllComponents(bool bForReregister = false) override;
	UE_API virtual void PostUnregisterAllComponents() override;
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual void Destroyed() override;
	//~ End AActor Interface

#if WITH_EDITOR
	UE_API virtual void PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void SetIsTemporarilyHiddenInEditor(bool bIsHidden) override;
	UE_API virtual bool ShouldTickIfViewportsOnly() const override;
#endif // WITH_EDITOR

	/** Get the (locally) active state. */
	UFUNCTION(BlueprintGetter)
	UE_API bool GetIsActive() const;

	/** Set the (locally) active state. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetIsActive(bool bInActive);

	/** Get the enabled state. */
	UFUNCTION(BlueprintGetter)
	UE_API bool GetIsEnabled() const;

	/** Set the enabled state. */
	UFUNCTION(BlueprintSetter, CallInEditor)
	UE_API void SetIsEnabled(bool bInEnabled);

	/** Camera component getter. */
	UFUNCTION(BlueprintGetter)
	UE_API UCameraComponent* GetCameraComponent() const;

	/** Camera component setter. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCameraComponent(UCameraComponent* InComponent);

	/** Camera actor getter. */
	UFUNCTION(BlueprintGetter)
	UE_API const TSoftObjectPtr<AActor>& GetCameraActor() const;

	/** Camera actor setter. */
	UFUNCTION(BlueprintSetter, CallInEditor)
	UE_API void SetCameraActor(const TSoftObjectPtr<AActor>& InActor);

	/** Clear the camera actor and any stored spawnable binding. Use this rather than SetCameraActor(nullptr) to fully clear; SetCameraActor(nullptr) preserves the binding so Tick can re-resolve through sequencer despawns. */
	UFUNCTION(BlueprintCallable, Category = "Composite")
	UE_API void ClearCameraActor();

	/** Get the composite layers. */
	UFUNCTION(BlueprintGetter)
	UE_API TArray<UCompositeLayerBase*> GetCompositeLayers();

	/** Get the composite layers. */
	UE_API const TArray<TObjectPtr<UCompositeLayerBase>>& GetCompositeLayers() const;

	/** Set the composite layers. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCompositeLayers(TArray<UCompositeLayerBase*> InLayers);

	/** Get the vp role. */
	UFUNCTION(BlueprintGetter)
	UE_API FGameplayTag GetVPRole() const;
	
	/** Set the vp role. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetVPRole(FGameplayTag Value);

#if WITH_EDITOR
	/**
	 * Displays the plate texture preview in the level editor's 'picture-in-picture'
	 * @param InRequestingKeyer The keyer pass on the plate layer that is requesting the preview be displayed
	 * @return true if the preview was started
	 */
	UE_API bool ShowPlatePreview(TObjectPtr<class UCompositePassColorKeyer> InRequestingKeyer);
	/** Hides the plate texture preview */
	UE_API void HidePlatePreview();

	/** Gets whether the plate texture preview is currently being displayed */
	UE_API bool IsPlatePreviewActive() const;
#endif

private:
	/** Component responsible for continuously updating a material parameter collection with the composite camera view projection matrix. */
	UPROPERTY(Instanced)
	TObjectPtr<class UCompositeViewProjectionComponent> ViewProjectionComponent;

#if WITH_EDITORONLY_DATA
	/** Component used to preview a plate layer texture in the level viewport */
	UPROPERTY(Transient)
	TObjectPtr<class UCompositePlateTexturePreviewComponent> PlatePreviewComponent;
#endif

	/** Whether or not the composite behavior is active locally - used primarily for multi-user. */
	UPROPERTY(NonTransactional, EditInstanceOnly, BlueprintGetter = GetIsActive, BlueprintSetter = SetIsActive, Category = "Composite", meta = (AllowPrivateAccess = true))
	bool bIsActive;

	/** Whether or not the composite behavior is enabled. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetIsEnabled, BlueprintSetter = SetIsEnabled, Interp, Category = "Composite", meta = (DisplayName = "Enabled", AllowPrivateAccess = true, DisplayPriority = "0"))
	bool bIsEnabled;

private:
	/** The primary camera used for the composite. Used by the camera view projection component for continuous view-projection matrix updates. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetCameraActor, BlueprintSetter = SetCameraActor, Interp, Category = "Composite", meta = (AllowPrivateAccess))
	TSoftObjectPtr<AActor> CameraActor;

	/** Spawnable binding for the camera actor. Used to resolve spawnable cameras across reloads. */
	UPROPERTY()
	FCompositeSpawnableBinding CameraSpawnableBinding;

public:
	/** Returns true if the actor has a stored spawnable camera binding. */
	bool HasSpawnableBinding() const { return CameraSpawnableBinding.IsValid(); }

	/** Detect if a camera actor is a sequencer spawnable and store its binding info. */
	UE_API void DetectAndStoreSpawnableBinding(AActor* InActor);

private:
	/** Array of composite layers for merging or processing images. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetCompositeLayers, BlueprintSetter = SetCompositeLayers, Instanced, Category = "Composite", meta = (DisplayName = "Layers"))
	TArray<TObjectPtr<UCompositeLayerBase>> CompositeLayers;

public:
	/** Also provide the post-processing composite graph to SSR input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	bool bEnableScreenSpaceReflections;

	/** Override default view user flags. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	bool bOverridesViewUserFlags;

	/** Custom user flags value used to alter materials in the composite render pass. Set to one by default such that branching can be used in Lit materials. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay, meta = (EditCondition = "bOverridesViewUserFlags"))
	int32 ViewUserFlags;

	/** Main render color space & encoding output mode. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	ECompositeMainRenderOutputMode MainRenderOutput;

	/** Constrain composite rendering to specific view modes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	ECompositeAllowedViewModes AllowedViewModes;

	/**
	 * When true, compositing is restricted to a single viewport: the one piloting CameraActor if any,
	 * otherwise the active level viewport (editor only). Use for multi-viewport setups to avoid
	 * scene-capture conflicts (e.g. Shadow Reflection Catcher), since per-actor captures render once
	 * per frame from a single camera. No effect in standalone/cooked builds.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	bool bRestrictToActiveViewport;

private:
	/**
	 * Optional virtual production role to deactivate the actor when its role does not match the current device role.
	 *
	 * When unspecified, activation is unmodified (on load) or re-activated (on property change).
	 */
	UPROPERTY(EditAnywhere, BlueprintGetter = "GetVPRole", BlueprintSetter = "SetVPRole", Category = "Composite", AdvancedDisplay, meta = (DisplayName = "VP Role"))
	FGameplayTag VPRole;

public:
	/** Convenience function used by passes or layers to request a scene capture component. */
	template<class RetType>
	UE_API RetType* FindOrCreateSceneCapture(const UCompositePassBase* InPass, int32 InIndex = 0, FName InBaseName = NAME_None);

	/** Convenience function used by passes or layers to remove scene capture components. */
	UE_API void DestroySceneCaptures(const UCompositePassBase* InPass);

	/** Returns true if the composite is actively rendering. */
	UE_API bool IsRendering() const;

	/** Check if the actor's role matches the session role. */
	UE_API bool IsInVPRole() const;

private:
	/** Propagate rendering state changes to layers and the composite core subsystem. */
	void PropagateStateChange(ECompositeStateChangeType ChangeType);

	/** Pass-managed scene capture components (populated at runtime by FindOrCreateSceneCapture). */
	UPROPERTY()
	TMap<TWeakObjectPtr<const UCompositePassBase>, FSceneCaptureComponentArray> SceneCapturesPerPass;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Camera has been deprecated, use CameraActor instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Camera has been deprecated, use CameraActor instead."))
	FComponentReference Camera_DEPRECATED;

	/** Used to update components after we remove layers. */
	TArray<UCompositeLayerBase*> PreEditCompositeLayers;

	/** Used to track changes to the VP role. */
	FGameplayTag PreEditVPRole;
#endif

	/** Flag used to determine in which cases we should ignore PostRegister / Unregister calls. */
	bool bIsModifyingAProperty;

	/** Befriend the actor's editor customizations. */
	friend class SCompositePanelLayerTree;
	friend class FCompositeActorCustomization;
	friend class FCompositeActorPanelDetailCustomization;
	friend struct FColorGradingHierarchyConfig_Composite;
	friend class UCompositeActorFactory;
};

#undef UE_API

