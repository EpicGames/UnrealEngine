// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "CompositeAssetUserData.generated.h"

class UPrimitiveComponent;

DECLARE_DELEGATE_OneParam(FOnPostEditChangeOwner, UPrimitiveComponent& /* */);

/**
 * AssetUserData to keep track of edits on composite meshes.
 */
UCLASS()
class UCompositeAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	/** Bind to this to get a callback when something is changed on the associated component. */
	FOnPostEditChangeOwner OnPostEditChangeOwner;
	
	//~ Begin UAssetUserData
	virtual void PostEditChangeOwner(const FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UAssetUserData
};
