// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"

#include "CompositeDepthMeshActor.generated.h"

#define UE_API COMPOSITE_API

class UCompositeDepthMeshComponent;

/** Convenience compositing depth mesh actor. This actor must still be assigned to a plate layer. */
UCLASS(MinimalAPI)
class ACompositeDepthMeshActor : public AActor
{
	GENERATED_BODY()

public:
	UE_API ACompositeDepthMeshActor(const FObjectInitializer& ObjectInitializer);
	UE_API ~ACompositeDepthMeshActor();

private:
	/** Composite mesh component. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Composite", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UCompositeDepthMeshComponent> CompositeDepthMeshComponent;
};

#undef UE_API
