// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"

#include "CompositeMeshActor.generated.h"

#define UE_API COMPOSITE_API

class UCompositeMeshComponent;

/** Convenience compositing mesh actor. This actor must still be assigned to a plate layer. */
UCLASS(MinimalAPI)
class ACompositeMeshActor : public AActor
{
	GENERATED_BODY()

public:
	UE_API ACompositeMeshActor(const FObjectInitializer& ObjectInitializer);
	UE_API ~ACompositeMeshActor();

private:
	/** Composite mesh component. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Composite", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UCompositeMeshComponent> CompositeMeshComponent;
};

#undef UE_API
