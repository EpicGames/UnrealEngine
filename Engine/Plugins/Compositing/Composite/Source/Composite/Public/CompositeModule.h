// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

COMPOSITE_API DECLARE_LOG_CATEGORY_EXTERN(LogComposite, Log, All);

DECLARE_STATS_GROUP(TEXT("Composite"), STATGROUP_Composite, STATCAT_Advanced);

class FCompositeModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface

private:
	// Engine all module loaded callback
	void OnAllModuleLoadingPhasesComplete();
	// Engine pre-exit callback
	void OnEnginePreExit();
};

