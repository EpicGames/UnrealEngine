// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"

#include "CompositeSkySphereActor.generated.h"

#define UE_API COMPOSITE_API

class USkyLightComponent;
class UStaticMeshComponent;
class UTexture;

/**
 * Convenience actor providing image-based lighting from a plate texture.
 * Wraps an inverted sphere mesh and a real-time skylight. Assign Texture and
 * a sky sphere material with a "CompositeTexture" parameter to get image-
 * based lighting from footage.
 */
UCLASS(MinimalAPI)
class ACompositeSkySphereActor : public AActor
{
	GENERATED_BODY()

public:
	UE_API ACompositeSkySphereActor(const FObjectInitializer& ObjectInitializer);
	UE_API ~ACompositeSkySphereActor();

	//~ Begin AActor Interface
	UE_API virtual void OnConstruction(const FTransform& Transform) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End AActor Interface

	/** Sets the sky sphere texture and immediately updates the material. */
	UFUNCTION(BlueprintCallable, Category = "Texture")
	UE_API void SetTexture(UTexture* InTexture);

	/** Returns the current sky sphere texture. */
	UFUNCTION(BlueprintPure, Category = "Texture")
	UE_API UTexture* GetTexture() const { return Texture; }

	/** Name of the texture parameter in the sky sphere material to receive Texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture")
	FName TextureParameterName = TEXT("CompositeTexture");

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	friend class FCompositeSkySphereActorCustomization;

	/** Updates the sky sphere MID with the current Texture value. */
	void UpdateMaterial();

	/** If Texture is a MediaTexture from the active Media Profile, registers this actor as a consumer. */
	void TryOpenMediaProfileSource();

	/** Unregisters this actor as a consumer of any Media Profile source it previously opened. */
	void TryCloseMediaProfileSource();

	/** Texture to drive the sky sphere material's "CompositeTexture" parameter. */
	UPROPERTY(EditAnywhere, Category = "Texture",
		meta = (AllowPrivateAccess = "true", AllowedClasses = "/Script/Engine.Texture2D,/Script/Engine.Texture2DDynamic,/Script/Engine.TextureRenderTarget2D,/Script/MediaAssets.MediaTexture"))
	TObjectPtr<UTexture> Texture;

	/** Sky sphere mesh component. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Composite", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> SkySphereComponent;

	/** Real-time skylight that captures the sky sphere for IBL. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Composite", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USkyLightComponent> SkyLightComponent;
};

#undef UE_API
