// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/SoftObjectPtr.h"

#include "CompositeSpawnableBinding.generated.h"

#define UE_API COMPOSITE_API

class AActor;
class ULevelSequence;
class UWorld;

/**
 * Stores the sequencer binding information needed to re-resolve a spawnable actor
 * after its transient instance is destroyed (e.g. on level reload or sequence restart).
 */
USTRUCT()
struct FCompositeSpawnableBinding
{
	GENERATED_BODY()

	/** Binding GUID of the actor within the level sequence. */
	UPROPERTY()
	FGuid BindingId;

	/** Level sequence containing the spawnable binding. */
	UPROPERTY()
	TSoftObjectPtr<ULevelSequence> LevelSequence;

	/** Returns true if this binding has been populated with valid data. */
	UE_API bool IsValid() const;

	/** Reset the binding to an invalid state. */
	UE_API void Invalidate();
};

/**
 * Helper that manages a parallel array of spawnable bindings alongside a
 * TArray<TSoftObjectPtr<AActor>>.  Provides detect, sync, and resolve
 * operations so that each layer only needs one-liner calls at its existing
 * lifecycle points (setter, PreEditChange, PostEditChangeProperty,
 * PostEditUndo, OnEndOfFrameUpdate).
 */
USTRUCT()
struct FCompositeSpawnableBindings
{
	GENERATED_BODY()

	/**
	 * True if a spawnable binding is stored at Index. Used by details-panel customizations
	 * to detect entries that should surface a [Pending Spawnable] hint — the caller is
	 * responsible for first checking that the corresponding actor soft pointer is unresolved.
	 */
	UE_API bool HasBindingAt(int32 Index) const;

	/**
	 * Resolve stale actor soft pointers from their stored bindings.
	 * Returns true if at least one actor was resolved (callers may want to
	 * re-apply side effects such as scene-capture component updates).
	 */
	UE_API bool TryResolveStale(TArray<TSoftObjectPtr<AActor>>& Actors, UWorld* InWorld) const;

	/**
	 * Throttled wrapper around TryResolveStale for per-frame contexts (Tick / OnEndOfFrameUpdate).
	 * Fires once every 60 frames (≈1 Hz at 60 FPS; scales with frame rate at higher refresh rates,
	 * which is intentional — the throttle is a per-frame amortization, not a wall-clock cadence).
	 * The frame phase is salted by InstanceId so different instances don't all hit Sequencer/world
	 * iteration on the same frame. Pass the owning UObject's GetUniqueID().
	 */
	UE_API bool TickResolveStale(TArray<TSoftObjectPtr<AActor>>& Actors, UWorld* InWorld, uint32 InstanceId) const;

	/**
	 * Cache pre-edit state. Call before any operation that will replace the actor array
	 * (PreEditChange, Blueprint/C++ setter) so that SyncOnPropertyChange can diff against it.
	 */
	UE_API void CachePreEditState(TArrayView<const TSoftObjectPtr<AActor>> Actors);

	/**
	 * Sync bindings after the actor array has been replaced, using the state cached by
	 * CachePreEditState. Carries over existing bindings for actors whose soft path is
	 * unchanged, detects bindings for newly resolved actors, and leaves the side-channel
	 * empty for stale entries with no prior binding. Call AFTER the internal setter has
	 * applied the new array.
	 */
	UE_API void SyncOnPropertyChange(TArrayView<TSoftObjectPtr<AActor>> NewActors, UWorld* InWorld);

private:
	/** Parallel array of bindings; entry i corresponds to actor i in the owning layer's actor array. */
	UPROPERTY()
	TArray<FCompositeSpawnableBinding> Bindings;

	/**
	 * Pre-edit bindings keyed by their actor's soft path, populated by CachePreEditState and
	 * consumed by SyncOnPropertyChange to carry bindings across path-stable edits. Only entries
	 * whose binding was valid at cache time are stored; nullptr/empty paths and invalid bindings
	 * are filtered out at cache time.
	 */
	TMap<FSoftObjectPath, FCompositeSpawnableBinding> PreEditBindingsByPath;
};

namespace UE::Composite
{
	/** Detect if an actor is a sequencer spawnable. Returns a valid binding or an empty struct. */
	UE_API FCompositeSpawnableBinding DetectSpawnableBinding(AActor* InActor, UWorld* InWorld);

	/** Resolve a spawnable actor from binding data. Returns nullptr if unresolved. */
	UE_API AActor* TryResolveSpawnable(const FCompositeSpawnableBinding& Binding, UWorld* InWorld);
}

#undef UE_API
