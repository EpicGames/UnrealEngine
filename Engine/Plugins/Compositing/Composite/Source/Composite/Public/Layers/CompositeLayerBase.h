// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"

#include "Passes/CompositePassBase.h"
#include "Passes/CompositeCorePassMergeProxy.h"

#include "CompositeLayerBase.generated.h"

#define UE_API COMPOSITE_API

class ACompositeActor;
class UWorld;

/** Describes the intent of a rendering state change propagated to layers. */
enum class ECompositeStateChangeType : uint8
{
	/** Show meshes, register primitives (activate rendering). */
	Activate,
	/** Hide meshes, unregister primitives (deactivate rendering). */
	Deactivate,
	/** Restore mesh visibility, unregister primitives (removal/destruction cleanup). */
	Release
};

/**
 * Base class for a composite layer: a named unit that merges an input into the composite stack and runs sub-passes on it.
 * Subclasses spawn a per-frame render proxy via GetProxy() and own a list of UCompositePassBase to apply after merging.
 */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UCompositeLayerBase : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerBase(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerBase();

	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	/**
	 * End-of-frame update callback to inherit.
	 * Called when enabled by RegisterEndOfFrameUpdate.
	*/
	UE_API virtual void OnEndOfFrameUpdate(UWorld* InWorld) {}

	/** Called when composite actor rendering state changes.
	* 
	* @param ChangeType - Activation, deactivation or release state.
	*/
	UE_API virtual void OnRenderingStateChange(ECompositeStateChangeType ChangeType) {}

	/**
	* Called when the layer is removed or composite actor is destroyed.
	*
	* @param LastOwner - Last composite actor owner, from which the layer was removed. Ignore if null.
	*/
	UE_API virtual void OnRemoved(ACompositeActor* LastOwner) {}

	/** Override to return a render-thread proxy for this layer. Proxy objects should be allocated from the provided allocator. Only called when GetIsActive() returns true. */
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

protected:
	/** Convenience function to update scene capture primitive components, depending on the scene capture primitive render mode. */
	UE_API static void UpdateSceneCaptureComponents(USceneCaptureComponent2D& SceneCaptureComponent, TArrayView<TSoftObjectPtr<AActor>> InActors);

	/** Activate the end-of-frame update callback. */
	UE_API void RegisterEndOfFrameUpdate(bool bInEnabled);

public:
	/** Whether or not the pass is solo. */
	UPROPERTY()
	bool bIsSolo = false;

#if WITH_EDITORONLY_DATA
	/** Deprecated layer display name, use UCompositePassBase::DisplayName. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use DisplayName instead."))
	FString Name_DEPRECATED;
#endif

	/** Merge operation applied on Input0 with Input1. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (DisplayPriority = "3"))
	ECompositeCoreMergeOp Operation = ECompositeCoreMergeOp::Over;

	/** Sub-passes applied to this layer's input before it is merged into the composite stack. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category = "Composite",
		meta = (DisallowedClasses = "/Script/Composite.CompositeLayerBase, /Script/Composite.CompositePassDistortion"))
	TArray<TObjectPtr<UCompositePassBase>> LayerPasses;

	/** Convenience function to register post-processing child passes. */
	void AddChildPasses(UE::CompositeCore::FPassInputDecl& InBasePassInput, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator, TArrayView<const TObjectPtr<UCompositePassBase>> InPasses) const;

	/** Fixed number of inputs used by layer merge passes. */
	static constexpr int32 FixedNumLayerInputs = 2;

	/** Convenience function that returns the merge operation depending on the traversal context. */
	ECompositeCoreMergeOp GetMergeOperation(const FCompositeTraversalContext& InContext) const;

	/** Convenience function that returns the default secondary input depending on the traversal context. */
	UE::CompositeCore::FPassInputDecl GetDefaultSecondInput(const FCompositeTraversalContext& InContext) const;

private:
	/** Private end of frame callback implementation to condition the inherited function. */
	void ConditionalOnEndOfFrameUpdate(UWorld* InWorld);

	/** Post-viewport update callback. */
	FDelegateHandle OnWorldPreSendAllEndOfFrameUpdatesHandle;
};

#undef UE_API

