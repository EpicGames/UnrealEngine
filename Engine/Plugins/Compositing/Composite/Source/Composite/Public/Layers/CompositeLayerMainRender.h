// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerBase.h"

#include "CompositeLayerMainRender.generated.h"

#define UE_API COMPOSITE_API

/**
 * Layer that fetches scene color from the main render during post-processing after tonemap.
 * Preferable (for performance and rendering systems) to use the main render instead of additional scene captures.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Main Render"))
class UCompositeLayerMainRender : public UCompositeLayerBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerMainRender(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerMainRender();

	/** Getter function to override, returning pass layer proxies to be passed to the render thread. (Proxy objects should be allocated from the provided allocator.) */
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	/**
	 * When enabled, blending with the layer beneath is performed in sRGB display color space rather than linear.
	 * Required to match the 2D-plate Ultimatte path when composite meshes use a translucent grayscale holdout.
	 *
	 * Additional Ultimatte requirements:
	 * - "Encoding Override" should be set to "Linear" on the (alpha) key media source to avoid color decoding.
	 * - On the Plate layer feeding the composite meshes, use an Ultimatte Masking pass to mask the fill texture with the alpha key texture.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay, meta = (DisplayName = "Ultimatte Blend"))
	bool bUltimatteBlend = false;
};

#undef UE_API
