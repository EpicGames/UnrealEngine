// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerBase.h"
#include "CompositeSpawnableBinding.h"

#include "CompositeLayerPlanarReflection.generated.h"

#define UE_API COMPOSITE_API

class AActor;
class ACompositeActor;
class USceneCaptureComponent2D;

/**
 * Layer that renders a planar reflection of selected actors via a mirrored scene capture.
 * The result is blended as a reflection contribution onto the layer below.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Planar Reflection"))
class UCompositeLayerPlanarReflection : public UCompositeLayerBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerPlanarReflection(const FObjectInitializer& ObjectInitializer);

	/** Destructor. */
	UE_API virtual ~UCompositeLayerPlanarReflection();

	UE_API virtual void OnRemoved(ACompositeActor* LastOwner) override;

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject Interface

	/** End-of-frame update callback. */
	UE_API virtual void OnEndOfFrameUpdate(UWorld* InWorld) override;

	UE_API virtual bool GetIsActive() const override;
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	/** Get reflection & shadow caster actors. */
	UFUNCTION(BlueprintGetter)
	UE_API const TArray<TSoftObjectPtr<AActor>> GetActors() const;

	/** Set reflection & shadow caster actors. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetActors(TArray<TSoftObjectPtr<AActor>> InActors);

	/** Get whether the planar reflection is rendered as a custom render pass. */
	UFUNCTION(BlueprintGetter)
	UE_API bool IsCustomRenderPass() const;

	/** Set whether the planar reflection is rendered as a custom render pass. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCustomRenderPass(bool bInIsFastRenderPass);

	/** Get the render target resolution for the planar reflection. */
	UFUNCTION(BlueprintGetter)
	UE_API FIntPoint GetRenderTargetResolution() const { return RenderTargetResolution; }

	/** Set the render target resolution for the planar reflection. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetRenderTargetResolution(FIntPoint InRenderTargetResolution);

	/** Get the planar reflection strength. */
	UFUNCTION(BlueprintGetter)
	UE_API float GetReflectionStrength() const { return ReflectionStrength; }

	/** Set the planar reflection strength. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetReflectionStrength(float InReflectionStrength);

private:

	/** Convenience function to access the actor-managed planar reflection component. */
	class UCompositeSceneCapture2DComponent* GetSceneCaptureComponent() const;

	/** Update the scene capture transform for planar reflection. */
	void UpdatePlanarReflectionTransform(USceneCaptureComponent2D* CaptureComponent);

	/** Apply actor-change side effects without touching spawnable bindings. */
	void SetActorsInternal(TArray<TSoftObjectPtr<AActor>> InActors);

	/** List of actors to be rendered from the planar reflection. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetActors, BlueprintSetter = SetActors, Category = "Composite", meta = (AllowPrivateAccess = true))
	TArray<TSoftObjectPtr<AActor>> Actors;

	/** Parallel spawnable binding data for the Actors array. */
	UPROPERTY()
	FCompositeSpawnableBindings SpawnableBindings;

	/** Render target resolution for the planar reflection. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetRenderTargetResolution, BlueprintSetter = SetRenderTargetResolution, Interp, Category = "Composite", meta = (AllowPrivateAccess = true))
	FIntPoint RenderTargetResolution;

	/** Strength of the planar reflection. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetReflectionStrength, BlueprintSetter = SetReflectionStrength, Interp, Category = "Composite", meta = (AllowPrivateAccess = true, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float ReflectionStrength = 0.5f;

	/** World-space Z coordinate of the horizontal plane for planar reflection. Note that we currently do not clip against this plane. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (AllowPrivateAccess = true))
	float PlaneHeight = 0.0f;

	/** Convenience toggle to render the planar reflection as a fast/minimal Custom Render Pass, inlined in the main render & without support for lighting. */
	UPROPERTY(EditAnywhere, BlueprintGetter = IsCustomRenderPass, BlueprintSetter = SetCustomRenderPass, Category = "Composite", AdvancedDisplay, meta = (AllowPrivateAccess = true))
	bool bCustomRenderPass;

	friend class FCompositeLayerPlanarReflectionCustomization;
};

#undef UE_API

