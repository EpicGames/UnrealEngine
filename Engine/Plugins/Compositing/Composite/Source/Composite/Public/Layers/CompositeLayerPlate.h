// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerBase.h"
#include "CompositeSpawnableBinding.h"
#include "Tickable.h"

#include "CompositeLayerPlate.generated.h"

#define UE_API COMPOSITE_API

class UMaterialInterface;
class UPrimitiveComponent;
class UTexture;
class FObjectPreSaveContext;
class UTextureRenderTarget2D;
class UCompositePassTransform2D;

/** Plate sampling & rendering mode. */
UENUM(BlueprintType)
enum class ECompositePlateMode : uint8
{
	Texture = 0 UMETA(ToolTip = "Sample the texture directly in 2D screen space."),
	CompositeMesh = 1 UMETA(ToolTip = "Sample the composite meshes rendered in a built-in custom render pass (default). Automatic fallback to Texture."),
};

USTRUCT()
struct FCompositeMeshPrimitiveReference
{
	GENERATED_BODY()
	
	/** The actor whose primitives will be used for the composite meshes */
	UPROPERTY()
	TSoftObjectPtr<AActor> CompositeMeshActor;
	
	/** The primitives to use for the composite meshes */
	UPROPERTY()
	TArray<TSoftObjectPtr<UPrimitiveComponent>> CompositeMeshPrimitives;
};

/**
 * Layer that renders a media texture into the composite stack, with optional scene projection via composite meshes.
 *
 * With composite meshes and default plugin materials, the media texture is projected
 * into the scene. These meshes are rendered separately in a custom render pass without
 * post-processing. By default, they are also marked as holdout such that the plate layer
 * can be composited under the main render.
 *
 * Without composite meshes, the plate layer can be used solely as a 2D layer. In this case
 * however, interactions between CG and media in the scene are lost.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Plate"))
class UCompositeLayerPlate : public UCompositeLayerBase, public FTickableGameObject
{
	GENERATED_BODY()

public:
	/** Composite (media) texture material parameter name. */
	UE_API static const FLazyName CompositeTextureName;

	/** Constructor. */
	UE_API UCompositeLayerPlate(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerPlate();

	UE_API virtual void OnRemoved(ACompositeActor* LastOwner) override;

	UE_API virtual void OnRenderingStateChange(ECompositeStateChangeType ChangeType) override;

	/** Set the enabled state. */
	UE_API virtual void SetIsEnabled(bool bInEnabled) override;

	//~ Begin FTickableGameObject interface
	UE_API virtual void Tick(float DeltaTime) override;
	
	UE_API virtual bool IsTickableWhenPaused() const override { return true; }
	UE_API virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual ETickableTickType GetTickableTickType() const override;
	UE_API virtual bool IsTickable() const override;

	UE_API virtual UWorld* GetTickableGameObjectWorld() const override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject interface

	/** Getter function to override, returning pass layer proxies to be passed to the render thread. (Proxy objects should be allocated from the provided allocator.) */
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	/** Get array of all composite mesh primitives. */
	UE_API TArray<UPrimitiveComponent*> GetPrimitives() const;

	/** Get list of composite mesh primitives for a specified composite mesh actor */
	UE_API TArray<UPrimitiveComponent*> GetPrimitivesForActor(TSoftObjectPtr<AActor> InActor) const;

	/** Gets the list of primitives that are specifically included for the composite mesh actor. Differs from GetPrimitivesForActor in that
	 * this returns an empty list if no primitives are specifically included, whereas GetPrimitivesForActor falls back to all primitives */
	UE_API TArray<UPrimitiveComponent*> GetIncludedPrimitivesForActor(TSoftObjectPtr<AActor> InActor) const;
	
	/** Sets whether a composite mesh actor's primitive should be included as a composite mesh primitive or not */
	UE_API void SetPrimitiveIncluded(UPrimitiveComponent* InPrimitive, bool bInIncluded);
	
public:
	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
#endif
	//~ End UObject Interface

public:

	/** Get composite meshes. */
	UFUNCTION(BlueprintGetter)
	UE_API const TArray<TSoftObjectPtr<AActor>> GetCompositeMeshes() const;

	/** Set composite meshes. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCompositeMeshes(TArray<TSoftObjectPtr<AActor>> InCompositeMeshes);

	/** Get the plate sampling & rendering mode */
	UFUNCTION(BlueprintGetter)
	UE_API ECompositePlateMode GetPlateMode() const { return PlateMode; }

	/** Set the plate sampling & rendering mode */
	UFUNCTION(BlueprintSetter)
	UE_API void SetPlateMode(ECompositePlateMode InPlateMode);

	/** Get whether primitive alpha holdout is active for composite meshes in the main render. */
	UFUNCTION(BlueprintGetter)
	UE_API bool GetIsHoldoutEnabled() const { return bIsHoldoutEnabled; }

	/** Set whether primitive alpha holdout is active for composite meshes in the main render. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetIsHoldoutEnabled(bool bInIsHoldoutEnabled);

	/** Get whether plate layers automatically control the visibility of registered composite meshes. */
	UFUNCTION(BlueprintGetter)
	UE_API bool GetControlMeshVisibility() const { return bControlMeshVisibility; }

	/**
	 * Set whether plate layers automatically control the visibility of registered composite meshes: turned invisible
	 * if the layer or actor is disabled or hidden. Disabling restores visibility on all registered composite meshes.
	 */
	UFUNCTION(BlueprintSetter)
	UE_API void SetControlMeshVisibility(bool bInControlMeshVisibility);

	/** Returns the media texture, optionally pre-processed with layer & scene-only passes into a render target. */
	UFUNCTION(BlueprintCallable, Category = "Composite")
	UTexture* GetCompositeTexture() const;

private:
	/** Actors whose primitive mesh components will be marked as primitive alpha holdout, and rendered separately with the projected (media) texture. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetCompositeMeshes, BlueprintSetter = SetCompositeMeshes, Category = "Composite", meta = (AllowPrivateAccess = true))
	TArray<TSoftObjectPtr<AActor>> CompositeMeshes;

	/** A list of actual primitives from the composite mesh actors that will be marked as primitive alpha holdout, and rendered separately with the projected (media) texture  */
	UPROPERTY()
	TArray<FCompositeMeshPrimitiveReference> CompositeMeshPrimitives;

	/** Parallel spawnable binding data for the CompositeMeshes array. */
	UPROPERTY()
	FCompositeSpawnableBindings SpawnableBindings;

public:
	/** Media texture for compositing layer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta = (AllowedClasses="/Script/Engine.Texture2D, /Script/Engine.Texture2DDynamic, /Script/Engine.TextureRenderTarget2D, /Script/MediaAssets.MediaTexture"))
	TObjectPtr<UTexture> Texture;

	/**
	* Media texture pre-processing passes, applied before rendering.
	* 
	* The texture is replaced with an internal render target.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category = "Composite", meta = (DisallowedClasses = "/Script/Composite.CompositeLayerBase, /Script/Composite.CompositePassDistortion"))
	TArray<TObjectPtr<UCompositePassBase>> MediaPasses;

	/**
	* Passes applied after media passes, onto composite meshes only.
	* 
	* The texture is replaced with an internal render target.
	* 
	* NOTE: Set the plate "Mode" to "Texture" in order to skip these scene passes during final compositing in post-processing.
	* We now have an automatic scene pass to undistort a plate projected onto composite meshes. Using Texture mode can therefore
	* avoid the automatic undistort -> distort pipeline.
	*/
	UPROPERTY(BlueprintReadWrite, Instanced, Category = "Composite", meta = (AllowedClasses = "/Script/Composite.CompositePassDistortion", DisallowedClasses = "/Script/Composite.CompositeLayerBase"))
	TArray<TObjectPtr<UCompositePassBase>> ScenePasses;

private:
	/** The plate sampling & rendering mode: direct texture sampling or separate composite mesh render. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetPlateMode, BlueprintSetter = SetPlateMode, Interp, Category = "Composite", AdvancedDisplay, meta = (AllowPrivateAccess = true, DisplayName = "Mode"))
	ECompositePlateMode PlateMode;

	/**
	* Composite meshes are marked as primitive alpha holdout for the main render.
	* 
	* This is default-enabled to bypass post-processing on media pixels, primarily to avoid temporal upscaling
	* artifacts and the tone curve. However when working in 3D, holdout can be disabled to only preserve the
	* plate projection functionality and media passes.
	*/
	UPROPERTY(EditAnywhere, BlueprintGetter = GetIsHoldoutEnabled, BlueprintSetter = SetIsHoldoutEnabled, Interp, Category = "Composite", AdvancedDisplay, meta = (AllowPrivateAccess = true))
	bool bIsHoldoutEnabled;

	/** When enabled, the plate layers automatically control the visibility of registered composite meshes. Disabling restores visibility on all registered composite meshes. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetControlMeshVisibility, BlueprintSetter = SetControlMeshVisibility, Category = "Composite", AdvancedDisplay, meta = (AllowPrivateAccess = true))
	bool bControlMeshVisibility;

public:
	/**
	 * When enabled, blending with the background is performed in sRGB display color space rather than linear.
	 * Use for Ultimatte Key & Fill workflows where the content is in display space.
	 *
	 * Additional Ultimatte requirements:
	 * - "Encoding Override" should be set to "Linear" on the (alpha) key media source to avoid color decoding.
	 * - Use an Ultimatte Masking pass to mask the fill texture with the alpha key texture.
	 *
	 * Note: With composite meshes acting as holdouts, Ultimatte Blend needs to be set on the main
	 * render layer instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay, meta = (DisplayName = "Ultimatte Blend"))
	bool bUltimatteBlend = false;

	/**
	 * Automatically remove overscan from the media texture by reading the camera's overscan value.
	 * This inserts an implicit media pass that scales the footage to counteract the overscan expansion.
	 * Has no effect when the camera overscan is zero.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bRemoveOverscan = true;

private:
	/** Append specifically-included primitives for InActor to OutPrimitiveComponents. Empty if none are specifically included. */
	void AppendIncludedPrimitivesForActor(const TSoftObjectPtr<AActor>& InActor, TArray<UPrimitiveComponent*>& OutPrimitiveComponents) const;

	/** Append effective primitives for InActor to OutPrimitiveComponents (included, or all valid as fallback). */
	void AppendPrimitivesForActor(const TSoftObjectPtr<AActor>& InActor, TArray<UPrimitiveComponent*>& OutPrimitiveComponents) const;

	/** Apply composite-mesh-change side effects without touching spawnable bindings. */
	void SetCompositeMeshesInternal(TArray<TSoftObjectPtr<AActor>> InCompositeMeshes);

	/** Condtionally register composite mesh primitives: active holdout and add to the scene view extension custom render pass. */
	void TryRegisterCompositePrimitives(const TArray<UPrimitiveComponent*>& InPrimitives, const UWorld* SourceWorld = nullptr) const;

	/** Unregister composite mesh primitives: deactive holdout and remove from the scene view extension custom render pass. */
	void UnregisterCompositePrimitives(const TArray<UPrimitiveComponent*>& InPrimitives, const UWorld* SourceWorld = nullptr) const;

	/** Update the (media) texture on all composite meshes. We also add a user asset data to track changes. */
	void UpdateCompositeMeshes(bool bRegisterOnly = false) const;
	
	/** Update the (media) texture on a primitive component. */
	void UpdatePrimitiveComponent(UPrimitiveComponent& InPrimitiveComponent) const;

	/** Propagate rendering state change to composite meshes. */
	void PropagateStateChange(ECompositeStateChangeType ChangeType, const UWorld* SourceWorld) const;

	/** Convenience function verify if the composite mesh actor is already in use by another layer. */
	bool IsCompositeMeshActorAlreadyInUse(TSoftObjectPtr<AActor> InCompositeMeshActor) const;

	/** Get set of primitives already in use by other actors. */
	TSet<UPrimitiveComponent*> GetPrimitivesUsedByOtherActors() const;

	/** Convenience function to get or create the specified render target at composite actor resolution. */
	TObjectPtr<UTextureRenderTarget2D> GetOrCreateRenderTarget(TObjectPtr<UTextureRenderTarget2D>& InRenderTarget) const;

	/** Get number of valid primitives. */
	int32 GetValidPrimitivesNum() const;

	/** Get number of valid passes. */
	int32 GetValidPassesNum(TArrayView<const TObjectPtr<UCompositePassBase>> InPasses) const;

	/** If Texture is a media profile media texture, attempt to open the associated media source */
	void TryOpenMediaProfileSource();
	
	/** Attempt to close any media profile sources being played for this plate */
	void TryCloseMediaProfileSource();
	
	/** Returns true if the plate layer is actively rendering. */
	bool IsRendering() const;
	
private:
	/** Convenience function to find the last valid pass. */
	int32 FindLastValidPassIndex(TArrayView<const TObjectPtr<UCompositePassBase>> InPasses) const;

	/** Convenience function to register pre-processing child passes. */
	UE::CompositeCore::ResourceId AddPreprocessingPasses(
		FCompositeTraversalContext& InContext,
		FSceneRenderingBulkObjectAllocator& InFrameAllocator,
		TArrayView<const TObjectPtr<UCompositePassBase>> InPasses,
		UE::CompositeCore::ResourceId TextureId,
		UE::CompositeCore::ResourceId OriginalTextureId,
		TFunction<TObjectPtr<UTextureRenderTarget2D>()> GetRenderTargetFn
	) const;

#if WITH_EDITOR
	/** Pre-edit list of registered composite meshes. */
	TArray<TSoftObjectPtr<AActor>> PreEditCompositeMeshes;

	/** Transient copy of CompositeMeshPrimitives that keeps non-valid soft pointers during a save as they will be removed from CompositeMeshPrimitives during saving */
	TArray<FCompositeMeshPrimitiveReference> PreSaveCompositeMeshPrimitives;
	
	/**
	 * Called before a level saves.
	 */
	void OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext);

	/**
	 * Called after a level has saved.
	 */
	void OnPostSaveWorld(UWorld* InWorld, FObjectPostSaveContext ObjectSaveContext);
#endif

	/**
	 * Override the required translucency console variables.
	 * These overrides are needed to avoid double depth-of-field on keyed footaged + composite meshes.
	*/
	void OverrideTranslucencyConsoleVariables(bool bOverride) const;

	/** Returns true if any composite mesh primitive in this layer currently uses at least one AfterDOF translucent material. */
	bool HasAfterDOFTranslucentMaterial() const;

	/** Pushes or releases the translucency CVar overrides based on whether AfterDOF translucent materials are currently present in this layer. */
	void UpdateTranslucencyOverride() const;

	/** Convenience function to check if the material is AfterDOF translucent. */
	bool IsUsingAfterDOFTranslucency(UMaterialInterface* MaterialInterface) const;

	/** Processed media texture render target.*/
	UPROPERTY(Transient, NonTransactional)
	mutable TObjectPtr<UTextureRenderTarget2D> MediaRenderTarget;

	/** Processed media texture with additional passes only for scene composite meshes.*/
	UPROPERTY(Transient, NonTransactional)
	mutable TObjectPtr<UTextureRenderTarget2D> SceneRenderTarget;

	/** Internal pass to automatically remove overscan from the media texture. Not serialized. */
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UCompositePassTransform2D> OverscanPass;

	/** Cached number of valid media passes. */
	int32 CachedValidMediaPasses = 0;

	/** Cached number of valid scene passes. */
	int32 CachedValidScenePasses = 0;

	/** Toggled when a render target is resized to trigger a mesh material texture parameter rebind in Tick. */
	mutable bool bRebindRenderTarget = false;

	friend class FCompositeLayerPlateCustomization;
};

#undef UE_API

