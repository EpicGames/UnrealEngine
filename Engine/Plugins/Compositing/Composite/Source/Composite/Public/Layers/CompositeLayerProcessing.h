// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerBase.h"

#include "CompositeLayerProcessing.generated.h"

#define UE_API COMPOSITE_API

/**
 * Layer that runs sub-passes on the output of previous layers without merging an additional input.
 * Useful for inserting transforms or color operations between merged layers.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Processing"))
class UCompositeLayerProcessing : public UCompositeLayerBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerProcessing(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerProcessing();

	/** Getter function to override, returning pass layer proxies to be passed to the render thread. (Proxy objects should be allocated from the provided allocator.) */
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

#undef UE_API
