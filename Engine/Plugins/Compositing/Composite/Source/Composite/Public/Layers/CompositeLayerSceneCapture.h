// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerBase.h"
#include "CompositeSpawnableBinding.h"

#include "CompositeLayerSceneCapture.generated.h"

#define UE_API COMPOSITE_API

class AActor;
class UPrimitiveComponent;
class USceneCaptureComponent2D;

/**
 * Layer that renders selected primitives through a scene capture (or inlined custom render pass) and merges the result.
 * By default, registered primitives are hidden from the main render so they are rendered exclusively into this capture.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Scene Capture"))
class UCompositeLayerSceneCapture : public UCompositeLayerBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerSceneCapture(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerSceneCapture();
	
	UE_API virtual void OnRemoved(ACompositeActor* LastOwner) override;

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject Interface

	/** End-of-frame update callback. */
	UE_API virtual void OnEndOfFrameUpdate(UWorld* InWorld) override;

	UE_API virtual bool GetIsActive() const override;
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	/** Get reflection & shadow caster actors. */
	UFUNCTION(BlueprintGetter)
	UE_API const TArray<TSoftObjectPtr<AActor>> GetActors() const;

	/** Set reflection & shadow caster actors. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetActors(TArray<TSoftObjectPtr<AActor>> InActors);

	/** Get whether the scene capture is rendered as a custom render pass. */
	UFUNCTION(BlueprintGetter)
	UE_API bool IsCustomRenderPass() const;

	/** Set whether the scene capture is rendered as a custom render pass. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetCustomRenderPass(bool bInIsFastRenderPass);

	/** Get whether registered meshes are marked as visible in scene capture only. */
	UFUNCTION(BlueprintGetter)
	UE_API bool IsVisibleInSceneCaptureOnly() const;

	/** Set whether registered meshes are marked as visible in scene capture only. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetVisibleInSceneCaptureOnly(bool bInVisible);

	/** Get the render target resolution for the scene capture. */
	UFUNCTION(BlueprintGetter)
	UE_API FIntPoint GetRenderTargetResolution() const { return RenderTargetResolution; }

	/** Set the render target resolution for the scene capture. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetRenderTargetResolution(FIntPoint InRenderTargetResolution);

private:

	/** Convenience function to access the actor-managed scene capture component. */
	class UCompositeSceneCapture2DComponent* GetSceneCaptureComponent() const;

	/** Update scene capture visibility on registered mesh actors primitives. */
	void UpdatePrimitiveVisibilityState(bool bInVisibleInSceneCaptureOnly) const;

	/** Restore scene capture visibility on registered mesh actors primitives. */
	void RestorePrimitiveVisibilityState()  const;

	/** Apply actor-change side effects (visibility, scene capture update) without touching spawnable bindings. */
	void SetActorsInternal(TArray<TSoftObjectPtr<AActor>> InActors);

	/**
	 * List of actors to be rendered from the scene capture.
	 *
	 * Automatically sets the scene capture's ShowOnlyComponents when updated.
	*/
	UPROPERTY(EditAnywhere, BlueprintGetter = GetActors, BlueprintSetter = SetActors, Category = "Composite", meta = (AllowPrivateAccess = true))
	TArray<TSoftObjectPtr<AActor>> Actors;

	/** Parallel spawnable binding data for the Actors array. */
	UPROPERTY()
	FCompositeSpawnableBindings SpawnableBindings;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "CachedVisibilityInSceneCapture is now deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "CachedVisibilityInSceneCapture is now deprecated."))
	TMap<TWeakObjectPtr<UPrimitiveComponent>, bool> CachedVisibilityInSceneCapture;
#endif

	/** Render target resolution for the scene capture. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetRenderTargetResolution, BlueprintSetter = SetRenderTargetResolution, Interp, Category = "Composite", meta = (AllowPrivateAccess = true))
	FIntPoint RenderTargetResolution;

	/** Visibility setting applied to registered actors. By default, registered meshes will only be visible in scene captures and no longer in the main render. */
	UPROPERTY(EditAnywhere, BlueprintGetter = IsVisibleInSceneCaptureOnly, BlueprintSetter = SetVisibleInSceneCaptureOnly, Category = "Composite", meta = (AllowPrivateAccess = true))
	bool bVisibleInSceneCaptureOnly;

	/**
	 * Convenience toggle to render the scene capture as a fast/minimal Custom Render Pass, inlined in the main render & without support for lighting.
	 * 
	 * Automatically sets the scene capture's CaptureSource, bRenderInMainRenderer & bIgnoreScreenPercentage when updated.
	 */
	UPROPERTY(EditAnywhere, BlueprintGetter = IsCustomRenderPass, BlueprintSetter = SetCustomRenderPass, Category = "Composite", meta = (AllowPrivateAccess = true))
	bool bCustomRenderPass;

	friend class FCompositeLayerSceneCaptureCustomization;
};

#undef UE_API

