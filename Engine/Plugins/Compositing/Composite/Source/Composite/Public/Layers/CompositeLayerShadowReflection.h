// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerBase.h"
#include "CompositeSpawnableBinding.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Containers/ContainersFwd.h"

#include "CompositeLayerShadowReflection.generated.h"

#define UE_API COMPOSITE_API

class AActor;
class ACompositeActor;
class UCompositePassBase;
class UCompositeShadowReflectionCatcherComponent;
class USceneCaptureComponent2D;

/** Automatic primitive configuration mode for registered primitives. */
UENUM(BlueprintType)
enum class UE_DEPRECATED(5.8, "ECompositeHiddenInSceneCaptureConfiguration is now deprecated.") ECompositeHiddenInSceneCaptureConfiguration : uint8
{
	None UMETA(ToolTip = "No primitive properties are updated"),
	Visible UMETA(ToolTip = "The following properties are set to false: bHiddenInSceneCapture, bAffectIndirectLightingWhileHidden & bCastHiddenShadow."),
	Hidden UMETA(ToolTip = "The following properties are set to true: bHiddenInSceneCapture, bAffectIndirectLightingWhileHidden & bCastHiddenShadow."),
};

/** Primitive visibility settings cached state. */
USTRUCT()
struct UE_DEPRECATED(5.8, "FCompositeShadowReflectionPrimitiveState is now deprecated.") FCompositeShadowReflectionPrimitiveState
{
public:
	GENERATED_USTRUCT_BODY()
	/** Hidden in scene capture. */
	UPROPERTY()
	bool bHiddenInSceneCapture = false;

	/** Affect indirect lighting while hidden. */
	UPROPERTY()
	bool bAffectIndirectLightingWhileHidden = false;

	/** Cast hidden shadow. */
	UPROPERTY()
	bool bCastHiddenShadow = false;
};

/**
 * Layer that catches shadows and reflections received by registered primitives, producing a matte that is merged onto the plate.
 * Implemented with two scene captures (one with the primitives, one without); the ratio of their results yields a multiplicative matte.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Shadow Reflection Catcher"))
class UCompositeLayerShadowReflection : public UCompositeLayerBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerShadowReflection(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerShadowReflection();

	UE_API virtual void OnRemoved(ACompositeActor* LastOwner) override;

	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject Interface

	/** End-of-frame update callback. */
	UE_API virtual void OnEndOfFrameUpdate(UWorld* InWorld) override;

	UE_API virtual bool GetIsActive() const override;
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	/** Get the render target resolution for the scene captures. */
	UFUNCTION(BlueprintGetter)
	UE_API FIntPoint GetRenderTargetResolution() const { return RenderTargetResolution; }

	/** Set the render target resolution for the scene captures. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetRenderTargetResolution(FIntPoint InRenderTargetResolution);

	/** Get reflection & shadow caster actors. */
	UFUNCTION(BlueprintGetter)
	UE_API const TArray<TSoftObjectPtr<AActor>> GetActors() const;

	/** Set reflection & shadow caster actors. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetActors(TArray<TSoftObjectPtr<AActor>> InActors);

	/** Set the enabled state. */
	UE_API virtual void SetIsEnabled(bool bInEnabled) override;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "AutoConfigureActors is now deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "AutoConfigureActors is now deprecated."))
	ECompositeHiddenInSceneCaptureConfiguration AutoConfigureActors_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

private:
	/** Update scene capture visibility on registered mesh actors primitives. */
	void UpdatePrimitiveVisibilityState(bool bInHiddenInSceneCapture, bool bInAffectIndirectLightingWhileHidden, bool bInCastHiddenShadow, bool bConditionOnVisibility) const;

	/** Set primitive visibility state and console variables required for hidden in scene capture. */
	void SetHiddenPrimitiveState(bool bInHiddenInSceneCapture) const;

	/** Find or create scene captures with CG and without CG. */
	TStaticArray<TWeakObjectPtr<UCompositeShadowReflectionCatcherComponent>, 2> FindOrCreateSceneCapturePair(ACompositeActor& InOuter) const;

private:
	/** Render target resolution for the scene captures. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetRenderTargetResolution, BlueprintSetter = SetRenderTargetResolution, Interp, Category = "Composite", meta = (AllowPrivateAccess = true))
	FIntPoint RenderTargetResolution;

	/** Apply actor-change side effects without touching spawnable bindings. */
	void SetActorsInternal(TArray<TSoftObjectPtr<AActor>> InActors);

	/** List of reflection & shadow caster actors. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetActors, BlueprintSetter = SetActors, Category = "Composite", meta = (AllowPrivateAccess = true))
	TArray<TSoftObjectPtr<AActor>> Actors;

	/** Parallel spawnable binding data for the Actors array. */
	UPROPERTY()
	FCompositeSpawnableBindings SpawnableBindings;

public:
	/**
	 * When true, the matte is computed from a single (grayscale) luminance ratio applied uniformly to all channels,
	 * instead of independent per-channel ratios. This can help minimize noise from divisions by values near zero,
	 * or avoid incorrect chromaticity in Lumen reflections.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite")
	bool bLuminanceOnly = false;

private:
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "CachedVisibilityInSceneCapture is now deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "CachedVisibilityInSceneCapture is now deprecated."))
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FCompositeShadowReflectionPrimitiveState> CachedVisibilityInSceneCapture;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif


	/** Cached pointer to the scene captures with CG and without CG. */
	mutable TStaticArray<TWeakObjectPtr<UCompositeShadowReflectionCatcherComponent>, 2> CachedSceneCaptures;

	friend class FCompositeLayerShadowReflectionCustomization;
};

#undef UE_API

