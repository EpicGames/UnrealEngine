// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "CompositeLayerBase.h"
#include "CompositeSpawnableBinding.h"
#include "UObject/WeakObjectPtr.h"

#include "CompositeLayerSingleLightShadow.generated.h"

/**
* Note that much of the custom logic here could be removed given access to the engine's CSM/VSM built-in functionality.
* However, those techniques remain out-of-reach to plugins since they are renderer-private.
*/

#define UE_API COMPOSITE_API

class AActor;
class ALight;
class UCompositePassBase;
class UCompositeSceneCapture2DComponent;

/**
 * Layer that catches shadows cast by registered actors from a single specified light.
 *
 * Uses classic shadow-map percentage-closer filtering (5x5 by default, no cascades).
 * Implemented with custom render passes for minimal performance cost.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Single Light Shadow"))
class UCompositeLayerSingleLightShadow : public UCompositeLayerBase
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositeLayerSingleLightShadow(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositeLayerSingleLightShadow() = default;

	UE_API virtual void OnRemoved(ACompositeActor* LastOwner) override;

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject Interface

	/** End-of-frame update callback. */
	UE_API virtual void OnEndOfFrameUpdate(UWorld* InWorld) override;

	UE_API virtual bool GetIsActive() const override;
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/**
	* The reference light, whose direction or exact transform will by used to render a shadow map.
	* 
	* Only directional lights are currently supported.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta=(AllowedClasses="/Script/Engine.DirectionalLight"))
	TObjectPtr<ALight> Light;

	/** Get shadow caster actors. */
	UFUNCTION(BlueprintGetter)
	UE_API const TArray<TSoftObjectPtr<AActor>> GetShadowCastingActors() const;

	/** Set shadow caster actors. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetShadowCastingActors(TArray<TSoftObjectPtr<AActor>> InShadowCaster);

	/** 
	* When enable, the shadow map render transform and orthographic width are automatically derived from the selected meshes and light angle.
	* When disabled, the shadow map transform is copied from the light transform, which should be piloted to align with shadow casting geometry.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	bool bAutomaticBounds;

	/**
	* The desired width (in world units) of the shadow map view (ignored if light is not directional).
	* Reduce size to only contain the shadow casting geometry in view of the light.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (EditCondition = "!bAutomaticBounds", EditConditionHides))
	float OrthographicWidth;

	/** Get the render target resolution for the scene depth capture. */
	UFUNCTION(BlueprintGetter)
	UE_API FIntPoint GetRenderTargetResolution() const { return RenderTargetResolution; }

	/** Set the render target resolution for the scene depth capture. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetRenderTargetResolution(FIntPoint InRenderTargetResolution);

	/**
	* Resolution of the shadow map texture. Lower resolutions will cause more blurry shadows.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta = (ClampMin = "128", UIMin = "128", UIMax="4096"))
	int32 ShadowMapResolution;

	/** Basic shadow strength multiplier, from 0 to 1. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float ShadowStrength;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Shadow Bias is now deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Shadow Bias is now deprecated."))
	float ShadowBias_DEPRECATED = 0.5f;
#endif

private:
	/** Find or create scene depth capture component, correctly configured. */
	UCompositeSceneCapture2DComponent* FindOrCreateSceneDepthCapture(ACompositeActor& InOuter) const;

	/** Find or create shadow map capture component, correctly configured. */
	UCompositeSceneCapture2DComponent* FindOrCreateShadowMapCapture(ACompositeActor& InOuter) const;

	/** Convenience function to calculate shadow matrices. */
	void GetShadowMatrices(UCompositeSceneCapture2DComponent& ShadowMapCapture, FMatrix44f& OutShadowMatrix, FVector4f& OutShadowInvDeviceZToWorldZ) const;

private:
	/** Render target resolution for the scene depth capture. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetRenderTargetResolution, BlueprintSetter = SetRenderTargetResolution, Interp, Category = "Composite", meta = (AllowPrivateAccess = true))
	FIntPoint RenderTargetResolution;

	/** Apply actor-change side effects without touching spawnable bindings. */
	void SetShadowCastingActorsInternal(TArray<TSoftObjectPtr<AActor>> InShadowCasters);

	/** List of shadow casting actors (required). */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetShadowCastingActors, BlueprintSetter = SetShadowCastingActors, Category = "Composite", meta = (AllowPrivateAccess = true))
	TArray<TSoftObjectPtr<AActor>> ShadowCastingActors;

	/** Parallel spawnable binding data for the ShadowCastingActors array. */
	UPROPERTY()
	FCompositeSpawnableBindings SpawnableBindings;

	/** Cached pointer to the composite-managed shadow map scene capture. */
	mutable TWeakObjectPtr<UCompositeSceneCapture2DComponent> CachedShadowMapCapture;

	/** Cached pointer to the scene depth capture. */
	mutable TWeakObjectPtr<UCompositeSceneCapture2DComponent> CachedSceneDepthCapture;

	friend class FCompositeLayerSingleLightShadowCustomization;
};

#undef UE_API

