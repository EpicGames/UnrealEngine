// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "UObject/Object.h"

#include "Passes/CompositeCorePassProxy.h"

#include "CompositePassBase.generated.h"

#define UE_API COMPOSITE_API

/** Context used while traversing passes and layers to collect frame proxies. */
struct FCompositeTraversalContext
{
	/** Flag to indicate solo pass */
	bool bIsSolo = false;

	/** Flag to indicate first pass */
	bool bIsFirstPass = true;

	/**
	 * Flag to indicate that a layer is dependent on scene textures.
	 * Work from scene-independent layers can be shared between post-processing locations.
	 */
	bool bNeedsSceneTextures = false;

	/** Flag to indicate that passes are being collected for preprocessing (before the main render). */
	bool bIsPreprocessing = false;

	/** Array of passes applied at the start of rendering. */
	TSortedMap<UE::CompositeCore::ResourceId, TArray<const FCompositeCorePassProxy*>> PreprocessingPasses;

	/**
	* Find or create an external texture resource.
	*
	* @return ResourceId Texture resource identifier
	*/
	UE::CompositeCore::ResourceId FindOrCreateExternalTexture(TWeakObjectPtr<UTexture> InTexture, UE::CompositeCore::FResourceMetadata InMetadata);

	/** Get external textures. */
	const TArray<UE::CompositeCore::FExternalTexture>& GetExternalTextures() const;

private:
	/** List of external textures */
	TArray<UE::CompositeCore::FExternalTexture> ExternalTextures;
};

/**
 * Base class for a composite pass: a render-thread operation applied to a layer's input.
 * Subclasses spawn a per-frame render proxy via GetProxy() that runs on the GPU.
 */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UCompositePassBase : public UObject
{
	GENERATED_BODY()

public:
	/** Constructor. */
	UE_API UCompositePassBase(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor. */
	UE_API virtual ~UCompositePassBase();

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const;
#endif

	/** Override to return a render-thread proxy for this pass. Proxy objects should be allocated from the provided allocator. Only called when GetIsActive() returns true. */
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const { return nullptr; }

	/** Get the enabled state. */
	UFUNCTION(BlueprintGetter)
	UE_API virtual bool GetIsEnabled() const;

	/** Set the enabled state. */
	UFUNCTION(BlueprintSetter, CallInEditor)
	UE_API virtual void SetIsEnabled(bool bInEnabled);

	/** Returns true if the pass should participate in rendering this frame. Checked before GetProxy() is called. */
	virtual bool GetIsActive() const { return GetIsEnabled(); }

#if WITH_EDITOR
	/** Gets the display name of the pass */
	FString GetDisplayName() const { return DisplayName; }

	/** Sets the display name of the pass */
	void SetDisplayName(const FString& InDisplayName) { DisplayName = InDisplayName; }

	/** Generates a unique display name by appending a number if siblings share the same base type name. First instance stays clean; duplicates start at 2. */
	template<typename T>
	static FString MakeUniqueDisplayName(const FString& BaseName, TArrayView<TObjectPtr<T>> Siblings, const UCompositePassBase* Exclude)
	{
		static_assert(std::is_base_of_v<UCompositePassBase, T>, "T must derive from UCompositePassBase");
		
		int32 MatchCount = 0;
		for (const TObjectPtr<T>& Sibling : Siblings)
		{
			if (Sibling && Sibling != Exclude && Sibling->GetClass()->GetDisplayNameText().ToString() == BaseName)
			{
				++MatchCount;
			}
		}
		
		if (MatchCount == 0)
		{
			return BaseName;
		}

		return FString::Printf(TEXT("%s %d"), *BaseName, MatchCount + 1);
	}
#endif
	
protected:
	/** Whether or not the pass is active. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetIsEnabled, BlueprintSetter = SetIsEnabled, Interp, Category = "Composite", meta = (DisplayPriority = "1"))
	bool bIsEnabled = true;

#if WITH_EDITORONLY_DATA
	/** The display name of the pass */
	UPROPERTY(EditAnywhere, Category = "Composite", meta = (DisplayName="Name", DisplayPriority = "0"))
	FString DisplayName;
#endif

	friend class FCompositePassBaseCustomization;
};

#undef UE_API

