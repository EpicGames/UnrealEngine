// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"

#include "CompositePassBlur.generated.h"

#define UE_API COMPOSITE_API

/** Blur method type. */
UENUM(BlueprintType)
enum class ECompositePassBlurMethod : uint8
{
	Gaussian,
};

/**
 * Applies a blur to the layer input.
 * Currently supports Gaussian blur with independent X and Y radius controls.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Blur"))
class UCompositePassBlur : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassBlur(const FObjectInitializer& ObjectInitializer);

	/** Destructor */
	UE_API ~UCompositePassBlur();

	UE_API virtual bool GetIsActive() const override;

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	UE_API virtual void PostLoad() override;

public:
	/** Blur pass method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite")
	ECompositePassBlurMethod Method;

	/** Blur radius in pixels (X = horizontal, Y = vertical). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (DisplayName = "Radius", ClampMin = "1", ClampMax = "64", UIMin = "1", UIMax = "64", AllowPreserveRatio))
	FIntPoint RadiusXY = FIntPoint(1, 1);

	/** Only blur the alpha channel, leaving RGB unchanged. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bAlphaOnly = false;

#if WITH_EDITORONLY_DATA
private:
	/** Deprecated scalar radius from prior versions. Tagged-property load matches old "Radius" int32 tag; migrated to RadiusXY in PostLoad. */
	UE_DEPRECATED(5.8, "Radius has been deprecated, use RadiusXY instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Radius has been deprecated, use RadiusXY instead."))
	int32 Radius_DEPRECATED = 0;
#endif
};

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			using namespace CompositeCore;

			/** Render-thread proxy for the Gaussian blur pass. Runs a separable horizontal + vertical convolution over the input. */
			class FCompositePassBlurProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FCompositePassBlurProxy);

				using FCompositeCorePassProxy::FCompositeCorePassProxy;

				/** Adds the Gaussian blur RDG passes (horizontal then vertical) and returns the blurred output. */
				FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

				/** Blur kernel half-width in pixels (X = horizontal, Y = vertical). */
				FIntPoint Radius = FIntPoint(1, 1);
				/** When true, only the alpha channel is blurred; RGB is passed through unchanged. */
				bool bAlphaOnly = false;
			};
		}
	}
}

#undef UE_API
