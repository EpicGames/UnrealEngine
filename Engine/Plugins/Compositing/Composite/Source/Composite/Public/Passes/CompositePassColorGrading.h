// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CompositePassBase.h"
#include "Passes/CompositeCorePassMergeProxy.h"
#include "Engine/Scene.h"

#include "CompositePassColorGrading.generated.h"

#define UE_API COMPOSITE_API

/** Color grading pass temperature settings. */
USTRUCT(BlueprintType)
struct FCompositeTemperatureSettings
{
	GENERATED_USTRUCT_BODY()

	/**
	* Selects the type of temperature calculation.
	* White Balance uses the Temperature value to control the virtual camera's White Balance. This is the default selection.
	* Color Temperature uses the Temperature value to adjust the color temperature of the scene, which is the inverse of the White Balance operation.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Temperature", meta = (DisplayName = "Temperature Type"))
	TEnumAsByte<enum ETemperatureMethod> TemperatureType = ETemperatureMethod::TEMP_WhiteBalance;

	/** Controls the color temperature or white balance in degrees Kelvin which the scene considers as white light. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Temperature", meta = (UIMin = "1500.0", UIMax = "15000.0", DisplayName = "Temp"))
	float WhiteTemp = 6500.0f;
	
	/** Controls the color of the scene along the magenta - green axis (orthogonal to the color temperature).  This feature is equivalent to color tint in digital cameras. */
	UPROPERTY(interp, BlueprintReadWrite, Category = "Color Grading|Temperature", meta = (UIMin = "-1.0", UIMax = "1.0", DisplayName = "Tint"))
	float WhiteTint = 0.0f;
};

/**
 * Applies exposure, contrast, saturation, temperature and color-wheel adjustments to the layer input.
 * Assumes input is in the linear working color space.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Color Grading"))
class UCompositePassColorGrading : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassColorGrading(const FObjectInitializer& ObjectInitializer);
	/** Destructor */
	UE_API ~UCompositePassColorGrading();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/**
	 * Set true when the input texture stores premultiplied alpha (RGB already scaled by A).
	 * The pass will unpremultiply before processing and re-premultiply after, so color
	 * grading operates on straight-alpha values. Leave false for straight-alpha inputs.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite")
	bool bInputIsPremultiplied;

	/** Color temperature settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite", meta = (DisplayName = "Temperature"))
	FCompositeTemperatureSettings TemperatureSettings;

	/** Color grading settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite", meta = (ShowOnlyInnerProperties))
	FColorGradingSettings ColorGradingSettings;
};

USTRUCT(BlueprintType)
struct FCompositeColorGradingPassSettings
{
	GENERATED_USTRUCT_BODY()

	/** Hidden in scene capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Composite")
	bool bEnableColorGrading = false;

	/**
	 * Set true when the input texture stores premultiplied alpha (RGB already scaled by A).
	 * The pass will unpremultiply before processing and re-premultiply after, so color
	 * grading operates on straight-alpha values. Leave false for straight-alpha inputs.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Composite")
	bool bInputIsPremultiplied = true;

	/** Color temperature settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Composite", meta = (DisplayName = "Temperature"))
	FCompositeTemperatureSettings TemperatureSettings;

	/** Color grading settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Composite", meta = (ShowOnlyInnerProperties))
	FColorGradingSettings ColorGradingSettings;
};

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			/** Render-thread proxy for the color grading pass. Applies white balance and shadow/midtone/highlight color grading to the input. */
			class FColorGradingCompositePassProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FColorGradingCompositePassProxy);

				/** Constructor */
				FColorGradingCompositePassProxy(CompositeCore::FPassInputDeclArray InPassDeclaredInputs);

				/** Adds the color grading RDG pass; applies temperature and color grading adjustments and returns the graded output. */
				CompositeCore::FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const CompositeCore::FPassInputArray& Inputs, const CompositeCore::FPassContext& PassContext) const override;

				/** When true, the input is treated as premultiplied alpha: RGB is divided by alpha before processing and multiplied back after. */
				bool bInputIsPremultiplied = true;
				
				/** Color temperature settings. */
				FCompositeTemperatureSettings TemperatureSettings;
				
				/** Color grading settings. */
				FColorGradingSettings ColorGradingSettings;
			};
		}
	}
}

#undef UE_API
