// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"

#include "CompositePassColorKeyer.generated.h"

#define UE_API COMPOSITE_API

/** Visualization mode, for inspecting your key.*/
UENUM(BlueprintType)
enum class ECompositeColorKeyerVisualization : uint8
{
	Key = 0,
	Fill = 1,
	Alpha = 2
};

/** Background screen color type (red, green or blue).*/
UENUM(BlueprintType)
enum class ECompositeColorKeyerScreenType : uint8
{
	Red = 0,
	Green = 1,
	Blue = 2
};

/** Keying mode: chroma key (color-based) or luma key (luminance-based). */
UENUM(BlueprintType)
enum class ECompositeColorKeyerMode : uint8
{
	Chroma = 0 UMETA(DisplayName = "Chroma Key"),
	Luma   = 1 UMETA(DisplayName = "Luma Key"),
};

/** Denoising method applied before keying. */
UENUM(BlueprintType)
enum class ECompositeDenoiseMethod : uint8
{
	None,
	Median3x3 UMETA(DisplayName = "Median 3x3"),
};

UENUM(BlueprintType)
enum class ECompositeKeyerSource : uint8
{
	Color,
	CleanPlate
};

/**
 * Extracts alpha from a colored background (red, green or blue) using chroma keying.
 * Supports clean-plate differencing, despill and denoising.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="Color Keyer"))
class UCompositePassColorKeyer : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassColorKeyer(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor */
	UE_API ~UCompositePassColorKeyer();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	UE_API virtual void PostLoad() override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	/** Type of screen color (required). The keyer works best against red, green or blue backgrounds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	ECompositeColorKeyerScreenType ScreenType;

	/** Source used by the keyer to derive the screen color: either a static key color or a clean plate texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta = (PropertyGroup = "Keyer Settings"))
	ECompositeKeyerSource KeyerSource;
	
	/** Static background key color.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (PropertyGroup = "Keyer Settings", OnlyUpdateOnInteractionEnd, EditCondition = "KeyerSource == ECompositeKeyerSource::Color", EditConditionHides))
	FLinearColor KeyColor;

	/**
	 * Clean plate background for calculating color differences per pixel, instead of the static key color.
	 * Resolution must match the composite actor render resolution.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta = (PropertyGroup = "Keyer Settings", OnlyUpdateOnInteractionEnd, AllowedClasses = "/Script/Engine.Texture2D, /Script/Engine.Texture2DDynamic, /Script/Engine.TextureRenderTarget2D, /Script/MediaAssets.MediaTexture", EditCondition = "KeyerSource == ECompositeKeyerSource::CleanPlate", EditConditionHides))
	TObjectPtr<UTexture> CleanPlate;

#if WITH_EDITORONLY_DATA
	/** Setting that controls how long to delay before capturing a clean plate */
	UPROPERTY(EditAnywhere, Category = "Composite", meta = (PropertyGroup = "Keyer Settings", EditCondition = "KeyerSource == ECompositeKeyerSource::CleanPlate", EditConditionHides, ClampMin=0.0, UIMin=0.0, ClampMax=60.0, UIMax=60.0))
	float CleanPlateCountdown = 0.0f;

	/** Setting that determines how large to display the plate texture 'picture-in-picture' preview in the level editor when using the eye dropper color picker */
	UPROPERTY(EditAnywhere, Category = "Composite", meta = (PropertyGroup = "Keyer Settings", EditCondition = "KeyerSource == ECompositeKeyerSource::Color", EditConditionHides, ClampMin=1.0, UIMin=1.0, ClampMax=10.0, UIMax=10.0))
	float PlateTexturePreviewSize = 5.0f;
#endif
	
	/** Weight of the foreground red channel contributing to the key matte hardness. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = 0.0f, UIMax = 1.0f, EditCondition = "ScreenType != ECompositeColorKeyerScreenType::Red"))
	float RedWeight;

	/** Weight of the foreground green channel contributing to the key matte hardness. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = 0.0f, UIMax = 1.0f, EditCondition = "ScreenType != ECompositeColorKeyerScreenType::Green"))
	float GreenWeight;

	/** Weight of the foreground blue channel contributing to the key matte hardness. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = 0.0f, UIMax = 1.0f, EditCondition = "ScreenType != ECompositeColorKeyerScreenType::Blue"))
	float BlueWeight;

	/** Thresholds any alpha value outside the specified range to zero or one respectively, with linear interpolation in-between. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = 0.0f, UIMax = 1.0f))
	FVector2f AlphaThreshold;

	/** Strength of the spill reduction, 0.0: none, 1.0: full. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = 0.0f, UIMax = 1.0f))
	float DespillStrength;

	/** Strength of the vignette removal. Used to improve plate uniformity & remove darker corners. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = 0.0f, UIMax = 1.0f))
	float DevignetteStrength;

	/** When enabled, we undo devignetting before outputting the keyed plate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bPreserveVignetteAfterKey;

	/** Denoising method applied before the keyer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	ECompositeDenoiseMethod DenoiseMethod;

	/**
	 * Provided so users can compensate for upstream media whose chroma may be offset relative to luma
	 * with chroma-subsampled sources (such as 4:2:2 YUV captures).
	 * 
	 * Applies a horizontal chroma shift, in input texels, to the foreground sampled by the color keyer pass.
	 * Center-pixel luma is preserved. Zero disables the shift.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay, meta = (UIMin = -1.0f, UIMax = 1.0f, ClampMin = -1.0f, ClampMax = 1.0f))
	float ChromaShift;

	/** Vizualize the alpha key or fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	ECompositeColorKeyerVisualization Visualization;

	/** Only apply the despill algorithm controlled by DespillStrength, alpha is unmodified. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bDespillOnly;

	/** Invert the alpha key. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bInvertAlpha;

#if WITH_EDITOR
	/** Flag tracking whether we disabled throttling to later restore it. */
	bool bEditorThrottleDisabled = false;
#endif

private:
	friend class FCompositePassColorKeyerCustomization;
};

/**
 * Extracts alpha from pixel luminance (luma key).
 * Useful for isolating bright or dark regions without a colored backdrop.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="Luma Keyer"))
class UCompositePassLumaKeyer : public UCompositePassBase
{
	GENERATED_BODY()

public:
	UE_API UCompositePassLumaKeyer(const FObjectInitializer& ObjectInitializer);
	UE_API ~UCompositePassLumaKeyer();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Luminance range [min, max] that maps to alpha [0, 1]. Pixels with luminance below min are fully transparent; above max are fully opaque. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (ClampMin = 0.0f, ClampMax = 1.0f, UIMin = 0.0f, UIMax = 1.0f))
	FVector2f LumaRange;

	/** Vizualize the alpha key or fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	ECompositeColorKeyerVisualization Visualization;

	/** Invert the alpha key. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bInvertAlpha;

	/**
	 * When enabled, luminance is computed after converting the pixel to display space (sRGB encoding) rather than in linear light.
	 * This matches the luma-key behavior of other tools, where LumaRange values correspond to what you see on screen.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bDisplaySpaceLuminance;

	/** Denoising method applied before the keyer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", AdvancedDisplay)
	ECompositeDenoiseMethod DenoiseMethod;
};

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			using namespace CompositeCore;
			/** Render-thread proxy for the color keyer pass. Performs chroma-key extraction and optional despill on the input. */
			class FCompositePassColorKeyerProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FCompositePassColorKeyerProxy);
				using FCompositeCorePassProxy::FCompositeCorePassProxy;

				/** Adds the color keyer RDG pass and returns a premultiplied keyed RGBA output. */
				FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

				/** Whether this proxy uses chroma or luma keying. */
				ECompositeColorKeyerMode          KeyerMode              = ECompositeColorKeyerMode::Chroma;
				/** Background screen color used to drive the key algorithm. */
				ECompositeColorKeyerScreenType    ScreenType             = ECompositeColorKeyerScreenType::Green;
				/** Static background reference color; used when no clean plate is provided. */
				FLinearColor                      KeyColor;
				/** Shader parameter vector: (RedWeight, GreenWeight, BlueWeight, DespillStrength). */
				FVector4f                         Params0                = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
				/** Shader parameter vector: chroma=(AlphaMin, AlphaMax, DevignetteStrength, ChromaShift), luma=(LumaMin, LumaMax, 0, 0). */
				FVector4f                         Params1                = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
				/** Which output channel to display: final composite key, fill layer, or raw alpha matte. */
				ECompositeColorKeyerVisualization Visualization          = ECompositeColorKeyerVisualization::Key;
				/** When true, devignette correction is reversed before outputting the keyed result. */
				bool                              bPreserveVignetteAfterKey = true;
				/** Denoising pre-pass applied to the foreground before keying. */
				ECompositeDenoiseMethod           DenoiseMethod          = ECompositeDenoiseMethod::None;
				/** When true, only despill is applied; alpha is left unchanged. */
				bool                              bDespillOnly           = false;
				/** When true, the output alpha is inverted (1 − alpha). */
				bool                              bInvertAlpha           = false;
				/** When true, luminance is computed in display (sRGB encoding) space to match other tools. */
				bool                              bDisplaySpaceLuminance = false;
				/** Weak pointer to the clean plate texture used as the per-pixel background reference. Null when KeyColor is used instead. */
				TWeakObjectPtr<UTexture>          CleanPlateWeakPtr;
			};
		}
	}
}

#undef UE_API

