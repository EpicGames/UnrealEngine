// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"
#include "Passes/CompositeCorePassProxy.h"

#include "CompositePassDilation.generated.h"

#define UE_API COMPOSITE_API

/** Channel bitmask controlling which channels the morphological operation reads and writes. */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ECompositeDilationChannel : uint8
{
	None  = 0        UMETA(Hidden),
	Red   = 0x1,
	Green = 0x2,
	Blue  = 0x4,
	Alpha = 0x8,
};
ENUM_CLASS_FLAGS(ECompositeDilationChannel);

/**
 * Morphologically dilates or erodes selected channels of the layer input by a configurable pixel radius.
 * Used to grow or shrink mattes before downstream compositing operations.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Dilation"))
class UCompositePassDilation : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassDilation(const FObjectInitializer& ObjectInitializer);

	/** Destructor */
	UE_API ~UCompositePassDilation();

	UE_API virtual bool GetIsActive() const override;

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Signed size: negative = erode (shrink), positive = dilate (expand), 0 = identity (pass disabled). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin = "-20", UIMax = "20", ClampMin = "-64", ClampMax = "64"))
	int32 Size = 1;

	/** Channels the operation reads and writes. Defaults to all channels so a fresh Dilation Pass affects RGB and alpha together. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite",
		meta = (Bitmask, BitmaskEnum = "/Script/Composite.ECompositeDilationChannel"))
	int32 Channels =
		static_cast<int32>(ECompositeDilationChannel::Red)
		| static_cast<int32>(ECompositeDilationChannel::Green)
		| static_cast<int32>(ECompositeDilationChannel::Blue)
		| static_cast<int32>(ECompositeDilationChannel::Alpha);

	/** When true, the operation only fires on pixels that are essentially off (at or below a small epsilon), thus only affecting matte edges. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, AdvancedDisplay, Category = "Composite")
	bool bUseThreshold = false;

	/** In standard opacity space (0 = transparent, 1 = opaque): a channel's value is treated as "off" at or below this threshold and "on" above it. Only used when Use Threshold is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, AdvancedDisplay, Category = "Composite", meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bUseThreshold", EditConditionHides))
	float Threshold = 0.0001f;

	/** When Alpha is selected, the alpha-winning neighbor's RGB is copied into any color channels that aren't explicitly in the Channels mask, so mattes grow/shrink visibly. Disable for strict "alpha only" semantics (RGB untouched). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Composite", meta = (DisplayName = "Carry RGB With Alpha"))
	bool bCarryRGBWithAlpha = true;
};

#undef UE_API

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			using namespace CompositeCore;

			/** Render-thread proxy for the morphological dilation/erosion pass. */
			class FCompositePassDilationProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FCompositePassDilationProxy);

				using FCompositeCorePassProxy::FCompositeCorePassProxy;

				/** Adds ceil(AbsSize/2) RDG compute passes covering AbsSize pixels of dilation or erosion and returns the result.
				 *  Each pass uses a DILATION_SIZE=2 kernel (2 pixels/dispatch) where possible, falling back to
				 *  DILATION_SIZE=1 for an odd remainder. Intermediate results ping-pong through internal textures. */
				FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

				/** Total pixel reach of the dilation/erosion (dispatched in greedy 2-pixel steps). */
				int32 AbsSize = 1;
				/** When true, runs erosion (shrink opaque); when false, runs dilation (expand opaque). */
				bool bErode = false;
				/** Bitmask of channels to operate on: R=0x1, G=0x2, B=0x4, A=0x8. */
				uint32 ChannelMask = 0xFu;
				/** When true, gate the operation on Threshold. When false, run true min/max morphology. */
				bool bUseThreshold = false;
				/** Opacity threshold (standard opacity space) used only when bUseThreshold is true. */
				float Threshold = 0.0001f;
				/** When true, copy the alpha-winning neighbor's RGB into RGB channels not in the mask (so mattes grow visibly). */
				bool bCarryRGBWithAlpha = true;
			};
		}
	}
}
