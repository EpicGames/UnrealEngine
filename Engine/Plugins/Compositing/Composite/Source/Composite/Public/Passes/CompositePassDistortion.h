// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CompositePassBase.h"
#include "Passes/CompositeCorePassMergeProxy.h"

#include "CompositePassDistortion.generated.h"

#define UE_API COMPOSITE_API

class FLensDistortionSceneViewExtension;

UENUM(BlueprintType)
enum class ECompositeDistortion : uint8
{
	Undistort,
	Distort,
};

/**
 * Applies a lens distortion or undistortion transform to the layer input.
 * Driven by a distortion model supplied through the camera calibration system.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Distortion"))
class UCompositePassDistortion : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassDistortion(const FObjectInitializer& ObjectInitializer);
	/** Destructor */
	UE_API ~UCompositePassDistortion();

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	virtual bool GetIsActive() const override;

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Distortion to apply on input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	ECompositeDistortion Distortion;

private:
	/** Cached lens distortion scene view extension. */
	TSharedPtr<FLensDistortionSceneViewExtension, ESPMode::ThreadSafe> LensDistortionSVE;
};

#undef UE_API
