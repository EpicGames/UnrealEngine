// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"

#include "CompositePassFXAA.generated.h"

#define UE_API COMPOSITE_API

/**
 * Applies FXAA anti-aliasing to the layer input.
 * Primarily intended for CG or motion-graphics layers rendered through a custom render pass.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="FXAA"))
class UCompositePassFXAA : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassFXAA(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor */
	UE_API ~UCompositePassFXAA();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:

	/** FXAA quality control, matching the "r.fxaa.quality" console variable values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta=(ClampMin="0", ClampMax="5", UIMin = "0", UIMax = "5"))
	int32 Quality;

	/** Apply a display transform (tonemap + gamma encode) before AA and invert it after. Improves perceptual edge detection on linear content but may darken edges at alpha boundaries. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", AdvancedDisplay)
	bool bDisplayTransform = false;
};

#undef UE_API

