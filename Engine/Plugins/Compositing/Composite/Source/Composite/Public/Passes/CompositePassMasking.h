// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"
#include "CompositeSpawnableBinding.h"
#include "Tickable.h"

#include "CompositePassMasking.generated.h"

#define UE_API COMPOSITE_API

/** Color channel. */
UENUM(BlueprintType)
enum class ECompositeColorChannel : uint8
{
	Red,
	Green,
	Blue,
	Alpha
};

/** Masking source mode. */
UENUM(BlueprintType)
enum class ECompositeMaskingMode : uint8
{
	Texture = 0  UMETA(ToolTip = "Use a texture as the mask source."),
	Geometry = 1 UMETA(ToolTip = "Render selected meshes as a geometry mask via custom render pass."),
};

class AActor;
class UPrimitiveComponent;
class UCompositeSceneCapture2DComponent;

/**
 * Masks the layer alpha using a texture, a geometry render of selected actors, or a combination of both.
 * The resulting mask is multiplied into the input alpha.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="Masking"))
class UCompositePassMasking : public UCompositePassBase, public FTickableGameObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassMasking(const FObjectInitializer& ObjectInitializer);
	
	/** Destructor */
	UE_API ~UCompositePassMasking();

	UE_API virtual bool GetIsActive() const override;

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif
	//~ End UObject Interface

	//~ Begin FTickableGameObject Interface
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual bool IsTickableWhenPaused() const override { return true; }
	UE_API virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual ETickableTickType GetTickableTickType() const override;
	UE_API virtual bool IsTickable() const override;
	UE_API virtual UWorld* GetTickableGameObjectWorld() const override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject Interface

	/** Get the masking mode. */
	UFUNCTION(BlueprintGetter)
	UE_API ECompositeMaskingMode GetMaskingMode() const { return MaskingMode; }

	/** Set the masking mode. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetMaskingMode(ECompositeMaskingMode InMaskingMode);

	/** Get mask actors. */
	UFUNCTION(BlueprintGetter)
	UE_API const TArray<TSoftObjectPtr<AActor>> GetMaskActors() const;

	/** Set mask actors. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetMaskActors(TArray<TSoftObjectPtr<AActor>> InMaskActors);

	/** Get whether mask meshes are visible in scene capture only. */
	UFUNCTION(BlueprintGetter)
	UE_API bool IsVisibleInSceneCaptureOnly() const;

	/** Set whether mask meshes are visible in scene capture only. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetVisibleInSceneCaptureOnly(bool bInVisible);

	/** Get the render target resolution for the geometry mask. */
	UFUNCTION(BlueprintGetter)
	UE_API FIntPoint GetRenderTargetResolution() const { return RenderTargetResolution; }

	/** Set the render target resolution for the geometry mask. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetRenderTargetResolution(FIntPoint InRenderTargetResolution);

private:
	/** Masking source mode. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetMaskingMode, BlueprintSetter = SetMaskingMode, Interp, Category = "Composite", meta = (AllowPrivateAccess = true))
	ECompositeMaskingMode MaskingMode;

public:
	/** Masking texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta = (AllowedClasses = "/Script/Engine.Texture2D, /Script/Engine.Texture2DDynamic, /Script/Engine.TextureRenderTarget2D, /Script/MediaAssets.MediaTexture", EditCondition = "MaskingMode == ECompositeMaskingMode::Texture", EditConditionHides))
	TObjectPtr<UTexture> MaskTexture;

	/** Color channel used as the mask source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite", meta = (EditCondition = "MaskingMode == ECompositeMaskingMode::Texture", EditConditionHides))
	ECompositeColorChannel MaskSource;

private:
	/** Actors whose mesh primitives are rendered as a geometry mask. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetMaskActors, BlueprintSetter = SetMaskActors, Category = "Composite", meta = (AllowPrivateAccess = true, EditCondition = "MaskingMode == ECompositeMaskingMode::Geometry", EditConditionHides))
	TArray<TSoftObjectPtr<AActor>> MaskActors;

	/** Parallel spawnable binding data for the MaskActors array. */
	UPROPERTY()
	FCompositeSpawnableBindings SpawnableBindings;

	/** Render target resolution for the geometry mask scene capture. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetRenderTargetResolution, BlueprintSetter = SetRenderTargetResolution, Interp, Category = "Composite", meta = (AllowPrivateAccess = true, EditCondition = "MaskingMode == ECompositeMaskingMode::Geometry", EditConditionHides))
	FIntPoint RenderTargetResolution;

	/** Visibility setting applied to mask actors. By default, mask meshes will only be visible in scene captures. */
	UPROPERTY(EditAnywhere, BlueprintGetter = IsVisibleInSceneCaptureOnly, BlueprintSetter = SetVisibleInSceneCaptureOnly, Category = "Composite", meta = (AllowPrivateAccess = true, EditCondition = "MaskingMode == ECompositeMaskingMode::Geometry", EditConditionHides))
	bool bVisibleInSceneCaptureOnly;

public:
	/**
	 * Set true when the input texture stores premultiplied alpha (RGB already scaled by A).
	 * The pass will unpremultiply before masking and re-premultiply against the new alpha
	 * after, so RGB stays consistent with the masked alpha. Leave false for straight-alpha inputs.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite")
	bool bInputIsPremultiplied;

	/**
	 * Set true when the input's premultiplication was applied in sRGB display space rather than linear
	 * (e.g. an Ultimatte fill, where the upstream processor multiplies the camera fill by the matte in
	 * display before delivering it).
	 *
	 * Has no effect when bInputIsPremultiplied is false.
	 */
	UPROPERTY(BlueprintReadWrite, Interp, Category = "Composite")
	bool bPremultipliedInDisplaySpace;

	/** Invert mask alpha. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite")
	bool bInvert;

private:
	/** Apply mask-actor-change side effects without touching spawnable bindings. */
	void SetMaskActorsInternal(TArray<TSoftObjectPtr<AActor>> InMaskActors);

	/** Find or create the scene capture component from the owning composite actor. Configures it as custom render pass on creation. */
	UCompositeSceneCapture2DComponent* FindOrCreateSceneCaptureComponent() const;

#if WITH_EDITORONLY_DATA
	/** Cached masking mode from PreEditChange, used to detect transitions in PostEditChangeProperty. */
	ECompositeMaskingMode PreEditMaskingMode = ECompositeMaskingMode::Texture;
#endif

	/** Update scene capture visibility on mask actor primitives. */
	void UpdatePrimitiveVisibilityState(bool bInVisibleInSceneCaptureOnly) const;

	/** Restore scene capture visibility on mask actor primitives to class defaults. */
	void RestorePrimitiveVisibilityState() const;

	friend class FCompositePassMaskingCustomization;
};

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			using namespace CompositeCore;

			/** Render-thread proxy for the alpha masking pass. Takes two inputs: [0] main, [1] mask. */
			class FMaskingPassProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FMaskingPassProxy);

				using FCompositeCorePassProxy::FCompositeCorePassProxy;

				/** Adds the masking RDG pass; multiplies the main input's alpha by the chosen mask channel and returns the masked output. */
				FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

				/** Index of the mask texture channel to read (0=R, 1=G, 2=B, 3=A). */
				uint32 MaskSourceChannel = 0;
				/** When true, RGB is divided by alpha before masking and multiplied back after, preserving straight-alpha color values. */
				bool   bInputIsPremultiplied = true;
				/** When true (and bInputIsPremultiplied is true), the unpremultiply step is performed in display space rather than linear so display-premult content (e.g. an Ultimatte fill) round-trips to a clean linear-premult output. */
				bool   bPremultipliedInDisplaySpace = false;
				/** When true, the mask value is inverted (1 − mask) before it is applied. */
				bool   bInvert    = false;
			};
		}
	}
}

/**
 * Convenience masking pass for Ultimatte inputs.
 * The input is always treated as premultiplied; only the mask texture and invert toggle are exposed.
 * The mask is sampled from the red channel (the matte is delivered as a grayscale image).
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="Ultimatte Masking"))
class UCompositePassUltimatteMasking : public UCompositePassBase
{
	GENERATED_BODY()

public:
	UE_API UCompositePassUltimatteMasking(const FObjectInitializer& ObjectInitializer);

	UE_API virtual bool GetIsActive() const override;
	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Masking texture. The red channel is used as the mask source (Ultimatte mattes are grayscale). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite", meta = (AllowedClasses = "/Script/Engine.Texture2D, /Script/Engine.Texture2DDynamic, /Script/Engine.TextureRenderTarget2D, /Script/MediaAssets.MediaTexture"))
	TObjectPtr<UTexture> MaskTexture;

	/** Invert mask alpha. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite")
	bool bInvert;
};

#undef UE_API

