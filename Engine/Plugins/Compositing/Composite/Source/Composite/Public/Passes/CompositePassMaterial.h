// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"

#include "CompositePassMaterial.generated.h"

#define UE_API COMPOSITE_API

class UMaterialInterface;

/**
 * Runs a post-process material on the layer input.
 * Input0 is mapped to the material's PostprocessInput0 SceneTexture.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "Post-Process Material"))
class UCompositePassMaterial : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassMaterial(const FObjectInitializer& ObjectInitializer);

	/** Destructor */
	UE_API ~UCompositePassMaterial();

	UE_API virtual bool GetIsActive() const override;

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Post-process material to execute. Input0 is connected to SceneTexture's PostprocessInput0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	TObjectPtr<UMaterialInterface> PostProcessMaterial;
};

#undef UE_API
