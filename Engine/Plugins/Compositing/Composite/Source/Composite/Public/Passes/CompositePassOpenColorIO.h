// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CompositePassBase.h"
#include "Passes/CompositeCorePassMergeProxy.h"

#include "OpenColorIOColorSpace.h"

#include "CompositePassOpenColorIO.generated.h"

#define UE_API COMPOSITE_API

/**
 * Applies an OpenColorIO color transform to the layer input.
 * Typically used for color-space conversion or to apply a view transform.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta = (DisplayName = "OpenColorIO"))
class UCompositePassOpenColorIO : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassOpenColorIO(const FObjectInitializer& ObjectInitializer);
	/** Destructor */
	UE_API ~UCompositePassOpenColorIO();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Color conversion settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composite")
	FOpenColorIOColorConversionSettings ColorConversionSettings;

	/**
	 * Set true when the input texture stores premultiplied alpha (RGB already scaled by A).
	 * The pass will unpremultiply before the color transform and re-premultiply after, so
	 * the transform operates on straight-alpha values. Leave false for straight-alpha inputs.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite")
	bool bInputIsPremultiplied;
};

#undef UE_API
