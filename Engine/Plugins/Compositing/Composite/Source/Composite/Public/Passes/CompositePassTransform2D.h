// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"

#include "CompositePassTransform2D.generated.h"

#define UE_API COMPOSITE_API

/** Primary scale calculation mode. */
UENUM(BlueprintType)
enum class ECompositePassScaleMode : uint8
{
	/** Disabled */
	None,

	/** Automatically derive aspect ratio from the parent layer media texture & composite actor camera. */
	Automatic,

	/** Scale based on the difference between container & content aspect ratios. */
	AspectRatio,

	/** Remove black bar padding by specifying pixels to crop on each axis. */
	Unpad,

	/** Manually define the scale factor.*/
	Manual,
};

/**
 * Scales, rotates and translates the layer input in UV space.
 * Supports aspect-ratio fit, padding removal and manual scale modes.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="Transform 2D"))
class UCompositePassTransform2D : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassTransform2D(const FObjectInitializer& ObjectInitializer);

	/** Destructor */
	UE_API ~UCompositePassTransform2D();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

	UE_API virtual bool GetIsActive() const override;

	/** Calculate the final texture UV scale. */
	UFUNCTION(BlueprintCallable, Category = "Composite")
	UE_API FVector2f CalculateScale() const;

public:
	/** Normalized pivot point for rotation and scale. (0.5, 0.5) = viewport center. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite")
	FVector2f Pivot;

	/** How to scale-fit the video content into the camera viewport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite")
	ECompositePassScaleMode ScaleMode;

	/** Container (source) aspect ratio, as width:height (e.g. 16:9). Automatic derives this from the plate texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (EditCondition = "ScaleMode == ECompositePassScaleMode::AspectRatio", EditConditionHides))
	FVector2f ContainerAspectRatio;

	/** Content (target) aspect ratio, typically matching the cine camera, as width:height (e.g. 16:9). Automatic derives this from the camera component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (EditCondition = "ScaleMode == ECompositePassScaleMode::AspectRatio", EditConditionHides))
	FVector2f ContentAspectRatio;

	/** How to resolve aspect ratio mismatch. Scales only the mismatched axis, or both axes equally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (EditCondition = "ScaleMode == ECompositePassScaleMode::Automatic || ScaleMode == ECompositePassScaleMode::AspectRatio", EditConditionHides))
	bool bScaleSingleAxis;

	/** Pixels of black bar padding to crop from each side, per axis. For example, (656, 0) removes 656px of pillarbox from left and right. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (EditCondition = "ScaleMode == ECompositePassScaleMode::Unpad", EditConditionHides, ClampMin = "0", UIMax = "500"))
	FIntPoint UnpadPixels;

	/** Manual scale factor per axis. Values > 1 zoom in, values < 1 zoom out. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (EditCondition = "ScaleMode == ECompositePassScaleMode::Manual", EditConditionHides))
	FVector2f ManualScale;

	/** Automatically remove overscan by reading the camera's overscan value. */
	UPROPERTY(BlueprintReadWrite, Category = "Composite")
	bool bRemoveOverscan;

	/** Counter-clockwise rotation in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite", meta = (UIMin="-360.0", UIMax="360.0", ClampMin="-360.0", ClampMax="360.0"))
	float RotationAngle;

	/** 2D translation offset in normalized viewport coordinates. Y > 0 moves the image up. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Composite")
	FVector2f Translation;

private:
	/** Return the plate texture size from the owning plate layer, or ZeroValue if unavailable. */
	FIntVector2 GetPlateTextureSize() const;

	/** Cached scale UV computed in GetIsActive() and reused in GetProxy(). Reset after consumption. */
	mutable TOptional<FVector2f> CachedScaleUV;
};

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			using namespace CompositeCore;

			/** Render-thread proxy for the 2D transform pass. Applies scale, rotation, and translation around a configurable pivot in UV space. */
			class FTransform2DPassProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FTransform2DPassProxy);

				using FCompositeCorePassProxy::FCompositeCorePassProxy;

				/** Adds the 2D transform RDG pass; bilinear-resamples the input with the full affine transform and returns the transformed output. */
				FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

				/** Normalized pivot point for rotation and scale. */
				FVector2f Pivot = FVector2f(0.5f, 0.5f);

				/** Per-axis UV scale factor. (1, 1) is identity. */
				FVector2f ScaleUV = FVector2f::One();

				/** Rotation in radians. */
				float RotationRadians = 0.0f;

				/** 2D translation in normalized viewport coordinates. */
				FVector2f Translation = FVector2f::ZeroVector;

				/** Viewport aspect ratio (width/height) for rotation correction. */
				float AspectRatio = 1.0f;
			};
		}
	}
}

#undef UE_API
