// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBase.h"
#include "Passes/CompositeCorePassProxy.h"

#include "CompositePassTranslucency.generated.h"

#define UE_API COMPOSITE_API

/** Alpha premultiplication operation type. */
UENUM(BlueprintType)
enum class ECompositeAlphaPremultiplication : uint8
{
	None,
	Premultiply,
	Unpremultiply,
};

UENUM(BlueprintType)
enum class ECompositeAlphaOverride : uint8
{
	None,
	Zero,
	One,
};

/**
 * Modifies the layer alpha channel.
 * Supports alpha-premultiplication toggles, fade and explicit alpha overrides.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, EditInlineNew, CollapseCategories, meta=(DisplayName="Translucency"))
class UCompositePassTranslucency : public UCompositePassBase
{
	GENERATED_BODY()

public:
	/** Constructor */
	UE_API UCompositePassTranslucency(const FObjectInitializer& ObjectInitializer);

	/** Destructor */
	UE_API ~UCompositePassTranslucency();

	UE_API virtual FCompositeCorePassProxy* GetProxy(const UE::CompositeCore::FPassInputDecl& InputDecl, FCompositeTraversalContext& InContext, FSceneRenderingBulkObjectAllocator& InFrameAllocator) const override;

public:
	/** Alpha premultiplication operation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite")
	ECompositeAlphaPremultiplication PremultOp;

	/** Fade the alpha from 0 to 1. Assumes pre-multiplied alpha, so both RGB and A are scaled together. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Composite", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Fade = 1.0f;

	UPROPERTY()
	ECompositeAlphaOverride OverrideOutputAlpha;
};

namespace UE
{
	namespace Composite
	{
		namespace Private
		{
			using namespace CompositeCore;

			/** Render-thread proxy for the translucency pass. Applies alpha premultiplication, fade, and optional alpha override. */
			class FTranslucencyPassProxy : public FCompositeCorePassProxy
			{
			public:
				IMPLEMENT_COMPOSITE_PASS(FTranslucencyPassProxy);

				using FCompositeCorePassProxy::FCompositeCorePassProxy;

				/** Adds the translucency RDG pass and returns the modified output texture. */
				FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

				/** Whether to premultiply, unpremultiply, or leave alpha unchanged before applying Fade. */
				ECompositeAlphaPremultiplication PremultOp = ECompositeAlphaPremultiplication::None;
				/** Uniform multiplier applied to the output (0 = fully transparent, 1 = unchanged). */
				float Fade = 1.0f;
				/** Optionally force the output alpha to 0 or 1 after all other operations. */
				ECompositeAlphaOverride OverrideOutputAlpha = ECompositeAlphaOverride::None;
			};
		}
	}
}

#undef UE_API
