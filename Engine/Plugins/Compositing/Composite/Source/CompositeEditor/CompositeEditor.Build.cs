// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CompositeEditor : ModuleRules
{
	public CompositeEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Composite"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AppFramework",
				"ColorGradingEditor",
				"CompositeCore",
				"ConcertSyncClient",
				"ContentBrowserData",
				"CoreUObject",
				"DetailCustomizations",
				"EditorWidgets",
				"EditorScriptingUtilities",
				"Engine",
				"InputCore",
				"LevelEditor",
				"LevelSequence",
				"MediaAssets",
				"MediaFrameworkUtilities",
				"MediaProfileEditor",
				"MovieScene",
				"MovieSceneTracks",
				"MovieSceneTools",
				"ObjectMixerEditor",
				"Projects",
				"PropertyEditor",
				"UnrealEd",
				"SceneOutliner",
				"Sequencer",
                "Slate",
                "SlateCore",
                "ToolMenus",
                "ToolWidgets",
                "WorkspaceMenuStructure",
			}
		);
	}
}
