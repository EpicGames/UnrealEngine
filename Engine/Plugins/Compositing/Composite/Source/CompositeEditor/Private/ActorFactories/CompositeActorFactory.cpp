// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeActorFactory.h"

#include "CompositeActor.h"
#include "CompositeAnalytics.h"
#include "Layers/CompositeLayerMainRender.h"
#include "Layers/CompositeLayerShadowReflection.h"
#include "Layers/CompositeLayerPlate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeActorFactory)

#define LOCTEXT_NAMESPACE "CompositeActorFactory"

UCompositeActorFactory::UCompositeActorFactory(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	DisplayName = LOCTEXT("DisplayName", "Composite Actor");
	NewActorClass = ACompositeActor::StaticClass();
}

void UCompositeActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	ACompositeActor* CompositeActor = Cast<ACompositeActor>(NewActor);

	// Configure newly spawned composite actor instance (except the transient drag & drop actor).
	if (IsValid(CompositeActor) && !CompositeActor->IsTemplate() && !CompositeActor->HasAnyFlags(RF_Transient))
	{
		// Default layers when creating a new actor
		TArray<TObjectPtr<UCompositeLayerBase>> NewLayers;
		NewLayers.Reserve(3);
		NewLayers.Add(NewObject<UCompositeLayerMainRender>(CompositeActor, TEXT("MainRenderLayer"), RF_Transactional));
		NewLayers.Add(NewObject<UCompositeLayerShadowReflection>(CompositeActor, TEXT("ShadowReflectionLayer"), RF_Transactional));
		NewLayers.Add(NewObject<UCompositeLayerPlate>(CompositeActor, TEXT("PlateLayer"), RF_Transactional));

		// By default, we disable the shadow reflection catcher due to its significant added cost.
		NewLayers[1]->SetIsEnabled(false);

		// No need to call Modify() on the spawned actor
		CompositeActor->SetCompositeLayers(NewLayers);

		// Refresh Composure panel
		FProperty* CompositeLayersProperty = FindFProperty<FProperty>(ACompositeActor::StaticClass(), GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));
		FPropertyChangedEvent LayersChangedEvent(CompositeLayersProperty, EPropertyChangeType::ArrayAdd);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(NewActor, LayersChangedEvent);

		// Record analytics
		Composite::Analytics::RecordActorAdded(*CompositeActor);
	}
}

#undef LOCTEXT_NAMESPACE

