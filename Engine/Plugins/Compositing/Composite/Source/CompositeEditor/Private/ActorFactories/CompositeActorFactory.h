// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"

#include "CompositeActorFactory.generated.h"

UCLASS()
class UCompositeActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	/** Constructor */
	UCompositeActorFactory(const FObjectInitializer& ObjInit);

public:

	//~ Begin UActorFactory interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	//~ End UActorFactory interface
};

