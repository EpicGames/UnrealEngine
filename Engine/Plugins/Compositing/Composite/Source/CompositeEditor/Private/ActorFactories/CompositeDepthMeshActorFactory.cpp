// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeDepthMeshActorFactory.h"

#include "CompositeAnalytics.h"
#include "CompositeDepthMeshActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeDepthMeshActorFactory)

#define LOCTEXT_NAMESPACE "CompositeDepthMeshActorFactory"

UCompositeDepthMeshActorFactory::UCompositeDepthMeshActorFactory(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	DisplayName = LOCTEXT("DisplayName", "Composite Depth Mesh Actor");
	NewActorClass = ACompositeDepthMeshActor::StaticClass();
}

void UCompositeDepthMeshActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	ACompositeDepthMeshActor* DepthMeshActor = Cast<ACompositeDepthMeshActor>(NewActor);

	if (IsValid(DepthMeshActor) && !DepthMeshActor->IsTemplate() && !DepthMeshActor->HasAnyFlags(RF_Transient))
	{
		Composite::Analytics::RecordActorAdded(*DepthMeshActor);
	}
}

#undef LOCTEXT_NAMESPACE
