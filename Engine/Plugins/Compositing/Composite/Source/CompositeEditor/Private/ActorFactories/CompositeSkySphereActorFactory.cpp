// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeSkySphereActorFactory.h"

#include "CompositeAnalytics.h"
#include "CompositeSkySphereActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeSkySphereActorFactory)

#define LOCTEXT_NAMESPACE "CompositeSkySphereActorFactory"

UCompositeSkySphereActorFactory::UCompositeSkySphereActorFactory(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	DisplayName = LOCTEXT("DisplayName", "Composite Sky Sphere Actor");
	NewActorClass = ACompositeSkySphereActor::StaticClass();
}

void UCompositeSkySphereActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	ACompositeSkySphereActor* SkySphereActor = Cast<ACompositeSkySphereActor>(NewActor);

	if (IsValid(SkySphereActor) && !SkySphereActor->IsTemplate() && !SkySphereActor->HasAnyFlags(RF_Transient))
	{
		Composite::Analytics::RecordActorAdded(*SkySphereActor);
	}
}

#undef LOCTEXT_NAMESPACE
