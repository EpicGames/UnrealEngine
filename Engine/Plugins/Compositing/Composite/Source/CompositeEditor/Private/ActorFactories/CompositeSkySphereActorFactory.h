// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"

#include "CompositeSkySphereActorFactory.generated.h"

UCLASS()
class UCompositeSkySphereActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	/** Constructor */
	UCompositeSkySphereActorFactory(const FObjectInitializer& ObjInit);

public:
	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	//~ End UActorFactory Interface
};
