// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingDataModelGenerator_Composite.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Passes/CompositePassColorGrading.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FColorGradingDataModelGenerator_Composite"

namespace UE::Composite::ColorGrading
{
	class FCompositePassColorGradingCustomization : public IDetailCustomization
	{
	public:
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
		{
			TArray<FName> Categories;
			DetailBuilder.GetCategoryNames(Categories);

			// Hide all default categories to avoid duplicate handles and unnecessary customization
			for (const FName& Category : Categories)
			{
				DetailBuilder.HideCategory(Category);
			}
		
			IDetailCategoryBuilder& ColorGradingHeaderCategory = DetailBuilder.EditCategory(TEXT("ColorGradingHeader"));
			ColorGradingHeaderCategory.AddProperty(DetailBuilder.GetProperty(TEXT("bIsEnabled"), UCompositePassBase::StaticClass()));
		
			IDetailCategoryBuilder& ColorGradingElementsCategory = DetailBuilder.EditCategory(TEXT("ColorGradingElements"));

			ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.Global)));
			ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.Shadows)));
			ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.Midtones)));
			ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.Highlights)));

			IDetailCategoryBuilder& TemperatureCategory = DetailBuilder.EditCategory(TEXT("Details_Temperature"), LOCTEXT("TemperatureCategory", "Temperature"));
			TemperatureCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, TemperatureSettings.TemperatureType)));
			TemperatureCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, TemperatureSettings.WhiteTemp)));
			TemperatureCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, TemperatureSettings.WhiteTint)));

			IDetailCategoryBuilder& ColorGradingCategory = DetailBuilder.EditCategory(TEXT("Details_ColorGrading"), LOCTEXT("ColorGradingCategory", "Color Grading"));
			ColorGradingCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.ShadowsMax)));
			ColorGradingCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.HighlightsMin)));
			ColorGradingCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorGrading, ColorGradingSettings.HighlightsMax)));
		}
	};

}

void FColorGradingDataModelGenerator_Composite::Initialize(const TSharedRef<FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(UCompositePassColorGrading::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([]
	{
		return MakeShared<UE::Composite::ColorGrading::FCompositePassColorGradingCustomization>();
	}));
}

void FColorGradingDataModelGenerator_Composite::Destroy(const TSharedRef<FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(UCompositePassColorGrading::StaticClass());
}

void FColorGradingDataModelGenerator_Composite::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel)
{
	TArray<TWeakObjectPtr<UCompositePassColorGrading>> SelectedPasses;
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();

	const bool bCanGenerateDataModel = SelectedObjects.ContainsByPredicate([](const TWeakObjectPtr<UObject>& Object)
	{
		return Object.IsValid() && Object->IsA<UCompositePassColorGrading>();
	});

	if (!bCanGenerateDataModel)
	{
		return;
	}

	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator.GetRootTreeNodes();

	const TSharedRef<IDetailTreeNode>* ColorGradingElementsPtr = RootNodes.FindByPredicate([](const TSharedRef<IDetailTreeNode>& Node)
	{
		return Node->GetNodeName() == TEXT("ColorGradingElements");
	});

	if (ColorGradingElementsPtr)
	{
		const TSharedRef<IDetailTreeNode> ColorGradingElements = *ColorGradingElementsPtr;
		FColorGradingEditorDataModel::FColorGradingGroup ColorGradingGroup;

		ColorGradingGroup.DetailsViewCategories.Append(
		{
			TEXT("Details_Temperature"),
			TEXT("Details_ColorGrading")
		});
		
		TArray<TSharedRef<IDetailTreeNode>> ColorGradingPropertyNodes;
		ColorGradingElements->GetChildren(ColorGradingPropertyNodes);

		for (const TSharedRef<IDetailTreeNode>& PropertyNode : ColorGradingPropertyNodes)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyNode->CreatePropertyHandle();

			FColorGradingEditorDataModel::FColorGradingElement ColorGradingElement = CreateColorGradingElement(PropertyNode, FText::FromName(PropertyNode->GetNodeName()));
			ColorGradingGroup.ColorGradingElements.Add(ColorGradingElement);
		}

		const TSharedRef<IDetailTreeNode>* ColorGradingHeaderPtr = RootNodes.FindByPredicate([](const TSharedRef<IDetailTreeNode>& Node)
		{
			return Node->GetNodeName() == TEXT("ColorGradingHeader");
		});

		if (ColorGradingHeaderPtr)
		{
			const TSharedRef<IDetailTreeNode> ColorGradingHeader = *ColorGradingHeaderPtr;
			
			TArray<TSharedRef<IDetailTreeNode>> ColorGradingHeaderProperties;
			ColorGradingHeader->GetChildren(ColorGradingHeaderProperties);

			if (ColorGradingHeaderProperties.Num() > 0)
			{
				ColorGradingGroup.EditConditionPropertyHandle = ColorGradingHeaderProperties[0]->CreatePropertyHandle();
				ColorGradingGroup.GroupHeaderWidget =
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 4, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("EnabledColorGradingPassLabel", "Enable Color Grading Pass"))
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					];
			}
		}
		
		OutColorGradingDataModel.ColorGradingGroups.Add(ColorGradingGroup);
	}
}

FColorGradingEditorDataModel::FColorGradingElement FColorGradingDataModelGenerator_Composite::CreateColorGradingElement(const TSharedRef<IDetailTreeNode>& GroupNode, FText ElementLabel)
{
	FColorGradingEditorDataModel::FColorGradingElement ColorGradingElement;
	ColorGradingElement.DisplayName = ElementLabel;

	TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
	GroupNode->GetChildren(ChildNodes);

	for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = ChildNode->CreatePropertyHandle();
		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			const FString ColorGradingModeString = PropertyHandle->GetProperty()->GetMetaData(TEXT("ColorGradingMode")).ToLower();

			if (!ColorGradingModeString.IsEmpty())
			{
				if (ColorGradingModeString.Compare(TEXT("saturation")) == 0)
				{
					ColorGradingElement.SaturationPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("contrast")) == 0)
				{
					ColorGradingElement.ContrastPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("gamma")) == 0)
				{
					ColorGradingElement.GammaPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("gain")) == 0)
				{
					ColorGradingElement.GainPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("offset")) == 0)
				{
					ColorGradingElement.OffsetPropertyHandle = PropertyHandle;
				}
			}
		}
	}

	return ColorGradingElement;
}

#undef LOCTEXT_NAMESPACE