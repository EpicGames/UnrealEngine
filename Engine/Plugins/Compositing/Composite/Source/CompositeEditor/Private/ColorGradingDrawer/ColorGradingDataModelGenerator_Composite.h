// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorGradingEditorDataModel.h"

/** Color drawer data model generator for UCompositePassColorGrading objects*/
class FColorGradingDataModelGenerator_Composite : public IColorGradingEditorDataModelGenerator
{
public:
	static TSharedRef<IColorGradingEditorDataModelGenerator> MakeInstance() { return MakeShareable(new FColorGradingDataModelGenerator_Composite());}
	
	//~ IColorGradingDataModelGenerator interface
	virtual void Initialize(const TSharedRef<FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel) override;
	//~ End IColorGradingDataModelGenerator interface

private:
	/** Creates a new color grading element structure for the specified detail tree node, which is expected to have child color properties with the ColorGradingMode metadata set */
	FColorGradingEditorDataModel::FColorGradingElement CreateColorGradingElement(const TSharedRef<IDetailTreeNode>& GroupNode, FText ElementLabel);
};
