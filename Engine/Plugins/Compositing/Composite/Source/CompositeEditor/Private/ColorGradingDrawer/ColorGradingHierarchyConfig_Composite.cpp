// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingHierarchyConfig_Composite.h"

#include "CompositeActor.h"
#include "CompositeEditorStyle.h"
#include "Layers/CompositeLayerBase.h"
#include "Layers/CompositeLayerPlate.h"
#include "ObjectFilter/ObjectMixerEditorActorSubObject.h"
#include "Passes/CompositePassBase.h"
#include "Passes/CompositePassColorGrading.h"
#include "UObject/UnrealType.h"

namespace UE::Composite::Private
{
	/** Collects ColorGrading passes from a named TArray<TObjectPtr<UCompositePassBase>> property on the layer, using reflection so any pass-supporting layer is handled uniformly. */
	static void CollectColorGradingPasses(
		UCompositeLayerBase* Layer,
		FName PropertyName,
		const FSlateBrush* IconBrush,
		TArray<FObjectMixerEditorActorSubObject>& OutSubObjects)
	{
		const FArrayProperty* ArrayProp = FindFProperty<FArrayProperty>(Layer->GetClass(), PropertyName);
		if (!ArrayProp)
		{
			return;
		}

		const FObjectProperty* InnerProp = CastField<FObjectProperty>(ArrayProp->Inner);
		if (!InnerProp || !InnerProp->PropertyClass->IsChildOf(UCompositePassBase::StaticClass()))
		{
			return;
		}

		FScriptArrayHelper Helper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(Layer));
		for (int32 Index = 0; Index < Helper.Num(); ++Index)
		{
			UCompositePassBase* Pass = Cast<UCompositePassBase>(InnerProp->GetObjectPropertyValue(Helper.GetRawPtr(Index)));
			if (IsValid(Pass) && Pass->IsA<UCompositePassColorGrading>())
			{
				FObjectMixerEditorActorSubObject SubObject;
				SubObject.Object = Pass;
				SubObject.DisplayName = FText::FromString(Pass->GetDisplayName());
				SubObject.bIncludeObjectStack = true;
				SubObject.ContainerProperty.PropertyName = PropertyName;
				SubObject.ContainerProperty.PropertyIconOverride = IconBrush;

				OutSubObjects.Add(MoveTemp(SubObject));
			}
		}
	}
}

TArray<FObjectMixerEditorActorSubObject> FColorGradingHierarchyConfig_Composite::GetActorSubObjects(UObject* ParentObject) const
{
	using namespace UE::Composite::Private;

	TArray<FObjectMixerEditorActorSubObject> SubObjects;

	const ACompositeActor* CompositeActor = Cast<ACompositeActor>(ParentObject);
	if (!CompositeActor)
	{
		return SubObjects;
	}

	static const FName LayerPassesName = TEXT("LayerPasses");
	static const FName MediaPassesName = TEXT("MediaPasses");

	for (const TObjectPtr<UCompositeLayerBase>& CompositeLayer : CompositeActor->GetCompositeLayers())
	{
		if (!IsValid(CompositeLayer))
		{
			continue;
		}

		CollectColorGradingPasses(CompositeLayer.Get(), LayerPassesName, FCompositeEditorStyle::Get().GetBrush("CompositeEditor.Passes.Layer"), SubObjects);

		// Plate is the only layer with a separate MediaPasses array.
		if (CompositeLayer->IsA<UCompositeLayerPlate>())
		{
			CollectColorGradingPasses(CompositeLayer.Get(), MediaPassesName, FCompositeEditorStyle::Get().GetBrush("CompositeEditor.Passes.Media"), SubObjects);
		}
	}

	return MoveTemp(SubObjects);
}

TSet<FName> FColorGradingHierarchyConfig_Composite::GetPropertiesThatRequireListRefresh() const
{
	return
	{
		TEXT("MediaPasses"),
		TEXT("LayerPasses"),
		GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers)
	};
}
