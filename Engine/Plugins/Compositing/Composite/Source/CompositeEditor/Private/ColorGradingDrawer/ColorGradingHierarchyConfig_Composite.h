// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorGradingMixerObjectFilterRegistry.h"

/** Color grading mixer hierarchy config for displaying UCompositePassColorGrading objects in the color grading drawer */
struct FColorGradingHierarchyConfig_Composite : IColorGradingMixerObjectHierarchyConfig
{
public:
	static TSharedRef<IColorGradingMixerObjectHierarchyConfig> MakeInstance() { return MakeShared<FColorGradingHierarchyConfig_Composite>(); }

	//~ IColorGradingMixerObjectHierarchyConfig interface
	virtual TArray<FObjectMixerEditorActorSubObject> GetActorSubObjects(UObject* ParentObject) const override;
	virtual TSet<FName> GetPropertiesThatRequireListRefresh() const override;
	//~ End IColorGradingMixerObjectHierarchyConfig interface
};
