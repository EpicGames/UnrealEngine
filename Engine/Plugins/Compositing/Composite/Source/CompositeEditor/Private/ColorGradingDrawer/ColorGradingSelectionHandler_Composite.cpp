// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingSelectionHandler_Composite.h"

#include "CompositeActor.h"
#include "Passes/CompositePassColorGrading.h"
#include "UI/SCompositeEditorPanel.h"

TSharedRef<IColorGradingMixerSelectionHandler> FColorGradingSelectionHandler_Composite::MakeInstance()
{
	TSharedRef<FColorGradingSelectionHandler_Composite> SelectionHandler = MakeShared<FColorGradingSelectionHandler_Composite>();

	SCompositeEditorPanel::GetOnSelectionChanged().AddSP(SelectionHandler, &FColorGradingSelectionHandler_Composite::OnComposurePanelSelectionChanged);

	return SelectionHandler;
}

bool FColorGradingSelectionHandler_Composite::CanSelectSubObject(const UObject* InObject)
{
	return InObject->IsA<ACompositeActor>() || InObject->IsA<UCompositeLayerBase>() || InObject->IsA<UCompositePassBase>();
}

void FColorGradingSelectionHandler_Composite::SelectSubObjects(const TArray<UObject*>& InSelectedObjects, bool bShouldSelect, bool bSelectEvenIfHidden) const
{
	if (bShouldSelect)
	{
		SCompositeEditorPanel::UpdateActivePanelSelection(InSelectedObjects);
	}
	else
	{
		SCompositeEditorPanel::UpdateActivePanelSelection({ });
	}
}

TArray<UObject*> FColorGradingSelectionHandler_Composite::GetSelectedSubObjectsInEditor() const
{
	TArray<UObject*> SelectedObjects = SCompositeEditorPanel::GetActivePanelSelection();

	// Remove any non-color grading objects from the selection
	SelectedObjects.RemoveAll([](const UObject* Object)
	{
		return !Object->IsA<UCompositePassColorGrading>();
	});

	return SelectedObjects;
}

void FColorGradingSelectionHandler_Composite::OnComposurePanelSelectionChanged()
{
	OnSelectionChanged.ExecuteIfBound();
}
