// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorGradingMixerObjectFilterRegistry.h"

/** Color grading selection handler that synchronizes selection between the color grading drawer and the Composure panel */
struct FColorGradingSelectionHandler_Composite : IColorGradingMixerSelectionHandler
{
public:
	static TSharedRef<IColorGradingMixerSelectionHandler> MakeInstance();

	//~ IColorGradingMixerSelectionHandler interface
	virtual  bool CanSelectSubObject(const UObject* InObject) override;
	virtual void SelectSubObjects(const TArray<UObject*>& InSelectedObjects, bool bShouldSelect, bool bSelectEvenIfHidden) const override;
	virtual TArray<UObject*> GetSelectedSubObjectsInEditor() const override;
	//~ End IColorGradingMixerSelectionHandler interface
	
private:
	void OnComposurePanelSelectionChanged();
};
