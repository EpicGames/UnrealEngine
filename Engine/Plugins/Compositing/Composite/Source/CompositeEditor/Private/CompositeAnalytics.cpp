// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeAnalytics.h"

#include "EngineAnalytics.h"
#include "GameFramework/Actor.h"
#include "Layers/CompositeLayerBase.h"
#include "Passes/CompositePassBase.h"

void Composite::Analytics::RecordActorAdded(const AActor& Actor)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attrs;
	Attrs.Add(FAnalyticsEventAttribute(TEXT("ActorClass"), Actor.GetClass()->GetName()));
	FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Composite.ActorAdded"), Attrs);
}

void Composite::Analytics::RecordLayerAdded(const UCompositeLayerBase& Layer)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attrs;
	Attrs.Add(FAnalyticsEventAttribute(TEXT("LayerClass"), Layer.GetClass()->GetName()));
	FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Composite.LayerAdded"), Attrs);
}

void Composite::Analytics::RecordPassAdded(const UCompositePassBase& Pass)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attrs;
	Attrs.Add(FAnalyticsEventAttribute(TEXT("PassClass"), Pass.GetClass()->GetName()));
	FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Composite.PassAdded"), Attrs);
}
