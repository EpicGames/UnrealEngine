// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class AActor;
class UCompositeLayerBase;
class UCompositePassBase;

namespace Composite::Analytics
{
	/** Records Editor.Usage.Composite.ActorAdded with the actor's class name. */
	void RecordActorAdded(const AActor& Actor);

	/** Records Editor.Usage.Composite.LayerAdded with the layer's class name. */
	void RecordLayerAdded(const UCompositeLayerBase& Layer);

	/** Records Editor.Usage.Composite.PassAdded with the pass's class name. */
	void RecordPassAdded(const UCompositePassBase& Pass);
}
