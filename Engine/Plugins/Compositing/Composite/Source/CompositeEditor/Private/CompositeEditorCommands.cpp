// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeEditorCommands.h"

#include "Framework/Commands/Commands.h"

#define LOCTEXT_NAMESPACE "FCompositeEditorCommands"

void FCompositeEditorCommands::RegisterCommands()
{
	UI_COMMAND(Enable, "Toggle Enable", "Toggle Enable State", EUserInterfaceActionType::Button, FInputChord(EModifierKey::None, EKeys::D));
	UI_COMMAND(RemoveActor, "Remove Actor", "Removes selected actor(s) from the list", EUserInterfaceActionType::Button, FInputChord(EModifierKey::None, EKeys::Delete));
}

#undef LOCTEXT_NAMESPACE
