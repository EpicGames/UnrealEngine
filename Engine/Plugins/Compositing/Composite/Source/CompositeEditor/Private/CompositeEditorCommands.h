// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

/**
 * UI commands for actions that are specific to Composite editor
 */
class FCompositeEditorCommands : public TCommands<FCompositeEditorCommands>
{
public:
	FCompositeEditorCommands()
		: TCommands<FCompositeEditorCommands>(
			"CompositeEditor",
			NSLOCTEXT("FCompositeEditorCommands", "CompositeEditor", "Composite Editor"),
			NAME_None, FAppStyle::GetAppStyleSetName()
		)
	{ }

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> Enable;
	TSharedPtr<FUICommandInfo> RemoveActor;
};
