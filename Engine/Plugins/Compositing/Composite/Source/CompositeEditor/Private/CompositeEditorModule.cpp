// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeEditorModule.h"

#include "IConcertClientTransactionBridge.h"
#include "IConcertSyncClientModule.h"
#include "ISequencerModule.h"
#include "Sequencer/CompositeActorObjectSchema.h"
#include "Util/CompositeSequencerAutoKeySuppression.h"

#include "ColorGradingEditorDataModel.h"
#include "ColorGradingMixerObjectFilterRegistry.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Filters/CustomClassFilterData.h"
#include "HoldoutCompositeComponent.h"
#include "Framework/Notifications/NotificationManager.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Components/CompositeDepthMeshComponent.h"
#include "Components/CompositeViewProjectionComponent.h"
#include "CompositeActor.h"
#include "CompositeSkySphereActor.h"
#include "CompositeDepthMeshActor.h"
#include "CompositeMeshActor.h"
#include "CompositeEditorCommands.h"
#include "CompositeEditorStyle.h"
#include "PropertyEditorModule.h"
#include "ColorGradingDrawer/ColorGradingDataModelGenerator_Composite.h"
#include "ColorGradingDrawer/ColorGradingHierarchyConfig_Composite.h"
#include "ColorGradingDrawer/ColorGradingSelectionHandler_Composite.h"
#include "Customizations/CompositeActorCustomization.h"
#include "Customizations/CompositeViewProjectionComponentCustomization.h"
#include "Customizations/CompositeLayerPlateCustomization.h"
#include "Customizations/CompositeLayerSimplePassesCustomization.h"
#include "Customizations/CompositeLayerSceneCaptureCustomization.h"
#include "Customizations/CompositeLayerPlanarReflectionCustomization.h"
#include "Customizations/CompositeLayerShadowReflectionCustomization.h"
#include "Customizations/CompositeLayerSingleLightShadowCustomization.h"
#include "Customizations/CompositeDepthMeshComponentCustomization.h"
#include "Customizations/CompositePassColorKeyerCustomization.h"
#include "Customizations/CompositePassMaskingCustomization.h"
#include "Customizations/CompositeSkySphereActorCustomization.h"
#include "Layers/CompositeLayerMainRender.h"
#include "Layers/CompositeLayerPlate.h"
#include "Layers/CompositeLayerProcessing.h"
#include "Layers/CompositeLayerPlanarReflection.h"
#include "Layers/CompositeLayerSceneCapture.h"
#include "Layers/CompositeLayerShadowReflection.h"
#include "Layers/CompositeLayerSingleLightShadow.h"
#include "Passes/CompositePassColorGrading.h"
#include "Passes/CompositePassColorKeyer.h"
#include "Passes/CompositePassMasking.h"
#include "UI/SCompositeEditorPanel.h"

#define LOCTEXT_NAMESPACE "CompositeEditorModule"

DEFINE_LOG_CATEGORY(LogCompositeEditor);

FCompositeEditorModule::FCompositeEditorModule() = default;
FCompositeEditorModule::~FCompositeEditorModule() = default;

void FCompositeEditorModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCompositeEditorModule::OnPostEngineInit);

	UHoldoutCompositeComponent::OnComponentCreatedDelegate.AddRaw(this, &FCompositeEditorModule::TriggerHoldoutCompositeWarning);

	FCompositeEditorCommands::Register();
	FCompositeEditorStyle::Register();
	
	RegisterTabSpawners();
	RegisterCustomizations();

	FColorGradingEditorDataModel::RegisterColorGradingDataModelGenerator<UCompositePassColorGrading>(
		FGetDetailsDataModelGenerator::CreateStatic(&FColorGradingDataModelGenerator_Composite::MakeInstance));
	
	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(ACompositeActor::StaticClass());

	FColorGradingMixerObjectFilterRegistry::RegisterObjectClassToFilter(
		ACompositeActor::StaticClass(),
		FGetObjectHierarchyConfig::CreateStatic(&FColorGradingHierarchyConfig_Composite::MakeInstance)
	);

	FColorGradingMixerObjectFilterRegistry::RegisterSelectionHandler(FGetSelectionHandler::CreateStatic(&FColorGradingSelectionHandler_Composite::MakeInstance));

	// Register Sequencer object schema for layer/pass sub-object discovery.
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	CompositeActorObjectSchema = MakeShared<UE::Sequencer::FCompositeActorObjectSchema>();
	SequencerModule.RegisterObjectSchema(CompositeActorObjectSchema);
}

void FCompositeEditorModule::ShutdownModule()
{
	UnregisterConcertApplyListener();

	if (CompositeActorObjectSchema)
	{
		if (ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer"))
		{
			SequencerModule->UnregisterObjectSchema(CompositeActorObjectSchema);
		}
		CompositeActorObjectSchema.Reset();
	}

	UnregisterCustomizations();
	UnregisterTabSpawners();

	FCompositeEditorCommands::Unregister();
	FCompositeEditorStyle::Unregister();
	
	if (GEditor)
	{
		GEditor->OnLevelActorAdded().RemoveAll(this);
	}

	UHoldoutCompositeComponent::OnComponentCreatedDelegate.RemoveAll(this);
	
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FCompositeEditorModule::OnPostEngineInit()
{
	// Subscribe here so ConcertSyncClient is reliably loaded (both modules load at Default phase, undefined order).
	RegisterConcertApplyListener();

	if (GEditor)
	{
		GEditor->OnLevelActorAdded().AddRaw(this, &FCompositeEditorModule::OnLevelActorAdded);
	}

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		const TSharedRef<FFilterCategory> ComposureFilterCategory = MakeShared<FFilterCategory>(
			LOCTEXT("ComposureFilterCategory", "Composure"), FText::GetEmpty());

		UClass* const CompositeActorClasses[] = {
			ACompositeActor::StaticClass(),
			ACompositeSkySphereActor::StaticClass(),
			ACompositeMeshActor::StaticClass(),
			ACompositeDepthMeshActor::StaticClass(),
		};

		for (UClass* ActorClass : CompositeActorClasses)
		{
			LevelEditorModule->AddCustomClassFilterToOutliner(
				MakeShared<FCustomClassFilterData>(ActorClass, ComposureFilterCategory, FLinearColor::White));
		}
	}
}

void FCompositeEditorModule::OnLevelActorAdded(AActor* InActor)
{
	if (!InActor || InActor->HasAnyFlags(RF_Transient) || !InActor->IsA<ACompositeActor>())
	{
		return;
	}

	UWorld* World = InActor->GetWorld();
	if (!IsValid(World))
	{
		return;
	}

	int32 CompositeActorCount = 0;
	for (TActorIterator<ACompositeActor> It(World); It; ++It)
	{
		CompositeActorCount++;

		if (CompositeActorCount > 1)
		{
			ACompositeActor* CompositeActor = Cast<ACompositeActor>(InActor);
			if (IsValid(CompositeActor))
			{
				CompositeActor->SetIsActive(false);

				UE_LOGF(LogCompositeEditor, Display, "No more than one composite actor should be active at the same time.");
			}
			break;
		}
	}
}

void FCompositeEditorModule::TriggerHoldoutCompositeWarning(const UHoldoutCompositeComponent* InComponent)
{
	static bool bWarnOnce = true;

	if (bWarnOnce)
	{
		int32 CompositeActorNum = 0;
		UWorld* World = nullptr;
		if (GEditor)
		{
			World = GEditor->GetEditorWorldContext().World();
		}

		if (IsValid(World))
		{
			for (TActorIterator<ACompositeActor> It(World); It; ++It)
			{
				ACompositeActor* CompositeActor = *It;
				if (IsValid(CompositeActor))
				{
					CompositeActorNum++;
				}
			}
		}

		if (CompositeActorNum > 0)
		{
			FNotificationInfo Info(LOCTEXT("FCompositeEditorModuleHoldoutCompositeCreated", "Holdout composite components are not designed to be used with the new Composite Actor. Prefer registering meshes on a plate layer directly."));
			Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
			Info.bFireAndForget = true;
			Info.FadeOutDuration = 4.0f;
			Info.ExpireDuration = 8.0f;
			FSlateNotificationManager::Get().AddNotification(Info);

			bWarnOnce = false;
		}
	}
}

void FCompositeEditorModule::RegisterTabSpawners()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	OnTabManagerChangedDelegateHandle = LevelEditorModule.OnTabManagerChanged().AddLambda([this]()
	{
		SCompositeEditorPanel::RegisterTabSpawner();
	});
}

void FCompositeEditorModule::UnregisterTabSpawners()
{
	SCompositeEditorPanel::UnregisterTabSpawner();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnTabManagerChanged().Remove(OnTabManagerChangedDelegateHandle);
}

void FCompositeEditorModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(
		UCompositeLayerPlate::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerPlateCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositeLayerShadowReflection::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerShadowReflectionCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
			UCompositeLayerSceneCapture::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerSceneCaptureCustomization::MakeInstance)
		);

	PropertyModule.RegisterCustomClassLayout(
			UCompositeLayerSingleLightShadow::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerSingleLightShadowCustomization::MakeInstance)
		);

	PropertyModule.RegisterCustomClassLayout(
		UCompositeLayerPlanarReflection::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerPlanarReflectionCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositeLayerMainRender::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerSimplePassesCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositeLayerProcessing::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeLayerSimplePassesCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		ACompositeActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeActorCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositeViewProjectionComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeViewProjectionComponentCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositeDepthMeshComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeDepthMeshComponentCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositePassColorKeyer::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositePassColorKeyerCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositePassMasking::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositePassMaskingCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		UCompositePassUltimatteMasking::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositePassMaskingCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		ACompositeSkySphereActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositeSkySphereActorCustomization::MakeInstance)
	);
}

void FCompositeEditorModule::UnregisterCustomizations()
{
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->UnregisterCustomClassLayout(ACompositeActor::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerPlate::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerShadowReflection::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerSceneCapture::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerSingleLightShadow::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerPlanarReflection::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerMainRender::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeLayerProcessing::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeViewProjectionComponent::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositeDepthMeshComponent::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositePassColorKeyer::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositePassMasking::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UCompositePassUltimatteMasking::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(ACompositeSkySphereActor::StaticClass()->GetFName());
		}
	}
}

void FCompositeEditorModule::RegisterConcertApplyListener()
{
	if (IConcertSyncClientModule::IsAvailable())
	{
		IConcertSyncClientModule::Get().GetTransactionBridge().OnApplyTransaction().AddRaw(this, &FCompositeEditorModule::OnConcertApplyTransaction);
	}
}

void FCompositeEditorModule::UnregisterConcertApplyListener()
{
	if (IConcertSyncClientModule::IsAvailable())
	{
		IConcertSyncClientModule::Get().GetTransactionBridge().OnApplyTransaction().RemoveAll(this);
	}
	ConcertApplyAutoKeySuppression.Reset();
}

void FCompositeEditorModule::OnConcertApplyTransaction(ETransactionNotification InNotification, const bool /*bIsSnapshot*/)
{
	// Hold the suppression guard across the apply window (both snapshot and final) so Sequencer's auto-key path bails on IsAllowedToChange().
	if (InNotification == ETransactionNotification::Begin)
	{
		ensureMsgf(!ConcertApplyAutoKeySuppression.IsValid(), TEXT("Concert apply Begin received while a suppression guard is already held; the prior End was missed."));
		ConcertApplyAutoKeySuppression = MakeUnique<FScopedSequencerAutoKeySuppression>();
	}
	else if (InNotification == ETransactionNotification::End)
	{
		ConcertApplyAutoKeySuppression.Reset();
	}
}

IMPLEMENT_MODULE(FCompositeEditorModule, CompositeEditor)

#undef LOCTEXT_NAMESPACE
