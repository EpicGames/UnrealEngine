// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FCompositeEditorStyle& FCompositeEditorStyle::Get()
{
	static FCompositeEditorStyle Instance;
	return Instance;
}

void FCompositeEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FCompositeEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FCompositeEditorStyle::FCompositeEditorStyle()
	: FSlateStyleSet("CompositeEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("Composite"))->GetContentDir();
	FSlateStyleSet::SetContentRoot(ContentDir);
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/"));
	
	Set("CompositeEditor.Composure", new CORE_IMAGE_BRUSH_SVG("Starship/Common/ComposureCompositing", Icon16x16));

	Set("ClassIcon.CompositeActor", new CORE_IMAGE_BRUSH_SVG("Starship/Common/ComposureCompositing", Icon16x16));
	Set("ClassIcon.CompositeSkySphereActor", new IMAGE_BRUSH_SVG("Editor/Icons/CompositeSkySphere_16", Icon16x16));
	Set("ClassIcon.CompositeMeshActor", new IMAGE_BRUSH_SVG("Editor/Icons/CompositeMesh_16", Icon16x16));
	Set("ClassIcon.CompositeDepthMeshActor", new IMAGE_BRUSH_SVG("Editor/Icons/CompositeDepthMesh_16", Icon16x16));
	Set("ClassIcon.CompositeLayerMainRender", new IMAGE_BRUSH_SVG("Editor/Icons/MainRender", Icon16x16));
	Set("ClassIcon.CompositeLayerShadowReflection", new IMAGE_BRUSH_SVG("Editor/Icons/ShadowReflectionCatcher_16", Icon16x16));
	Set("ClassIcon.CompositeLayerSingleLightShadow", new IMAGE_BRUSH_SVG("Editor/Icons/Shadow", Icon16x16));
	Set("ClassIcon.CompositeLayerSceneCapture", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/SceneCapture2D_16", Icon16x16));
	Set("ClassIcon.CompositeLayerPlate", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/MediaPlayer_16", Icon16x16));
	Set("ClassIcon.CompositeLayerPlanarReflection", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Reflections", Icon16x16));
	Set("ClassIcon.CompositePassColorGrading", new CORE_IMAGE_BRUSH_SVG("Starship/TimelineEditor/TrackTypeColor", Icon16x16));
	Set("ClassIcon.CompositePassColorKeyer", new IMAGE_BRUSH_SVG("Editor/Icons/Chromakey", Icon16x16));
	Set("ClassIcon.CompositePassFXAA", new CORE_IMAGE_BRUSH_SVG("Starship/Common/AntiAliasing", Icon16x16));
	Set("ClassIcon.CompositePassSMAA", new CORE_IMAGE_BRUSH_SVG("Starship/Common/AntiAliasing", Icon16x16));
	Set("ClassIcon.CompositePassOpenColorIO", new IMAGE_BRUSH_SVG("Editor/Icons/OCIO", Icon16x16));
	Set("ClassIcon.CompositePassMaterial", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/Material_16", Icon16x16));
	Set("ClassIcon.CompositePassBlur", new IMAGE_BRUSH_SVG("Editor/Icons/BlurPass_16", Icon16x16));
	Set("ClassIcon.CompositePassMasking", new IMAGE_BRUSH_SVG("Editor/Icons/MaskingPass_16", Icon16x16));
	Set("ClassIcon.CompositePassUltimatteMasking", new IMAGE_BRUSH_SVG("Editor/Icons/MaskingPass_16", Icon16x16));
	Set("ClassIcon.CompositePassLumaKeyer", new IMAGE_BRUSH_SVG("Editor/Icons/Chromakey", Icon16x16));
	Set("ClassIcon.CompositePassTransform2D", new CORE_IMAGE_BRUSH_SVG("Starship/EditorViewport/translate", Icon16x16));
	Set("ClassIcon.CompositePassTranslucency", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Transparency", Icon16x16));
	
    Set("CompositeEditor.Passes.Media", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/MediaPlayer_16", Icon16x16));
	Set("CompositeEditor.Passes.Scene", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Levels", Icon16x16));
	Set("CompositeEditor.Passes.Layer", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Layers", Icon16x16));

	Set("CompositeEditor.MediaProfile", new IMAGE_BRUSH_SVG("Editor/Icons/MediaProfile", Icon16x16));
	Set("CompositeEditor.CaptureCleanPlate", new IMAGE_BRUSH_SVG("Editor/Icons/TextureCreate_16", Icon16x16));
	Set("CompositeEditor.CancelCapture", new IMAGE_BRUSH_SVG("Editor/Icons/TextureCancel_16", Icon16x16));
	Set("CompositeEditor.CompositeMeshVisualization", new IMAGE_BRUSH_SVG("Editor/Icons/CompositeMeshVisualization_16", Icon16x16));
	
	Set("CompositeEditor.LitMaterial", new CORE_IMAGE_BRUSH_SVG("Starship/Common/LitCube", Icon16x16));
	Set("CompositeEditor.UnlitMaterial", new CORE_IMAGE_BRUSH_SVG("Starship/Common/UnlitCube", Icon16x16));
	Set("CompositeEditor.CustomMaterial", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/Material_16", Icon16x16));
	Set("CompositeEditor.SyncSelection", new CORE_IMAGE_BRUSH("Icons/GeneralTools/SelectAll_40x", Icon16x16));
}
