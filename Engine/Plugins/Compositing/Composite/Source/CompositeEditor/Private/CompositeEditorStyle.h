// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/SlateStyle.h"

class FCompositeEditorStyle	final : public FSlateStyleSet
{
public:
	static FCompositeEditorStyle& Get();

	static void Register();
	static void Unregister();
	
private:
	FCompositeEditorStyle();
};
