// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeActorCustomization.h"

#include "CompositeActor.h"
#include "CompositeCustomizationHelpers.h"
#include "CompositeEditorStyle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "LevelEditor.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleManager.h"
#include "UI/SCompositeEditorPanel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FCompositeActorCustomization"

TSharedRef<IDetailCustomization> FCompositeActorCustomization::MakeInstance()
{
	return MakeShared<FCompositeActorCustomization>();
}

void FCompositeActorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	CompositeActors.Reserve(Objects.Num());
	for (TWeakObjectPtr<UObject>& Obj : Objects)
	{
		TWeakObjectPtr<ACompositeActor> CompositeActor = Cast<ACompositeActor>(Obj.Get());
		if (CompositeActor.IsValid())
		{
			CompositeActors.Add(CompositeActor);
		}
	}
	
	DetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(ACompositeActor, bIsActive));

	DetailLayout.EditCategory("Composite")
	.AddCustomRow(FText::GetEmpty())
	.WholeRowContent()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.OnClicked(this, &FCompositeActorCustomization::OpenComposurePanel)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
						
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0, 1))
			[
				SNew(SImage)
				.Image(FCompositeEditorStyle::Get().GetBrush("CompositeEditor.Composure"))
			]
			
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(2, 0, 0, 0))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OpenComposureButtonLabel", "Open Composure"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];

	TSharedPtr<IPropertyHandle> CameraActorHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositeActor, CameraActor));
	CompositeCustomizationHelpers::CustomizeCameraActorProperty(DetailLayout, CameraActorHandle);
}

FReply FCompositeActorCustomization::OpenComposurePanel()
{
	const FName ComposurePanelTabName = "CompositeEditorTab";
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
	{
		if (TSharedPtr<SDockTab> Tab = LevelEditorTabManager->TryInvokeTab(ComposurePanelTabName))
		{
			if (TSharedPtr<SCompositeEditorPanel> CompositePanel = StaticCastSharedPtr<SCompositeEditorPanel>(Tab->GetContent().ToSharedPtr()))
			{
				CompositePanel->SelectCompositeActors(CompositeActors);
			}
		}
	}
	
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
