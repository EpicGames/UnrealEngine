// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"

class ACompositeActor;

/**
 * Customization for the composite actor.
 */
class FCompositeActorCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of the details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	/** Raised when the 'Open Composure' button is clicked, and opens the Composure panel  */
	FReply OpenComposurePanel();

private:
	/** List of composite actors being displayed by this customization */
	TArray<TWeakObjectPtr<ACompositeActor>> CompositeActors;
};
