// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCustomizationHelpers.h"

#include "AssetRegistry/AssetData.h"
#include "Camera/CameraComponent.h"
#include "CompositeActor.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorUtils.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Util/CompositeSequencerAutoKeySuppression.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CompositeCustomizationHelpers"

namespace CompositeCustomizationHelpers
{

// Clears any stale spawnable binding on the composite actors behind the handle. Needed when
// SetValue(FAssetData()) is a no-op (CameraActor already null) and PostEditChangeChainProperty
// therefore doesn't run.
static void ClearPendingSpawnableBinding(TSharedPtr<IPropertyHandle> CameraActorHandle)
{
	TArray<UObject*> OuterObjects;
	CameraActorHandle->GetOuterObjects(OuterObjects);
	for (UObject* Outer : OuterObjects)
	{
		if (ACompositeActor* CompositeActor = Cast<ACompositeActor>(Outer))
		{
			if (CompositeActor->HasSpawnableBinding())
			{
				CompositeActor->Modify();
				CompositeActor->DetectAndStoreSpawnableBinding(nullptr);
			}
		}
	}
}

bool ShouldFilterAssetByAllowedClasses(const FAssetData& AssetData, const TSharedRef<IPropertyHandle>& PropertyHandle)
{
	FProperty* Property = PropertyHandle->GetProperty();
	if (!Property)
	{
		return false;
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	TArray<const UClass*> AllowedClassFilters;
	TArray<const UClass*> DisallowedClassFilters;
	PropertyEditorUtils::GetAllowedAndDisallowedClasses(OuterObjects, *Property, AllowedClassFilters, DisallowedClassFilters, /*bExactClass=*/ false);

	if (AllowedClassFilters.IsEmpty() && DisallowedClassFilters.IsEmpty())
	{
		return false;
	}

	UClass* AssetClass = AssetData.GetClass();
	if (!AssetClass)
	{
		return true;
	}

	for (const UClass* DisallowedClass : DisallowedClassFilters)
	{
		if (DisallowedClass && AssetClass->IsChildOf(DisallowedClass))
		{
			return true;
		}
	}

	if (AllowedClassFilters.IsEmpty())
	{
		return false;
	}

	for (const UClass* AllowedClass : AllowedClassFilters)
	{
		if (AllowedClass && AssetClass->IsChildOf(AllowedClass))
		{
			return false;
		}
	}

	return true;
}

void CustomizeCameraActorProperty(IDetailLayoutBuilder& DetailLayout, TSharedPtr<IPropertyHandle> CameraActorHandle)
{
	if (!CameraActorHandle.IsValid() || !CameraActorHandle->IsValidHandle())
	{
		return;
	}

	auto GetCurrentActor = [CameraActorHandle]() -> AActor*
	{
		UObject* Object = nullptr;
		if (CameraActorHandle->GetValue(Object) == FPropertyAccess::Success)
		{
			return Cast<AActor>(Object);
		}
		return nullptr;
	};

	auto CameraFilter = FOnShouldFilterActor::CreateLambda([](const AActor* InActor)
	{
		// In FSceneOutlinerFilter, returning true means the item passes the filter (is shown).
		return InActor && InActor->GetComponentByClass<UCameraComponent>();
	});

	TSharedPtr<SComboButton> PickerCombo;

	IDetailPropertyRow* PropertyRow = DetailLayout.EditDefaultProperty(CameraActorHandle);

	// Reset-to-default needs custom visibility so the button shows when CameraActor matches default but the binding doesn't.
	PropertyRow->OverrideResetToDefault(FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateLambda([CameraActorHandle](TSharedPtr<IPropertyHandle>) -> bool
		{
			if (!CameraActorHandle.IsValid() || !CameraActorHandle->IsValidHandle())
			{
				return false;
			}

			TArray<UObject*> OuterObjects;
			CameraActorHandle->GetOuterObjects(OuterObjects);
			for (UObject* Outer : OuterObjects)
			{
				if (ACompositeActor* CompositeActor = Cast<ACompositeActor>(Outer))
				{
					if (!CompositeActor->GetCameraActor().IsNull() || CompositeActor->HasSpawnableBinding())
					{
						return true;
					}
				}
			}
			return false;
		}),
		FResetToDefaultHandler::CreateLambda([CameraActorHandle](TSharedPtr<IPropertyHandle>)
		{
			if (!CameraActorHandle.IsValid() || !CameraActorHandle->IsValidHandle())
			{
				return;
			}

			FScopedTransaction Transaction(LOCTEXT("ResetCompositeCamera", "Reset Composite Camera"));
			{
				FScopedSequencerAutoKeySuppression SequencerSuppress;
				CameraActorHandle->SetValue(FAssetData());
			}
			ClearPendingSpawnableBinding(CameraActorHandle);
		})
	));

	PropertyRow->CustomWidget()
		.NameContent()
		[
			CameraActorHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		[
			SAssignNew(PickerCombo, SComboButton)
			.ContentPadding(FMargin(4.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda([GetCurrentActor, CameraActorHandle]()
				{
					if (AActor* Actor = GetCurrentActor())
					{
						return FText::FromString(Actor->GetActorLabel());
					}
					// Show a hint when a spawnable binding exists but the actor isn't spawned yet.
					TArray<UObject*> OuterObjects;
					CameraActorHandle->GetOuterObjects(OuterObjects);
					for (UObject* Outer : OuterObjects)
					{
						if (ACompositeActor* CompositeActor = Cast<ACompositeActor>(Outer))
						{
							if (CompositeActor->HasSpawnableBinding())
							{
								return LOCTEXT("SpawnableRef", "[Pending Spawnable]");
							}
						}
					}
					return LOCTEXT("None", "None");
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	PickerCombo->SetOnGetMenuContent(FOnGetContent::CreateLambda(
		[CameraActorHandle, GetCurrentActor, CameraFilter, WeakPickerCombo = PickerCombo.ToWeakPtr()]() -> TSharedRef<SWidget>
		{
			return PropertyCustomizationHelpers::MakeActorPickerWithMenu(
				GetCurrentActor(),
				/*AllowClear=*/ true,
				/*AllowPickingLevelInstanceContent=*/ false,
				CameraFilter,
				FOnActorSelected::CreateLambda([CameraActorHandle](AActor* InActor)
				{
					if (!CameraActorHandle.IsValid() || !CameraActorHandle->IsValidHandle())
					{
						return;
					}

					// Wrap in an outer transaction so the whole pick is one undoable unit.
					FScopedTransaction Transaction(LOCTEXT("SetCompositeCamera", "Set Composite Camera"));
					{
						FScopedSequencerAutoKeySuppression SequencerSuppress;
						CameraActorHandle->SetValue(FAssetData(InActor));
					}
					if (!InActor)
					{
						ClearPendingSpawnableBinding(CameraActorHandle);
					}
				}),
				FSimpleDelegate::CreateLambda([WeakPickerCombo]()
				{
					if (TSharedPtr<SComboButton> Combo = WeakPickerCombo.Pin())
					{
						Combo->SetIsOpen(false);
					}
				}),
				FSimpleDelegate(), // OnUseSelected
				/*bDisplayUseSelected=*/ false,
				/*bShowTransient=*/ true
			);
		}
	));
}

} // namespace CompositeCustomizationHelpers

#undef LOCTEXT_NAMESPACE
