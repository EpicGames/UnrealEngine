// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
struct FAssetData;

namespace CompositeCustomizationHelpers
{

/**
 * Customize a CameraActor soft object property to show an actor picker filtered to actors with a UCameraComponent.
 * Shows transient actors (e.g. Sequencer spawnables) in the picker.
 */
void CustomizeCameraActorProperty(IDetailLayoutBuilder& DetailLayout, TSharedPtr<IPropertyHandle> CameraActorHandle);

/**
 * Asset filter predicate that consults the property's AllowedClasses / DisallowedClasses metadata.
 * Returns true (filter the asset OUT) when the asset's class is not a child of any AllowedClasses entry,
 * or when it is a child of a DisallowedClasses entry. Returns false (keep the asset) when the metadata
 * is empty or absent. Use as the OnShouldFilterAsset predicate for asset pickers so the metadata on the
 * UPROPERTY remains the single source of truth.
 */
bool ShouldFilterAssetByAllowedClasses(const FAssetData& AssetData, const TSharedRef<IPropertyHandle>& PropertyHandle);

} // namespace CompositeCustomizationHelpers
