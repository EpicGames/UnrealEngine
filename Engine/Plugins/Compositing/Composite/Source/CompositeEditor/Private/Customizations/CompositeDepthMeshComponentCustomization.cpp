// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeDepthMeshComponentCustomization.h"

#include "CompositeCustomizationHelpers.h"
#include "Components/CompositeDepthMeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Texture.h"
#include "PropertyHandle.h"
#include "SMediaProfileSourceTexturePicker.h"

#define LOCTEXT_NAMESPACE "FCompositeDepthMeshComponentCustomization"

TSharedRef<IDetailCustomization> FCompositeDepthMeshComponentCustomization::MakeInstance()
{
	return MakeShared<FCompositeDepthMeshComponentCustomization>();
}

void FCompositeDepthMeshComponentCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	IDetailCustomization::CustomizeDetails(DetailBuilder);
}

void FCompositeDepthMeshComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Promote "Composite" category
	IDetailCategoryBuilder& CompositeCategory = DetailBuilder.EditCategory("Composite", FText::GetEmpty(), ECategoryPriority::Important);

	TSharedRef<IPropertyHandle> DepthTextureHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositeDepthMeshComponent, DepthTexture));
	DetailBuilder.HideProperty(DepthTextureHandle);

	IDetailPropertyRow& DepthTextureRow = CompositeCategory.AddProperty(DepthTextureHandle);
	DepthTextureRow.CustomWidget()
	.NameContent()
	[
		DepthTextureHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SMediaProfileSourceTexturePicker)
		.TexturePropertyHandle(DepthTextureHandle)
		.ThumbnailPool(CachedDetailBuilder.IsValid() ? CachedDetailBuilder.Pin()->GetThumbnailPool() : nullptr)
		.AllowedClass(UTexture::StaticClass())
		.OnShouldFilterAsset_Lambda([Handle = DepthTextureHandle](const FAssetData& AssetData)
		{
			return CompositeCustomizationHelpers::ShouldFilterAssetByAllowedClasses(AssetData, Handle);
		})
	];
}

#undef LOCTEXT_NAMESPACE
