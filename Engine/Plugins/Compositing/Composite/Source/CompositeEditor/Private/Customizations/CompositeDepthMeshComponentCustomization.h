// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

/**
 * Customization for the composite depth mesh component.
 */
class FCompositeDepthMeshComponentCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of the details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface

private:
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;
};
