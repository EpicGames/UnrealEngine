// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Layers/CompositeLayerBase.h"
#include "PropertyHandle.h"
#include "UI/CompositeLayerPassListOwner.h"
#include "UI/SCompositePassTreePanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CompositeLayerCustomization"

namespace UE::Composite::Editor::Private
{
	static bool IsLayerPassesProperty(const TSharedRef<IPropertyHandle>& InProperty)
	{
		static const FName LayerPassesName = GET_MEMBER_NAME_CHECKED(UCompositeLayerBase, LayerPasses);
		const FProperty* Property = InProperty->GetProperty();
		return Property != nullptr && Property->GetFName() == LayerPassesName;
	}
}

void FCompositeLayerCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	if (DetailBuilder.IsValid())
	{
		CustomizeLayerDetails(*DetailBuilder);
	}
}

TSharedPtr<SCompositePassTreePanel> FCompositeLayerCustomization::GetPassPanelWidget() const
{
	return PassPanel.Pin();
}

bool FCompositeLayerCustomization::IsCustomizingObject(UObject* InObject) const
{
	if (TSharedPtr<IDetailLayoutBuilder> Builder = CachedDetailBuilder.Pin())
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		Builder->GetObjectsBeingCustomized(ObjectsBeingCustomized);
		return ObjectsBeingCustomized.Contains(InObject);
	}
	return false;
}

void FCompositeLayerCustomization::AddDefaultLayerProperties(
	IDetailCategoryBuilder& InCategory,
	TArrayView<const FName> InAdditionalHidden,
	const FOnRowAdded& InOnRowAdded)
{
	TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
	InCategory.GetDefaultProperties(DefaultProperties, /* bSimpleProperties */ true, /* bAdvancedProperties */ false);

	for (const TSharedRef<IPropertyHandle>& Property : DefaultProperties)
	{
		if (UE::Composite::Editor::Private::IsLayerPassesProperty(Property))
		{
			continue;
		}
		if (Property->IsCustomized())
		{
			continue;
		}
		const FProperty* RawProperty = Property->GetProperty();
		if (RawProperty == nullptr || InAdditionalHidden.Contains(RawProperty->GetFName()))
		{
			continue;
		}

		IDetailPropertyRow& Row = InCategory.AddProperty(Property);
		if (InOnRowAdded)
		{
			InOnRowAdded(Property, Row);
		}
	}
}

void FCompositeLayerCustomization::AddPassesGroup(
	IDetailLayoutBuilder& InDetailLayout,
	IDetailCategoryBuilder& InCategory,
	UCompositeLayerBase* InLayer)
{
	HideLayerPasses(InDetailLayout);

	if (InLayer != nullptr)
	{
		IDetailGroup& PassesGroup = InCategory.AddGroup("Passes", LOCTEXT("PassesGroupName", "Passes"), false, true);

		TSharedRef<SCompositePassTreePanel> Panel =
			SNew(SCompositePassTreePanel, MakeShared<FCompositeLayerPassListOwner>(InLayer))
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh));

		PassPanel = Panel;

		PassesGroup.AddWidgetRow()
		.WholeRowContent()
		[
			Panel
		];
	}
	else
	{
		InCategory.AddCustomRow(LOCTEXT("PassesGroupName", "Passes"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PassesGroupName", "Passes"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
}

void FCompositeLayerCustomization::HideLayerPasses(IDetailLayoutBuilder& InDetailLayout)
{
	// LayerPasses lives on UCompositeLayerBase, not on the customized derived class.
	// Pass the explicit class outer so HideProperty's lookup resolves correctly.
	InDetailLayout.HideProperty(
		GET_MEMBER_NAME_CHECKED(UCompositeLayerBase, LayerPasses),
		UCompositeLayerBase::StaticClass());
}

void FCompositeLayerCustomization::RequestLayoutRefresh()
{
	// Queued/debounced refresh — child widgets bound to this callback can fire OnLayoutSizeChanged
	// on every relayout, so ForceRefreshDetails would create a feedback loop.
	if (TSharedPtr<IDetailLayoutBuilder> Builder = CachedDetailBuilder.Pin())
	{
		Builder->GetPropertyUtilities()->RequestRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
