// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "IDetailCustomization.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;
class SCompositePassTreePanel;
class UCompositeLayerBase;
class UObject;

/**
 * Common base for all composite-layer detail customizations.
 *
 * Owns the shared layout-builder cache and pass-panel weak ptr, finalizes the IDetailCustomization
 * dispatch (caches the builder, then forwards to CustomizeLayerDetails), and provides the helpers
 * every layer customization needs:
 *
 *  - AddDefaultLayerProperties: re-adds the default "Composite" simple-tier properties to a category
 *    in declaration order, with the inherited LayerPasses array filtered out by construction.
 *  - AddPassesGroup: hides LayerPasses and inserts the SCompositePassTreePanel group at the current
 *    insertion point, falling back to a "Multiple Values" row when multiple objects are selected.
 *  - HideLayerPasses: standalone hide for layers that drive their own pass tree (Plate).
 *  - RequestLayoutRefresh: invalidates the layout when a child widget reports a size change.
 *
 * SCompositeEditorPanel `static_cast`s stored IDetailCustomizations to this base when iterating
 * customizations registered on UCompositeLayerBase-derived classes. UE modules disable RTTI, so the
 * cast is the supported way to reach IsCustomizingObject / GetPassPanelWidget.
 */
class FCompositeLayerCustomization : public IDetailCustomization
{
public:
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override final;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override final {}
	//~ End IDetailCustomization

	/** Layer-specific layout. Called after the base has cached the builder. */
	virtual void CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout) = 0;

	/** Pass panel widget hosted by this customization, if alive. */
	TSharedPtr<SCompositePassTreePanel> GetPassPanelWidget() const;

	/** True if this customization is currently customizing the given object. */
	bool IsCustomizingObject(UObject* InObject) const;

	/** Forces the cached layout to regenerate. Public so derived classes can bind it as a
	 *  SLATE_EVENT callback via FSimpleDelegate::CreateSP. */
	void RequestLayoutRefresh();

protected:
	using FOnRowAdded = TFunction<void(const TSharedRef<IPropertyHandle>&, IDetailPropertyRow&)>;

	/**
	 * Adds the default simple-tier properties of InCategory in declaration order, skipping the
	 * inherited UCompositeLayerBase::LayerPasses (handled by AddPassesGroup) and any names in
	 * InAdditionalHidden. Already-customized properties are skipped to avoid double-adding rows.
	 *
	 * If supplied, InOnRowAdded runs for each added row, allowing per-property overrides.
	 */
	void AddDefaultLayerProperties(
		IDetailCategoryBuilder& InCategory,
		TArrayView<const FName> InAdditionalHidden = {},
		const FOnRowAdded& InOnRowAdded = nullptr);

	/**
	 * Hides the inherited LayerPasses property and adds a "Passes" group to InCategory hosting an
	 * SCompositePassTreePanel over the layer's LayerPasses array.
	 *
	 * Pass nullptr for InLayer when multiple objects are selected; the helper inserts a "Multiple
	 * Values" row instead. Stores the created panel as PassPanel.
	 */
	void AddPassesGroup(
		IDetailLayoutBuilder& InDetailLayout,
		IDetailCategoryBuilder& InCategory,
		UCompositeLayerBase* InLayer);

	/**
	 * Hides the inherited UCompositeLayerBase::LayerPasses property. Use when the layer drives
	 * its own pass tree and AddPassesGroup is not appropriate.
	 */
	void HideLayerPasses(IDetailLayoutBuilder& InDetailLayout);

	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;
	TWeakPtr<SCompositePassTreePanel> PassPanel;
};
