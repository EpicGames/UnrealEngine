// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerPlanarReflectionCustomization.h"

#include "CompositeRenderTargetResolutionCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Layers/CompositeLayerPlanarReflection.h"
#include "UI/SCompositeActorPickerTable.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FCompositeLayerPlanarReflectionCustomization"

TSharedRef<IDetailCustomization> FCompositeLayerPlanarReflectionCustomization::MakeInstance()
{
	return MakeShared<FCompositeLayerPlanarReflectionCustomization>();
}

void FCompositeLayerPlanarReflectionCustomization::CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout)
{
	InDetailLayout.RegisterInstancedCustomPropertyTypeLayout(NAME_IntPoint, FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FResolutionTypeCustomization>();
		}),
		MakeShared<FResolutionPropertyIdentifier>());

	static const FName HiddenProperties[] =
	{
		GET_MEMBER_NAME_CHECKED(UCompositeLayerPlanarReflection, Actors)
	};

	for (const FName& PropertyName : HiddenProperties)
	{
		InDetailLayout.HideProperty(PropertyName);
	}

	TArray<TWeakObjectPtr<UCompositeLayerPlanarReflection>> Objects = InDetailLayout.GetObjectsOfTypeBeingCustomized<UCompositeLayerPlanarReflection>();

	IDetailCategoryBuilder& LayerCategory = InDetailLayout.EditCategory("Composite", UCompositeLayerPlanarReflection::StaticClass()->GetDisplayNameText());

	AddDefaultLayerProperties(LayerCategory, HiddenProperties);

	if (Objects.Num() == 1 && Objects[0].IsValid())
	{
		UCompositeLayerPlanarReflection* PlanarReflection = Cast<UCompositeLayerPlanarReflection>(Objects[0].Get());

		AddPassesGroup(InDetailLayout, LayerCategory, PlanarReflection);

		IDetailGroup& ActorListGroup = LayerCategory.AddGroup("ActorContent", LOCTEXT("ActorListGroupName", "Planar Reflection Content"), false, true);
		FCompositeActorPickerListRef ActorListRef(PlanarReflection, GET_MEMBER_NAME_CHECKED(UCompositeLayerPlanarReflection, Actors), &PlanarReflection->Actors, &PlanarReflection->SpawnableBindings);

		ActorListGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SNew(SCompositeActorPickerTable, ActorListRef)
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh))
		];
	}
	else
	{
		AddPassesGroup(InDetailLayout, LayerCategory, /* InLayer */ nullptr);

		// Can't display actor list if multiple layers are selected, so simply put a "Multiple Values" entry in the property list
		LayerCategory.AddCustomRow(LOCTEXT("ActorListGroupName", "Planar Reflection Content"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ActorListGroupName", "Planar Reflection Content"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
}

#undef LOCTEXT_NAMESPACE
