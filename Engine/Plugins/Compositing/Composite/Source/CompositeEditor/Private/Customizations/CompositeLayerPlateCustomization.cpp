// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerPlateCustomization.h"

#include "CompositeActor.h"
#include "CompositeCoreSettings.h"
#include "CompositeCustomizationHelpers.h"
#include "CompositeDepthMeshActor.h"
#include "CompositeEditorStyle.h"
#include "CompositeSkySphereActor.h"
#include "CompositeMeshActor.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "Engine/Texture2D.h"
#include "IDetailGroup.h"
#include "LevelEditorSubsystem.h"
#include "MediaTexture.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "SMediaProfileSourceTexturePicker.h"
#include "UnrealEdGlobals.h"
#include "Camera/CameraComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/SkyLight.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layers/CompositeLayerPlate.h"
#include "Materials/MaterialInstance.h"
#include "Styling/SlateIconFinder.h"
#include "UI/CompositePlatePassListOwner.h"
#include "UI/SCompositeActorPickerTable.h"
#include "UI/SCompositePassTreePanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "FCompositeLayerPlateCustomization"

namespace CompositeLayerPlateCustomization
{
	static FName CompositeMeshListColumn_Components = "Components";
	static FName CompositeMeshListColumn_Material = "Material";
}

TSharedRef<IDetailCustomization> FCompositeLayerPlateCustomization::MakeInstance()
{
	return MakeShared<FCompositeLayerPlateCustomization>();
}

void FCompositeLayerPlateCustomization::CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout)
{
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Objects = InDetailLayout.GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();

	static const FName HiddenProperties[] =
	{
		GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, CompositeMeshes),
		GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, MediaPasses)
	};

	for (const FName& PropertyName : HiddenProperties)
	{
		InDetailLayout.HideProperty(PropertyName);
	}

	HideLayerPasses(InDetailLayout);

	IDetailCategoryBuilder& PlateCategory = InDetailLayout.EditCategory("Composite", UCompositeLayerPlate::StaticClass()->GetDisplayNameText());

	static const FName TextureName = GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, Texture);
	auto CustomizeRow = [this](const TSharedRef<IPropertyHandle>& InProperty, IDetailPropertyRow& InRow)
	{
		if (InProperty->GetProperty()->GetFName() == TextureName)
		{
			CustomizeTexturePropertyRow(InProperty, InRow);
		}
	};
	AddDefaultLayerProperties(PlateCategory, HiddenProperties, CustomizeRow);
	
	if (Objects.Num() == 1 && Objects[0].IsValid())
	{
		UCompositeLayerPlate* Plate = Cast<UCompositeLayerPlate>(Objects[0].Get());

		IDetailGroup& ActorListGroup = PlateCategory.AddGroup("CompositeMeshContent", LOCTEXT("CompositeMeshGroupName", "Composite Mesh Content"), false, true);
		FCompositeActorPickerListRef ActorListRef(Plate, GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, CompositeMeshes), &Plate->CompositeMeshes, &Plate->SpawnableBindings);
		ActorListRef.OnActorsAdded.BindSP(this, &FCompositeLayerPlateCustomization::OnCompositeMeshActorsAdded);

		TArray<FName> HiddenColumns;
		if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
		{
			HiddenColumns = Settings->CompositeMeshTableHiddenColumns;
		}
	
		ActorListGroup.SetToolTip(ActorListRef.GetToolTipText());
		ActorListGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SAssignNew(CompositeMeshPicker, SCompositeActorPickerTable, ActorListRef)
			.SceneOutlinerFilters_Lambda([]()
			{
				constexpr bool bExcludeCompositeMeshActors = false;
				return SCompositeActorPickerTable::MakeDefaultSceneOutlinerFilters(bExcludeCompositeMeshActors);
			})
			.HiddenColumnsList(HiddenColumns)
			.OnHiddenColumnsListChanged(this, &FCompositeLayerPlateCustomization::OnHiddenColumnsListChanged)
			.OnGenerateHeaderColumns_Lambda([](TSharedRef<SHeaderRow> InHeaderRow)
			{
				TArray<FName> HiddenColumns;
				if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
				{
					HiddenColumns = Settings->CompositeMeshTableHiddenColumns;
				}
				
				// .HiddenColumnsList workaround:
				// simulate what SHeaderRow::AddColumn( const FColumn::FArguments& NewColumnArgs ) does but allowing us to modify the bIsVisible property
				// Memory handling (delete) is done by TIndirectArray<FColumn> Columns; defined in SHeaderRow
				{
					SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn(SHeaderRow::Column(CompositeLayerPlateCustomization::CompositeMeshListColumn_Material)
						.DefaultLabel(LOCTEXT("MaterialColumnLabel", "Material"))
						.FillWidth(0.4));
					NewColumn->bIsVisible = !HiddenColumns.Contains(CompositeLayerPlateCustomization::CompositeMeshListColumn_Material);
					InHeaderRow->AddColumn(*NewColumn);
				}
				
				{
					SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn(SHeaderRow::Column(CompositeLayerPlateCustomization::CompositeMeshListColumn_Components)
						.DefaultLabel(LOCTEXT("ComponentsColumnLabel", "Components"))
						.FillWidth(0.4));
					NewColumn->bIsVisible = !HiddenColumns.Contains(CompositeLayerPlateCustomization::CompositeMeshListColumn_Components);
					InHeaderRow->AddColumn(*NewColumn);
				}
			})
			.OnGenerateColumnWidget(this, &FCompositeLayerPlateCustomization::GetCompositeMeshColumnWidget)
			.OnExtendAddMenu(this, &FCompositeLayerPlateCustomization::OnExtendCompositeMeshAddMenu)
			.OnExtendContextMenu(this, &FCompositeLayerPlateCustomization::OnExtendCompositeMeshContextMenu)
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh))
			.ShowApplyMaterialSection(true)
			.OnApplyMaterialToActor(this, &FCompositeLayerPlateCustomization::ApplyMaterialToActor)
		];
		
		IDetailGroup& PassesGroup = PlateCategory.AddGroup("Passes", LOCTEXT("PassesGroupName", "Passes"), false, true);

		PassesGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SAssignNew(PassPanel, SCompositePassTreePanel, MakeShared<FCompositePlatePassListOwner>(Plate))
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh))
		];
	}  
	else
	{
		// Can't display plate pass panel if multiple plates are selected, so simply put a "Multiple Values" entry in the property list
		PlateCategory.AddCustomRow(LOCTEXT("PassesGroupName", "Passes"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PassesGroupName", "Passes"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
}

TSharedPtr<SCompositeActorPickerTable> FCompositeLayerPlateCustomization::GetCompositeMeshPickerWidget() const
{
	return CompositeMeshPicker.Pin();
}

void FCompositeLayerPlateCustomization::CustomizeTexturePropertyRow(const TSharedPtr<IPropertyHandle>& InPropertyHandle, IDetailPropertyRow& InPropertyRow)
{
	TexturePropertyHandle = InPropertyHandle;

	InPropertyRow.CustomWidget()
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SMediaProfileSourceTexturePicker)
		.TexturePropertyHandle(InPropertyHandle)
		.ThumbnailPool(CachedDetailBuilder.IsValid() ? CachedDetailBuilder.Pin()->GetThumbnailPool() : nullptr)
		.AllowedClass(UTexture::StaticClass())
		.OnShouldFilterAsset_Lambda([Handle = InPropertyHandle.ToSharedRef()](const FAssetData& AssetData)
		{
			return CompositeCustomizationHelpers::ShouldFilterAssetByAllowedClasses(AssetData, Handle);
		})
		.AdditionalContentBeforeMediaButton(true)
		.AdditionalContent()
		[
			SNew(SComboButton)
			.HasDownArrow(true)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.ToolTipText(LOCTEXT("PlaceSkySphereButtonToolTip", "Place a Composite Sky Sphere Actor using the plate's current texture"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnGetMenuContent(this, &FCompositeLayerPlateCustomization::GetSkySphereActorMenu)
			.IsEnabled(this, &FCompositeLayerPlateCustomization::CanPlaceCompositeSkySphereActor)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FCompositeEditorStyle::Get().GetBrush("ClassIcon.CompositeSkySphereActor"))
			]
		]
	];
}

void FCompositeLayerPlateCustomization::OnHiddenColumnsListChanged()
{
	TSharedPtr<SCompositeActorPickerTable> PinnedCompositeMeshPicker = CompositeMeshPicker.Pin();
	if (!PinnedCompositeMeshPicker.IsValid())
	{
		return;
	}

	if (UCompositeEditorPanelSettings* Settings = GetMutableDefault<UCompositeEditorPanelSettings>())
	{
		Settings->CompositeMeshTableHiddenColumns = PinnedCompositeMeshPicker->GetHiddenColumnsList();
		Settings->SaveConfig();
	}
}

TSharedRef<SWidget> FCompositeLayerPlateCustomization::GetCompositeMeshColumnWidget(const TSoftObjectPtr<AActor>& InActor, const FName& InColumnName)
{
	if (InColumnName == CompositeLayerPlateCustomization::CompositeMeshListColumn_Material)
	{
		return SNew(STextBlock).Text(this, &FCompositeLayerPlateCustomization::GetCompositeMeshMaterialLabel, InActor);	
	}
	
	if (InColumnName == CompositeLayerPlateCustomization::CompositeMeshListColumn_Components)
	{
		return SNew(STextBlock).Text(this, &FCompositeLayerPlateCustomization::GetCompositeMeshComponentsLabel, InActor);
	}
	
	return SNullWidget::NullWidget;
}

FText FCompositeLayerPlateCustomization::GetCompositeMeshMaterialLabel(TSoftObjectPtr<AActor> InActor) const
{
	if (!InActor.IsValid())
	{
		return FText::GetEmpty();
	}

	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return FText::GetEmpty();
	}

	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return FText::GetEmpty();
	}

	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetPrimitivesForActor(InActor);

	TSet<FString> MaterialNames;
	for (const UPrimitiveComponent* Primitive : CompositeMeshPrimitives)
	{
		if (UMaterialInterface* Material = Primitive->GetMaterial(0))
		{
			MaterialNames.Add(Material->GetName());
		}
	}

	if (MaterialNames.Num() > 1)
	{
		return FText::Format(LOCTEXT("MultipleMaterialsLabelFormat", "{0} Materials"), MaterialNames.Num());
	}
	if (MaterialNames.Num() == 1)
	{
		const FString MaterialName = MaterialNames.Array()[0];
		
		// For the default lit and unlit materials, output friendly names
		if (MaterialName.Contains("CompositeMesh_Lit_Masked"))
		{
			return LOCTEXT("LitMaskedMaterialFriendlyName", "Lit Masked");
		}
		if (MaterialName.Contains("CompositeMesh_Unlit_AlphaComposite"))
		{
			return LOCTEXT("UnlitAlphaMaterialFriendlyName", "Unlit Alpha");
		}
		
		return FText::FromString(MaterialName);
	}

	return LOCTEXT("NoMaterialsLabel", "No Materials");
}

FText FCompositeLayerPlateCustomization::GetCompositeMeshComponentsLabel(TSoftObjectPtr<AActor> InActor) const
{
	if (!InActor.IsValid())
	{
		return FText::GetEmpty();
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return FText::GetEmpty();
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return FText::GetEmpty();
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetIncludedPrimitivesForActor(InActor);
	
	if (CompositeMeshPrimitives.Num() > 1)
	{
		return FText::Format(LOCTEXT("MultipleComponentsLabelFormat", "{0} Components"), CompositeMeshPrimitives.Num());
	}
	if (CompositeMeshPrimitives.Num() == 1)
	{
		return FText::FromString(CompositeMeshPrimitives[0]->GetName());
	}
	
	return FText::GetEmpty(); 
}

void FCompositeLayerPlateCustomization::OnExtendCompositeMeshAddMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("PlaceCompositeMeshActorEntryLabel", "Place Composite Mesh Actor"),
		LOCTEXT("PlaceCompositeMeshActorEntryToolTip", "Place a composite mesh actor at the appropriate position in the level"),
		FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.Composure"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::CreateCompositeMeshActor),
			FCanExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::CanCreateCompositeMeshActor))
	);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("PlaceCompositeDepthMeshActorEntryLabel", "Place Composite Depth Mesh Actor"),
		LOCTEXT("PlaceCompositeDepthMeshActorEntryToolTip", "Place a composite depth mesh actor at the appropriate position in the level"),
		FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.Composure"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::CreateCompositeDepthMeshActor),
			FCanExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::CanCreateCompositeMeshActor))
	);
	
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("ApplyMaterial", LOCTEXT("ApplyMaterialAddMenuSectionLabel", "Apply Material"));
	
	FMenuEntryParams AutoApplyMaterialEntry;
	AutoApplyMaterialEntry.LabelOverride = LOCTEXT("AutoApplyMaterialLabel", "Automatically Apply Material");
	AutoApplyMaterialEntry.ToolTipOverride = LOCTEXT("AutoApplyMaterialToolTip", "Automatically applies the configured material to any actors added to this list");
	AutoApplyMaterialEntry.UserInterfaceActionType = EUserInterfaceActionType::RadioButton;
	AutoApplyMaterialEntry.bShouldCloseWindowAfterMenuSelection = false;
	AutoApplyMaterialEntry.DirectActions = FUIAction(
		FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::SetAutoApplyMaterial, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsAutoApplyMaterialChecked, true));
	MenuBuilder.AddMenuEntry(AutoApplyMaterialEntry);
	
	FMenuEntryParams NoModifyMaterialEntry;
	NoModifyMaterialEntry.LabelOverride = LOCTEXT("NoModifyMaterialLabel", "Do Not Modify Actor's Material");
	NoModifyMaterialEntry.ToolTipOverride = LOCTEXT("NoModifyMaterialToolTip", "The actor's materials are not modified when the actor is added to this list");
	NoModifyMaterialEntry.UserInterfaceActionType = EUserInterfaceActionType::RadioButton;
	NoModifyMaterialEntry.bShouldCloseWindowAfterMenuSelection = false;
	NoModifyMaterialEntry.DirectActions = FUIAction(
		FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::SetAutoApplyMaterial, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsAutoApplyMaterialChecked, false));
	MenuBuilder.AddMenuEntry(NoModifyMaterialEntry);
	
	MenuBuilder.AddSeparator();
	
	MenuBuilder.AddSubMenu(
		TAttribute<FText>::CreateSP(this, &FCompositeLayerPlateCustomization::GetMaterialSubMenuLabel),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &FCompositeLayerPlateCustomization::CreateMaterialSubMenu),
		false,
		FSlateIcon(),
		false
	);
}

void FCompositeLayerPlateCustomization::SetAutoApplyMaterial(bool bInAutoApplyMaterial)
{
	if (UCompositeEditorPanelSettings* Settings = GetMutableDefault<UCompositeEditorPanelSettings>())
	{
		Settings->bAutoApplyCompositeMeshMaterial = bInAutoApplyMaterial;
		Settings->SaveConfig();
	}
}

bool FCompositeLayerPlateCustomization::IsAutoApplyMaterialChecked(bool bInAutoApplyMaterial) const
{
	if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
	{
		return Settings->bAutoApplyCompositeMeshMaterial == bInAutoApplyMaterial;
	}
	
	return false;
}

FText FCompositeLayerPlateCustomization::GetMaterialSubMenuLabel() const
{
	FText MaterialName = FText::GetEmpty();
	
	if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
	{
		switch (Settings->CompositeMeshMaterialType)
		{
		case ECompositeMeshAppliedMaterialType::LitMaskedMaterial:
			MaterialName = LOCTEXT("LitMaskedMaterialName", "Lit Masked");
			break;
		
		case ECompositeMeshAppliedMaterialType::UnlitAlphaMaterial:
			MaterialName = LOCTEXT("UnlitAlphaMaterialName", "Unlit Alpha");
			break;
		
		case ECompositeMeshAppliedMaterialType::CustomMaterial:
			if (CustomMaterialAsset.IsValid())
			{
				MaterialName = FText::FromName(CustomMaterialAsset.AssetName);
			}
			break;
		}
	}
	
	return FText::Format(LOCTEXT("AutoApplyMaterialSubMenuLabelFormat", "Material ({0})"), MaterialName);
}

void FCompositeLayerPlateCustomization::CreateMaterialSubMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("Material", LOCTEXT("MaterialSectionLabel", "Material"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ApplyLitMaskedMaterial", "Apply Lit Masked Material"),
			LOCTEXT("ApplyLitMaskedMaterialToolTip", "Apply the plugin default lit masked material to selected actors: best for catching shadows and reflections."),
			FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.LitMaterial"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::SetAutoAppliedMaterialType, ECompositeMeshAppliedMaterialType::LitMaskedMaterial),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsAutoAppliedMaterialTypeChecked, ECompositeMeshAppliedMaterialType::LitMaskedMaterial)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ApplyUnlitAlphaMaterial", "Apply Unlit Alpha Material"),
			LOCTEXT("ApplyUnlitAlphaMaterialToolTip", "Apply the plugin default unlit alpha material to selected actors: best for keying media."),
			FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.UnlitMaterial"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::SetAutoAppliedMaterialType, ECompositeMeshAppliedMaterialType::UnlitAlphaMaterial),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsAutoAppliedMaterialTypeChecked, ECompositeMeshAppliedMaterialType::UnlitAlphaMaterial)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ApplyCustomMaterial", "Apply Custom Material"),
			LOCTEXT("ApplyCustomMaterialToolTip", "Apply a selected custom material"),
			FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.CustomMaterial"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::SetAutoAppliedMaterialType, ECompositeMeshAppliedMaterialType::CustomMaterial),
				FCanExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::CanApplyCustomMaterial),
				FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsAutoAppliedMaterialTypeChecked, ECompositeMeshAppliedMaterialType::CustomMaterial)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		InMenuBuilder.AddSeparator();
		InMenuBuilder.AddMenuEntry(
			TAttribute<FText>::CreateSP(this, &FCompositeLayerPlateCustomization::GetCustomMaterialEntryLabel),
			FText::GetEmpty(),
			FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.CustomMaterial"),
			FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([] { return false; })));
	}
	InMenuBuilder.EndSection();

	InMenuBuilder.BeginSection("MaterialBrowser", LOCTEXT("MaterialBrowserSectionLabel", "Browse"));
	{
		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.ClassPaths.Add(UMaterialInstance::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FCompositeLayerPlateCustomization::OnCustomMaterialSelected);
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.InitialAssetSelection = CustomMaterialAsset;

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		InMenuBuilder.AddWidget(
			SNew(SBox)
			.MaxDesiredHeight(600.0f)
			.WidthOverride(256.0f)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			], 
			FText(),
			true
		);
	}
	InMenuBuilder.EndSection();
}

void FCompositeLayerPlateCustomization::SetAutoAppliedMaterialType(ECompositeMeshAppliedMaterialType InMaterialType)
{
	if (UCompositeEditorPanelSettings* Settings = GetMutableDefault<UCompositeEditorPanelSettings>())
	{
		Settings->CompositeMeshMaterialType = InMaterialType;
		Settings->SaveConfig();
	}
}

bool FCompositeLayerPlateCustomization::IsAutoAppliedMaterialTypeChecked(ECompositeMeshAppliedMaterialType InMaterialType) const
{
	if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
	{
		return Settings->CompositeMeshMaterialType == InMaterialType;
	}
	
	return false;
}

FText FCompositeLayerPlateCustomization::GetCustomMaterialEntryLabel() const
{
	if (CustomMaterialAsset.IsValid() && CustomMaterialAsset.IsInstanceOf(UMaterialInterface::StaticClass()))
	{
		return FText::FromName(CustomMaterialAsset.AssetName);
	}
	
	return LOCTEXT("SelectCustomMaterial", "Select a Custom Material");
}

bool FCompositeLayerPlateCustomization::CanApplyCustomMaterial() const
{
	return CustomMaterialAsset.IsValid() && CustomMaterialAsset.IsInstanceOf(UMaterialInterface::StaticClass());
}

void FCompositeLayerPlateCustomization::OnCustomMaterialSelected(const FAssetData& AssetData)
{
	CustomMaterialAsset = AssetData;
	FSlateApplication::Get().DismissAllMenus();
}

void FCompositeLayerPlateCustomization::OnExtendCompositeMeshContextMenu(FMenuBuilder& MenuBuilder, TArray<TSoftObjectPtr<AActor>>& SelectedActors)
{
	if (SelectedActors.Num() != 1)
	{
		return;
	}
	
	TSoftObjectPtr<AActor> Actor = SelectedActors[0];
	if (!Actor.IsValid())
	{
		return;
	}

	MenuBuilder.BeginSection("ComponentSection", LOCTEXT("ComponentSectionLabel", "Component"));
	{
		FMenuEntryParams SelectedActorEntry;
		SelectedActorEntry.LabelOverride = LOCTEXT("SelectedActorEntryLabel", "Selected Actor");
		SelectedActorEntry.ToolTipOverride = LOCTEXT("SelectedActorEntryToolTip", "Select all components of this actor");
		SelectedActorEntry.IconOverride = FSlateIconFinder::FindIconForClass(Actor->GetClass());
		SelectedActorEntry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
		SelectedActorEntry.bShouldCloseWindowAfterMenuSelection = false;
		SelectedActorEntry.DirectActions = FUIAction(
			FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::OnCompositeMeshActorToggled, Actor),
			FCanExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::CanToggleCompositeMeshActor, Actor),
			FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsCompositeMeshActorChecked, Actor));
		
		MenuBuilder.AddMenuEntry(SelectedActorEntry);
		
		MenuBuilder.AddSeparator();
		
		const UCompositeCorePluginSettings* Settings = GetDefault<UCompositeCorePluginSettings>();
		TInlineComponentArray<UPrimitiveComponent*> PrimComponents(Actor.Get());

		for (UPrimitiveComponent* PrimitiveComponent : PrimComponents)
		{
			if (Settings && Settings->IsAllowedPrimitiveClass(PrimitiveComponent))
			{
				FMenuEntryParams ComponentEntry;
				ComponentEntry.LabelOverride = FText::FromString(PrimitiveComponent->GetName());
				ComponentEntry.IconOverride = FSlateIconFinder::FindIconForClass(PrimitiveComponent->GetClass());
				ComponentEntry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
				ComponentEntry.bShouldCloseWindowAfterMenuSelection = false;
				ComponentEntry.DirectActions = FUIAction(
					FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::OnCompositeMeshPrimitiveToggled, Actor, PrimitiveComponent),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &FCompositeLayerPlateCustomization::IsCompositeMeshPrimitiveChecked, Actor, PrimitiveComponent));
		
				MenuBuilder.AddMenuEntry(ComponentEntry);
			}
		}
	}
	MenuBuilder.EndSection();
}

void FCompositeLayerPlateCustomization::OnCompositeMeshActorToggled(TSoftObjectPtr<AActor> InActor)
{
	if (!InActor.IsValid())
	{
		return;
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return;
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetIncludedPrimitivesForActor(InActor);

	FScopedTransaction Transaction(LOCTEXT("ToggleCompositeMeshActor", "Toggle Composite Mesh Actor"));
	Plate->Modify();
	
	// Toggling the whole actor means removing all specifically included primitives so that all valid primitives in the actor are used
	for (UPrimitiveComponent* PrimitiveComponent : CompositeMeshPrimitives)
	{
		Plate->SetPrimitiveIncluded(PrimitiveComponent, false);
	}
}

bool FCompositeLayerPlateCustomization::CanToggleCompositeMeshActor(TSoftObjectPtr<AActor> InActor) const
{
	if (!InActor.IsValid())
	{
		return false;
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return false;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return false;
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetIncludedPrimitivesForActor(InActor);
	
	return !CompositeMeshPrimitives.IsEmpty();
}

bool FCompositeLayerPlateCustomization::IsCompositeMeshActorChecked(TSoftObjectPtr<AActor> InActor) const
{
	if (!InActor.IsValid())
	{
		return false;
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return false;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return false;
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetIncludedPrimitivesForActor(InActor);
	
	return CompositeMeshPrimitives.IsEmpty();
}

void FCompositeLayerPlateCustomization::OnCompositeMeshPrimitiveToggled(TSoftObjectPtr<AActor> InActor, UPrimitiveComponent* InPrimitive)
{
	if (!InActor.IsValid())
	{
		return;
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return;
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetIncludedPrimitivesForActor(InActor);
	
	FScopedTransaction Transaction(LOCTEXT("ToggleCompositeMeshPrimitives", "Toggle Composite Mesh Primitive"));
	Plate->Modify();
	
	const bool bInclude = !CompositeMeshPrimitives.Contains(InPrimitive);
	Plate->SetPrimitiveIncluded(InPrimitive, bInclude);
}

bool FCompositeLayerPlateCustomization::IsCompositeMeshPrimitiveChecked(TSoftObjectPtr<AActor> InActor, UPrimitiveComponent* InPrimitive) const
{
	if (!InActor.IsValid())
	{
		return false;
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return false;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return false;
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetIncludedPrimitivesForActor(InActor);
	
	return CompositeMeshPrimitives.Contains(InPrimitive);
}

void FCompositeLayerPlateCustomization::OnCompositeMeshActorsAdded(TArray<TSoftObjectPtr<AActor>>& InActors)
{
	if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
	{
		if (Settings->bAutoApplyCompositeMeshMaterial)
		{
			UMaterialInterface* Material = nullptr;
			switch (Settings->CompositeMeshMaterialType)
			{
			case ECompositeMeshAppliedMaterialType::LitMaskedMaterial:
				Material = LoadObject<UMaterialInterface>(nullptr, TEXT("/Composite/Materials/M_CompositeMesh_Lit_Masked.M_CompositeMesh_Lit_Masked"));
				break;
			
			case ECompositeMeshAppliedMaterialType::UnlitAlphaMaterial:
				Material = LoadObject<UMaterialInterface>(nullptr, TEXT("/Composite/Materials/M_CompositeMesh_Unlit_AlphaComposite.M_CompositeMesh_Unlit_AlphaComposite"));
				break;
			
			case ECompositeMeshAppliedMaterialType::CustomMaterial:
				if (CustomMaterialAsset.IsValid() && CustomMaterialAsset.IsInstanceOf(UMaterialInterface::StaticClass()))
				{
					Material = Cast<UMaterialInterface>(CustomMaterialAsset.GetAsset());
				}
				break;
			}

			if (Material == nullptr)
			{
				return;
			}

			for (TSoftObjectPtr<AActor>& Actor : InActors)
			{
				// Skip depth mesh actors, as they have a special material
				if (Actor->IsA<ACompositeDepthMeshActor>())
				{
					continue;
				}
				
				ApplyMaterialToActor(Actor, Material);
			}
		}
	}
}

void FCompositeLayerPlateCustomization::ApplyMaterialToActor(TSoftObjectPtr<AActor>& InActor, UMaterialInterface* InMaterial)
{
	if (!InActor.IsValid())
	{
		return;
	}
	
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Plates = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Plates.Num() != 1 || !Plates[0].IsValid())
	{
		return;
	}
	
	UCompositeLayerPlate* Plate = Plates[0].Get();
	TArray<UPrimitiveComponent*> CompositeMeshPrimitives = Plate->GetPrimitivesForActor(InActor);
	
	for (UPrimitiveComponent* Primitive : CompositeMeshPrimitives)
	{
		Primitive->Modify();
		const int32 MaterialCount = Primitive->GetNumMaterials();
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; MaterialIndex++)
		{
			Primitive->SetMaterial(MaterialIndex, InMaterial);
		}
	}
}

void FCompositeLayerPlateCustomization::CreateCompositeMeshActor()
{
	TSharedPtr<IDetailLayoutBuilder> PinnedDetailBuilder = CachedDetailBuilder.Pin();
	if (!PinnedDetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Objects = PinnedDetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Objects.Num() != 1)
	{
		return;
	}

	TStrongObjectPtr<UCompositeLayerPlate> Plate = Objects[0].Pin();
	if (!Plate.IsValid())
	{
		return;
	}

	ACompositeActor* CompositeActor = Plate->GetTypedOuter<ACompositeActor>();
	if (!CompositeActor)
	{
		return;
	}

	FTransform SpawnTransform = FTransform::Identity;
	if (UCameraComponent* CameraComponent = CompositeActor->GetCameraComponent())
	{
		// Size of one wall of the composite mesh
		constexpr float MeshSize = 1000.0f;
		
		const float FieldOfView = FMath::DegreesToRadians(CameraComponent->FieldOfView);

		// The distance from the camera a wall of MeshSize needs to be to fill the camera's horizontal field of view
		const float Distance = 0.5f * MeshSize / FMath::Tan(0.5f * FieldOfView);
		
		const FVector Forward = CameraComponent->GetForwardVector();
		const FVector Right = CameraComponent->GetRightVector();
		const FVector Up = CameraComponent->GetUpVector();
		
		const FVector CameraLoc = CameraComponent->GetComponentLocation();

		// Mesh location is set it is horizontally centered with the camera
		const FVector MeshLoc = CameraLoc - 0.333 * MeshSize * Up + (Distance - MeshSize) * Forward;
		
		SpawnTransform.SetLocation(MeshLoc);
		
		FRotator Rotation = CameraComponent->GetComponentRotation();
		// Only preserve the yaw to keep the mesh horizontally level
		Rotation.Pitch = 0.0f;
		Rotation.Roll = 0.0f;
		Rotation.Yaw -= 45.0f; // Additional rotation so that the mesh corner is aligned with the camera direction
		SpawnTransform.SetRotation(Rotation.Quaternion());

		// Increase default mesh scale
		SpawnTransform.SetScale3D(2.0 * FVector::One());
	}

	FScopedTransaction AddCompositeMeshTransaction(LOCTEXT("AddCompositeMeshTransaction", "Add Composite Mesh"));
	ACompositeMeshActor* AddedMeshActor = Cast<ACompositeMeshActor>(CompositeActor->GetWorld()->SpawnActor(ACompositeMeshActor::StaticClass(), &SpawnTransform));
	if (AddedMeshActor)
	{
		Plate->Modify();

		FProperty* CompositeMeshesProperty = FindFProperty<FProperty>(UCompositeLayerPlate::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, CompositeMeshes));
		Plate->PreEditChange(CompositeMeshesProperty);

		Plate->CompositeMeshes.Add(AddedMeshActor);
		
		TArray<TSoftObjectPtr<AActor>> AddedActors;
		AddedActors.Add(AddedMeshActor);

		OnCompositeMeshActorsAdded(AddedActors);

		FPropertyChangedEvent ChangedEvent(CompositeMeshesProperty, EPropertyChangeType::ArrayAdd);
		Plate->PostEditChangeProperty(ChangedEvent);

		if (GEditor)
		{
			GEditor->SelectNone(true, true);
			GEditor->SelectActor(AddedMeshActor, true, true);

			bool bAlignViewport = true;
			if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
			{
				// Don't align the viewport if there is an active pilot actor, as that will cause the actor to be moved when the viewport
				// camera is moved
				bAlignViewport = !IsValid(LevelEditorSubsystem->GetPilotLevelActor());
			}

			if (bAlignViewport)
			{
				GUnrealEd->Exec(CompositeActor->GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY"));
			}
		}
	}
}

void FCompositeLayerPlateCustomization::CreateCompositeDepthMeshActor()
{
	TSharedPtr<IDetailLayoutBuilder> PinnedDetailBuilder = CachedDetailBuilder.Pin();
	if (!PinnedDetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Objects = PinnedDetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Objects.Num() != 1)
	{
		return;
	}

	TStrongObjectPtr<UCompositeLayerPlate> Plate = Objects[0].Pin();
	if (!Plate.IsValid())
	{
		return;
	}

	ACompositeActor* CompositeActor = Plate->GetTypedOuter<ACompositeActor>();
	if (!CompositeActor)
	{
		return;
	}

	FTransform SpawnTransform = FTransform::Identity;

	FScopedTransaction AddCompositeDepthMeshTransaction(LOCTEXT("AddCompositeDepthMeshTransaction", "Add Composite Depth Mesh"));
	ACompositeDepthMeshActor* AddedDepthMeshActor = Cast<ACompositeDepthMeshActor>(CompositeActor->GetWorld()->SpawnActor(ACompositeDepthMeshActor::StaticClass(), &SpawnTransform));
	if (AddedDepthMeshActor)
	{
		Plate->Modify();

		FProperty* CompositeMeshesProperty = FindFProperty<FProperty>(UCompositeLayerPlate::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, CompositeMeshes));
		Plate->PreEditChange(CompositeMeshesProperty);

		Plate->CompositeMeshes.Add(AddedDepthMeshActor);

		FPropertyChangedEvent ChangedEvent(CompositeMeshesProperty, EPropertyChangeType::ArrayAdd);
		Plate->PostEditChangeProperty(ChangedEvent);
	}
}

bool FCompositeLayerPlateCustomization::CanCreateCompositeMeshActor() const
{
	TSharedPtr<IDetailLayoutBuilder> PinnedDetailBuilder = CachedDetailBuilder.Pin();
	if (!PinnedDetailBuilder.IsValid())
	{
		return false;
	}
	
	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Objects = PinnedDetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Objects.Num() != 1)
	{
		return false;
	}

	ACompositeActor* CompositeActor = Objects[0].IsValid() ? Objects[0]->GetTypedOuter<ACompositeActor>() : nullptr;
	if (!CompositeActor)
	{
		return false;
	}
	
	return true;
}

TSharedRef<SWidget> FCompositeLayerPlateCustomization::GetSkySphereActorMenu()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("PlaceSkySphereActorEntryLabel", "Place Composite SkySphere Actor"),
		FText::GetEmpty(),
		FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "ClassIcon.CompositeSkySphereActor"),
		FUIAction(FExecuteAction::CreateSP(this, &FCompositeLayerPlateCustomization::PlaceCompositeSkySphereActor))
	);

	return MenuBuilder.MakeWidget();
}

void FCompositeLayerPlateCustomization::PlaceCompositeSkySphereActor()
{
	TSharedPtr<IDetailLayoutBuilder> PinnedDetailBuilder = CachedDetailBuilder.Pin();
	if (!PinnedDetailBuilder.IsValid())
	{
		return;
	}

	TArray<TWeakObjectPtr<UCompositeLayerPlate>> Objects = PinnedDetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositeLayerPlate>();
	if (Objects.Num() != 1 || !Objects[0].IsValid())
	{
		return;
	}

	ACompositeActor* CompositeActor = Objects[0]->GetTypedOuter<ACompositeActor>();
	if (!CompositeActor)
	{
		return;
	}

	// Resolve the currently assigned plate texture.
	UObject* TextureObj = nullptr;
	if (TexturePropertyHandle.IsValid())
	{
		TexturePropertyHandle->GetValue(TextureObj);
	}
	UTexture* CurrentTexture = Cast<UTexture>(TextureObj);

	UWorld* World = CompositeActor->GetWorld();
	if (!World)
	{
		return;
	}

	// Only prompt if the scene already has a sky light — the sky sphere adds its own real-time sky light.
	if (TActorIterator<ASkyLight>(World))
	{
		const FText DialogMessage = LOCTEXT("PlaceSkySphereConfirmSkyLightWarning",
			"The scene already contains a Sky Light actor. The Composite Sky Sphere Actor includes its own real-time sky light, which may produce unexpected results.");

		if (FMessageDialog::Open(EAppMsgType::YesNo, DialogMessage,
			LOCTEXT("PlaceSkySphereDialogTitle", "Create Composite Sky Sphere")) != EAppReturnType::Yes)
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("PlaceSkySphereTransaction", "Place Composite Sky Sphere"));

	// Use deferred spawn so the texture is set before OnConstruction runs UpdateMaterial.
	ACompositeSkySphereActor* SkySphereActor = World->SpawnActorDeferred<ACompositeSkySphereActor>(
		ACompositeSkySphereActor::StaticClass(), FTransform::Identity);
	if (SkySphereActor)
	{
		SkySphereActor->SetTexture(CurrentTexture);
		SkySphereActor->FinishSpawning(FTransform::Identity);

		if (GEditor)
		{
			GEditor->SelectNone(true, true);
			GEditor->SelectActor(SkySphereActor, true, true);
		}
	}
}

bool FCompositeLayerPlateCustomization::CanPlaceCompositeSkySphereActor() const
{
	if (!CanCreateCompositeMeshActor())
	{
		return false;
	}

	UObject* TextureObj = nullptr;
	if (TexturePropertyHandle.IsValid())
	{
		TexturePropertyHandle->GetValue(TextureObj);
	}

	return Cast<UTexture>(TextureObj) != nullptr;
}


#undef LOCTEXT_NAMESPACE
