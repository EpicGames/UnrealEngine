// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerCustomization.h"
#include "AssetRegistry/AssetData.h"
#include "Materials/Material.h"
#include "UObject/SoftObjectPtr.h"
#include "Styling/SlateTypes.h"
#include "UI/CompositeEditorPanelSettings.h"

class SWidget;
class IDetailPropertyRow;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SCompositeActorPickerTable;
class FMenuBuilder;

/**
 * Customization for the composite plate layer, primary for displaying a custom widget for passes and composite meshes
 */
class FCompositeLayerPlateCustomization : public FCompositeLayerCustomization
{
public:
	/** Makes a new instance of the details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin FCompositeLayerCustomization
	virtual void CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout) override;
	//~ End FCompositeLayerCustomization

	TSharedPtr<SCompositeActorPickerTable> GetCompositeMeshPickerWidget() const;

private:
	/** Customizes the Texture property in UCompositeLayerPlate to add in the media profile source selector */
	void CustomizeTexturePropertyRow(const TSharedPtr<IPropertyHandle>& InPropertyHandle, IDetailPropertyRow& InPropertyRow);

	/** Raised when the user has hidden columns in the composite mesh actor picker table */
	void OnHiddenColumnsListChanged();

	/** Creates the widget to display for each actor in the Components column of the actor picker table */
	TSharedRef<SWidget> GetCompositeMeshColumnWidget(const TSoftObjectPtr<AActor>& InActor, const FName& InColumnName);

	/** Gets the label to display for the Material column for each actor in the actor picker table */
	FText GetCompositeMeshMaterialLabel(TSoftObjectPtr<AActor> Actor) const;

	/** Gets the label to display for each actor in the Components column of the actor picker table */
	FText GetCompositeMeshComponentsLabel(TSoftObjectPtr<AActor> InActor) const;

	/** Raised by the composite mesh actor list when building the Add Actor menu */
	void OnExtendCompositeMeshAddMenu(FMenuBuilder& MenuBuilder);

	/** Sets the AutoApplyMaterial flag to the specified value */
	void SetAutoApplyMaterial(bool bInAutoApplyMaterial);

	/** Checks if the AutoApplyMaterial flag is set to the specified value */
	bool IsAutoApplyMaterialChecked(bool bInAutoApplyMaterial) const;

	/** Gets the label for the Material selector sub menu in the Add menu */
	FText GetMaterialSubMenuLabel() const;

	/** Creates the Material selector submenu in the Add menu */
	void CreateMaterialSubMenu(FMenuBuilder& InMenuBuilder);

	/** Sets the material type of the auto-applied material */
	void SetAutoAppliedMaterialType(ECompositeMeshAppliedMaterialType InMaterialType);

	/** Gets whether the specified material type is the currently selected type */
	bool IsAutoAppliedMaterialTypeChecked(ECompositeMeshAppliedMaterialType InMaterialType) const;

	/** Gets the label for the custom material menu entry in the Add dropdown menu */
	FText GetCustomMaterialEntryLabel() const;

	/** Gets whether a custom material can currently be auto-applied */
	bool CanApplyCustomMaterial() const;

	/** Raised when a new custom material is selected from the material browser in the Material submenu */
	void OnCustomMaterialSelected(const FAssetData& AssetData);

	/** Raised by the composite mesh actor list when building the context menu */
	void OnExtendCompositeMeshContextMenu(FMenuBuilder& MenuBuilder, TArray<TSoftObjectPtr<AActor>>& SelectedActors);

	/** Raised when the composite mesh actor's entry in the Component section of the context menu is toggled */
	void OnCompositeMeshActorToggled(TSoftObjectPtr<AActor> InActor);

	/** Gets whether the composite mesh actor's entry in the Component section of the context menu can be toggled */
	bool CanToggleCompositeMeshActor(TSoftObjectPtr<AActor> InActor) const;

	/** Gets the check state for the composite mesh actor's entry in the Component section of the context menu */
	bool IsCompositeMeshActorChecked(TSoftObjectPtr<AActor> InActor) const;

	/** Raised when a composite mesh primitive entry is toggled in the Component section of the context menu */
	void OnCompositeMeshPrimitiveToggled(TSoftObjectPtr<AActor> InActor, UPrimitiveComponent* InPrimitive);

	/** Gets whether a composite mesh primitive entry is checked in the Component section of the context menu */
	bool IsCompositeMeshPrimitiveChecked(TSoftObjectPtr<AActor> InActor, UPrimitiveComponent* InPrimitive) const;

	/** Raised when actors are added to the Plate's CompositeMeshes list */
	void OnCompositeMeshActorsAdded(TArray<TSoftObjectPtr<AActor>>& InActors);

	/** Raised when a material should be appplied to an actor in the actor picker list */
	void ApplyMaterialToActor(TSoftObjectPtr<AActor>& InActor, UMaterialInterface* InMaterial);

	/** Creates a composite mesh actor at the appropriate position in the level and adds it to the plate's composite meshes list */
	void CreateCompositeMeshActor();

	/** Creates a composite depth mesh actor at the appropriate position in the level and adds it to the plate's composite meshes list */
	void CreateCompositeDepthMeshActor();

	/** Gets whether a composite mesh actor can be created for this plate layer */
	bool CanCreateCompositeMeshActor() const;

	/** Gets the dropdown menu widget to display when the sky sphere button is pressed */
	TSharedRef<SWidget> GetSkySphereActorMenu();

	/** Spawns a CompositeSkySphereActor in the current level */
	void PlaceCompositeSkySphereActor();

	/** Gets whether a sky sphere actor can be placed */
	bool CanPlaceCompositeSkySphereActor() const;

private:
	TWeakPtr<SCompositeActorPickerTable> CompositeMeshPicker;
	TSharedPtr<IPropertyHandle> TexturePropertyHandle;

	/** The custom material the user has selected to auto-apply to actors added to the composite meshes list */
	FAssetData CustomMaterialAsset = FAssetData();
};
