// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerSceneCaptureCustomization.h"

#include "CompositeRenderTargetResolutionCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Layers/CompositeLayerSceneCapture.h"
#include "UI/SCompositeActorPickerTable.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FCompositeLayerSceneCaptureCustomization"

TSharedRef<IDetailCustomization> FCompositeLayerSceneCaptureCustomization::MakeInstance()
{
	return MakeShared<FCompositeLayerSceneCaptureCustomization>();
}

void FCompositeLayerSceneCaptureCustomization::CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout)
{
	InDetailLayout.RegisterInstancedCustomPropertyTypeLayout(NAME_IntPoint, FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FResolutionTypeCustomization>();
		}),
		MakeShared<FResolutionPropertyIdentifier>());

	static const FName HiddenProperties[] =
	{
		GET_MEMBER_NAME_CHECKED(UCompositeLayerSceneCapture, Actors)
	};

	for (const FName& PropertyName : HiddenProperties)
	{
		InDetailLayout.HideProperty(PropertyName);
	}

	TArray<TWeakObjectPtr<UCompositeLayerSceneCapture>> Objects = InDetailLayout.GetObjectsOfTypeBeingCustomized<UCompositeLayerSceneCapture>();

	IDetailCategoryBuilder& LayerCategory = InDetailLayout.EditCategory("Composite", UCompositeLayerSceneCapture::StaticClass()->GetDisplayNameText());

	AddDefaultLayerProperties(LayerCategory, HiddenProperties);

	if (Objects.Num() == 1 && Objects[0].IsValid())
	{
		UCompositeLayerSceneCapture* SceneCapture = Cast<UCompositeLayerSceneCapture>(Objects[0].Get());

		AddPassesGroup(InDetailLayout, LayerCategory, SceneCapture);

		IDetailGroup& ActorListGroup = LayerCategory.AddGroup("ActorContent", LOCTEXT("ActorListGroupName", "Scene Capture Content"), false, true);
		FCompositeActorPickerListRef ActorListRef(SceneCapture, GET_MEMBER_NAME_CHECKED(UCompositeLayerSceneCapture, Actors), &SceneCapture->Actors, &SceneCapture->SpawnableBindings);

		ActorListGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SNew(SCompositeActorPickerTable, ActorListRef)
			.SceneOutlinerFilters_Lambda([]()
			{
				constexpr bool bExcludeCompositeMeshActors = false;
				return SCompositeActorPickerTable::MakeDefaultSceneOutlinerFilters(bExcludeCompositeMeshActors);
			})
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh))
		];
	}
	else
	{
		AddPassesGroup(InDetailLayout, LayerCategory, /* InLayer */ nullptr);

		// Can't display actor list if multiple layers are selected, so simply put a "Multiple Values" entry in the property list
		LayerCategory.AddCustomRow(LOCTEXT("ActorListGroupName", "Scene Capture Content"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ActorListGroupName", "Scene Capture Content"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
}

#undef LOCTEXT_NAMESPACE
