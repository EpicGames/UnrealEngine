// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerCustomization.h"

class IDetailLayoutBuilder;

/**
 * Customization for the composite scene capture layer, primary for displaying a custom widget for picking actors
 */
class FCompositeLayerSceneCaptureCustomization : public FCompositeLayerCustomization
{
public:
	/** Makes a new instance of the details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin FCompositeLayerCustomization
	virtual void CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout) override;
	//~ End FCompositeLayerCustomization
};
