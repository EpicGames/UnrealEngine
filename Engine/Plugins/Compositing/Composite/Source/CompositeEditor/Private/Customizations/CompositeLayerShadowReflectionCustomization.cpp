// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerShadowReflectionCustomization.h"

#include "CompositeRenderTargetResolutionCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Layers/CompositeLayerShadowReflection.h"
#include "UI/SCompositeActorPickerTable.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FCompositeLayerShadowReflectionCustomization"

TSharedRef<IDetailCustomization> FCompositeLayerShadowReflectionCustomization::MakeInstance()
{
	return MakeShared<FCompositeLayerShadowReflectionCustomization>();
}

void FCompositeLayerShadowReflectionCustomization::CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout)
{
	InDetailLayout.RegisterInstancedCustomPropertyTypeLayout(NAME_IntPoint, FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FResolutionTypeCustomization>();
		}),
		MakeShared<FResolutionPropertyIdentifier>());

	static const FName HiddenProperties[] =
	{
		GET_MEMBER_NAME_CHECKED(UCompositeLayerShadowReflection, Actors)
	};

	for (const FName& PropertyName : HiddenProperties)
	{
		InDetailLayout.HideProperty(PropertyName);
	}

	TArray<TWeakObjectPtr<UCompositeLayerShadowReflection>> Objects = InDetailLayout.GetObjectsOfTypeBeingCustomized<UCompositeLayerShadowReflection>();

	IDetailCategoryBuilder& LayerCategory = InDetailLayout.EditCategory("Composite", UCompositeLayerShadowReflection::StaticClass()->GetDisplayNameText());

	AddDefaultLayerProperties(LayerCategory, HiddenProperties);

	if (Objects.Num() == 1 && Objects[0].IsValid())
	{
		UCompositeLayerShadowReflection* ShadowReflection = Cast<UCompositeLayerShadowReflection>(Objects[0].Get());

		IDetailGroup& ActorListGroup = LayerCategory.AddGroup("ShadowReflectionCatcherContent", LOCTEXT("ActorListGroupName", "Shadow/Reflection Catcher Content"), false, true);
		FCompositeActorPickerListRef ActorListRef(ShadowReflection, GET_MEMBER_NAME_CHECKED(UCompositeLayerShadowReflection, Actors), &ShadowReflection->Actors, &ShadowReflection->SpawnableBindings);

		ActorListGroup.SetToolTip(ActorListRef.GetToolTipText());
		ActorListGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SNew(SCompositeActorPickerTable, ActorListRef)
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh))
		];

		AddPassesGroup(InDetailLayout, LayerCategory, ShadowReflection);
	}
	else
	{
		// Can't display actor list if multiple layers are selected, so simply put a "Multiple Values" entry in the property list
		LayerCategory.AddCustomRow(LOCTEXT("ActorListGroupName", "Shadow/Reflection Catcher Content"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ActorListGroupName", "Shadow/Reflection Catcher Content"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		AddPassesGroup(InDetailLayout, LayerCategory, /* InLayer */ nullptr);
	}
}

#undef LOCTEXT_NAMESPACE
