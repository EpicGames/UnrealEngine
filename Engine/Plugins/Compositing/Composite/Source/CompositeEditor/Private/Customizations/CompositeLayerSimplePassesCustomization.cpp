// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerSimplePassesCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Layers/CompositeLayerBase.h"

TSharedRef<IDetailCustomization> FCompositeLayerSimplePassesCustomization::MakeInstance()
{
	return MakeShared<FCompositeLayerSimplePassesCustomization>();
}

void FCompositeLayerSimplePassesCustomization::CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout)
{
	TArray<TWeakObjectPtr<UCompositeLayerBase>> Objects = InDetailLayout.GetObjectsOfTypeBeingCustomized<UCompositeLayerBase>();

	IDetailCategoryBuilder& LayerCategory = (Objects.Num() > 0 && Objects[0].IsValid())
		? InDetailLayout.EditCategory("Composite", Objects[0]->GetClass()->GetDisplayNameText())
		: InDetailLayout.EditCategory("Composite");

	AddDefaultLayerProperties(LayerCategory);

	UCompositeLayerBase* Layer = (Objects.Num() == 1 && Objects[0].IsValid()) ? Objects[0].Get() : nullptr;
	AddPassesGroup(InDetailLayout, LayerCategory, Layer);
}
