// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeLayerCustomization.h"

class IDetailLayoutBuilder;

/**
 * Customization for layers that only need the standard "Composite" category rename + a Passes group
 * over the inherited UCompositeLayerBase::LayerPasses array. Used by UCompositeLayerMainRender and
 * UCompositeLayerProcessing, which have no other custom widgets.
 */
class FCompositeLayerSimplePassesCustomization : public FCompositeLayerCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin FCompositeLayerCustomization
	virtual void CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout) override;
	//~ End FCompositeLayerCustomization
};
