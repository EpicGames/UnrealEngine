// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerSingleLightShadowCustomization.h"

#include "CompositeRenderTargetResolutionCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Layers/CompositeLayerSingleLightShadow.h"
#include "UI/SCompositeActorPickerTable.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FCompositeLayerSingleLightShadowCustomization"

TSharedRef<IDetailCustomization> FCompositeLayerSingleLightShadowCustomization::MakeInstance()
{
	return MakeShared<FCompositeLayerSingleLightShadowCustomization>();
}

void FCompositeLayerSingleLightShadowCustomization::CustomizeLayerDetails(IDetailLayoutBuilder& InDetailLayout)
{
	InDetailLayout.RegisterInstancedCustomPropertyTypeLayout(NAME_IntPoint, FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FResolutionTypeCustomization>();
		}),
		MakeShared<FResolutionPropertyIdentifier>());

	static const FName HiddenProperties[] =
	{
		GET_MEMBER_NAME_CHECKED(UCompositeLayerSingleLightShadow, ShadowCastingActors)
	};

	for (const FName& PropertyName : HiddenProperties)
	{
		InDetailLayout.HideProperty(PropertyName);
	}

	TArray<TWeakObjectPtr<UCompositeLayerSingleLightShadow>> Objects = InDetailLayout.GetObjectsOfTypeBeingCustomized<UCompositeLayerSingleLightShadow>();

	IDetailCategoryBuilder& LayerCategory = InDetailLayout.EditCategory("Composite", UCompositeLayerSingleLightShadow::StaticClass()->GetDisplayNameText());

	AddDefaultLayerProperties(LayerCategory, HiddenProperties);

	if (Objects.Num() == 1 && Objects[0].IsValid())
	{
		UCompositeLayerSingleLightShadow* SingleLightShadow = Cast<UCompositeLayerSingleLightShadow>(Objects[0].Get());

		IDetailGroup& ActorListGroup = LayerCategory.AddGroup("ActorContent", LOCTEXT("ActorListGroupName", "Shadow Casting Content"), false, true);
		FCompositeActorPickerListRef ActorListRef(SingleLightShadow, GET_MEMBER_NAME_CHECKED(UCompositeLayerSingleLightShadow, ShadowCastingActors), &SingleLightShadow->ShadowCastingActors, &SingleLightShadow->SpawnableBindings);

		ActorListGroup.SetToolTip(ActorListRef.GetToolTipText());
		ActorListGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SNew(SCompositeActorPickerTable, ActorListRef)
			.OnLayoutSizeChanged(FSimpleDelegate::CreateSP(this, &FCompositeLayerCustomization::RequestLayoutRefresh))
		];

		AddPassesGroup(InDetailLayout, LayerCategory, SingleLightShadow);
	}
	else
	{
		// Can't display actor list if multiple layers are selected, so simply put a "Multiple Values" entry in the property list
		LayerCategory.AddCustomRow(LOCTEXT("ActorListGroupName", "Shadow Casting Content"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ActorListGroupName", "Shadow Casting Content"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		AddPassesGroup(InDetailLayout, LayerCategory, /* InLayer */ nullptr);
	}
}

#undef LOCTEXT_NAMESPACE
