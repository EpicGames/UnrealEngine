// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePassBaseCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Passes/CompositePassBase.h"

#define LOCTEXT_NAMESPACE "FCompositePassBaseCustomization"

TSharedRef<IDetailCustomization> FCompositePassBaseCustomization::MakeInstance()
{
	return MakeShared<FCompositePassBaseCustomization>();
}

void FCompositePassBaseCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Rename "Composite" category to the pass class display name (e.g. "Blur", "Color Grading")
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	if (Objects.Num() > 0)
	{
		if (TStrongObjectPtr<UObject> PinnedObject = Objects[0].Pin())
		{
			DetailLayout.EditCategory("Composite", PinnedObject->GetClass()->GetDisplayNameText());
		}
	}

	CustomizeIsEnabledProperty(DetailLayout);
}

void FCompositePassBaseCustomization::CustomizeIsEnabledProperty(IDetailLayoutBuilder& DetailLayout, bool bInPlace)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	// Rename the bIsEnabled property to specify the type of pass being enabled/disabled. If multiple passes are displayed at once, show "Enable All Passes"
	FText ObjectTypeName = FText::GetEmpty();
	if (Objects.Num() == 1)
	{
		if (TStrongObjectPtr<UObject> PinnedObject = Objects[0].Pin())
		{
			ObjectTypeName = PinnedObject->GetClass()->GetDisplayNameText();
		}
	}
	else if (Objects.Num() > 1)
	{
		ObjectTypeName = LOCTEXT("AllPassesLabel", "All Passes");
	}
	
	TSharedPtr<IPropertyHandle> EnabledHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassBase, bIsEnabled), UCompositePassBase::StaticClass());

	IDetailPropertyRow* PropertyRow;
	if (bInPlace)
	{
		PropertyRow = DetailLayout.EditDefaultProperty(EnabledHandle);
	}
	else
	{
		IDetailCategoryBuilder& DefaultCategory = DetailLayout.EditCategory("Composite");
		PropertyRow = &DefaultCategory.AddProperty(EnabledHandle);
	}

	PropertyRow->DisplayName(FText::Format(LOCTEXT("PassEnabledLabelFormat", "Enable {0}"), ObjectTypeName));
}

#undef LOCTEXT_NAMESPACE