// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

/**
 * Base customization for all derivations of UCompositePassBase
 */
class FCompositePassBaseCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

protected:
	/**
	 * Renames the IsEnabled property on the composite pass to reference the property owner's type (e.g. 'Enable Color Grading Pass')
	 * @param DetailLayout The detail layout builder
	 * @param bInPlace Whether the IsEnabled property should remain in its default position or be placed with the rest of the customized properties
	 */
	void CustomizeIsEnabledProperty(IDetailLayoutBuilder& DetailLayout, bool bInPlace = true);
};
