// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePassColorKeyerCustomization.h"

#include "AssetToolsModule.h"
#include "CompositeActor.h"
#include "CompositeCustomizationHelpers.h"
#include "CompositeEditorModule.h"
#include "CompositeEditorStyle.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EditorAssetLibrary.h"
#include "FileHelpers.h"
#include "IContentBrowserSingleton.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "LevelEditorViewport.h"
#include "MediaTexture.h"
#include "NumericPropertyParams.h"
#include "ScopedTransaction.h"
#include "SLevelViewport.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Composite/Private/Components/CompositePlateTexturePreviewComponent.h"
#include "Customizations/ColorStructCustomization.h"
#include "Engine/Texture2D.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Layers/CompositeLayerPlate.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Passes/CompositePassColorKeyer.h"
#include "SMediaProfileSourceTexturePicker.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FCompositePassColorKeyerCustomization"

class FContentBrowserModule;

/** Widget that manages a button with an optional delay, after which its Action delegate is invoked */
class SDelayedActionButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDelayedActionButton) {}
		SLATE_NAMED_SLOT(FArguments, ButtonContent)
		SLATE_NAMED_SLOT(FArguments, CancelContent)
		SLATE_ATTRIBUTE(float, Delay)
		SLATE_EVENT(FSimpleDelegate, OnActivateAction)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Delay = InArgs._Delay;
		OnActivateAction = InArgs._OnActivateAction;

		ChildSlot
		[
			SNew(SOverlay)
		
			+SOverlay::Slot()
			.Padding(0.0f, 2.0f)
			[
				SNew(SButton)
				.OnClicked(this, &SDelayedActionButton::OnButtonClicked)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex(this, &SDelayedActionButton::GetButtonWidgetIndex)

					+SWidgetSwitcher::Slot()
					[
						InArgs._ButtonContent.Widget
					]

					+SWidgetSwitcher::Slot()
					[
						InArgs._CancelContent.Widget
					]
				]
			]
		
			+SOverlay::Slot()
			.Padding(0.0f, 2.0f)
			.VAlign(VAlign_Bottom)
			[
				SNew(SVerticalBox)
				
				+SVerticalBox::Slot()
				.MaxHeight(2.0f)
				[
					SNew(SProgressBar)
					.Percent(this, &SDelayedActionButton::GetDelayProgress)
					.Visibility(this, &SDelayedActionButton::GetProgressBarVisibility)
				]
			]
		];
	}
	
private:
	FReply OnButtonClicked()
	{
		// If there is active delay action timer, clicking on the button should cancel it
		if (DelayedActionTimer.IsValid())
		{
			GEditor->GetTimerManager()->ClearTimer(DelayedActionTimer);
			return FReply::Handled();
		}

		// If the amount to delay is non-negligible, start the timer; otherwise, invoke the action immediately
		const float AmountToDelay = Delay.Get(0.0f);
		if (AmountToDelay > UE_KINDA_SMALL_NUMBER)
		{
			FTimerManagerTimerParameters Args;
			GEditor->GetTimerManager()->SetTimer(DelayedActionTimer, FTimerDelegate::CreateSP(this, &SDelayedActionButton::OnTimerExpired), AmountToDelay, Args);
		}
		else
		{
			OnActivateAction.ExecuteIfBound();
		}

		return FReply::Handled();
	}

	int GetButtonWidgetIndex() const
	{
		return DelayedActionTimer.IsValid() ? 1 : 0;
	}

	TOptional<float> GetDelayProgress() const
	{
		if (DelayedActionTimer.IsValid())
		{
			// Compute the fraction of time elapsed to total delay time to display with the progress bar
			const float TimeElapsed = GEditor->GetTimerManager()->GetTimerElapsed(DelayedActionTimer);
			const float TimeRemaining = GEditor->GetTimerManager()->GetTimerRemaining(DelayedActionTimer);
			return TimeElapsed / (TimeElapsed + TimeRemaining);
		}
		
		return TOptional<float>();
	}

	EVisibility GetProgressBarVisibility() const
	{
		return DelayedActionTimer.IsValid() ? EVisibility::Visible : EVisibility::Hidden;
	}

	void OnTimerExpired()
	{
		// Once the timer has expired, invoke the action delegate and clear the timer
		GEditor->GetTimerManager()->ClearTimer(DelayedActionTimer);
		OnActivateAction.ExecuteIfBound();
	}
	
private:
	/** Attribute to retrieve the amount of time to delay before invoking the action */
	TAttribute<float> Delay;

	/** Invoked once the delay duration has elapsed after clicking the button */
	FSimpleDelegate OnActivateAction;

	/** Handle for the delay timer */
	FTimerHandle DelayedActionTimer;
};

TSharedRef<IDetailCustomization> FCompositePassColorKeyerCustomization::MakeInstance()
{
	return MakeShared<FCompositePassColorKeyerCustomization>();
}

void FCompositePassColorKeyerCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	UMaterial* CaptureCleanPlateMat = LoadObject<UMaterial>(nullptr, TEXT("/Composite/Materials/M_Composite_CleanPlateCapture.M_Composite_CleanPlateCapture"));
	CaptureCleanPlateMID = TStrongObjectPtr<UMaterialInstanceDynamic>(UMaterialInstanceDynamic::Create(CaptureCleanPlateMat, nullptr));
	
	CachedDetailBuilder = DetailBuilder;
	IDetailCustomization::CustomizeDetails(DetailBuilder);
}

/**
 * Custom color struct customization that adds an inline eye dropper button, as well as hooking the opening and closing of the full color picker to displaying
 * the plate texture preview in the level editor
 */
class FKeyerColorStructCustomization : public FColorStructCustomization
{
public:
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		// Find the keyer pass being customized, and store a pointer to it
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = CustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
		if (SelectedObjects.Num() > 0)
		{
			Keyer = Cast<UCompositePassColorKeyer>(SelectedObjects[0]);
		}
		
		FColorStructCustomization::CustomizeHeader(PropertyHandle, HeaderRow, CustomizationUtils);
	}

	virtual void MakeHeaderRow(TSharedRef<IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row) override
	{
		TSharedPtr<SWidget> ColorWidget;
		float ContentWidth = 125.0f;

		TWeakPtr<IPropertyHandle> StructWeakHandlePtr = StructPropertyHandle;
		ColorWidget = CreateColorWidget(StructWeakHandlePtr);
		
		SColorPicker::FEyeDropperButtonArgs ButtonArgs;
		ButtonArgs.OnBegin.BindSP(this, &FKeyerColorStructCustomization::OnBeginEyeDropper);
		ButtonArgs.OnComplete.BindSP(this, &FKeyerColorStructCustomization::OnEndEyeDropper);
		ButtonArgs.OnValueChanged.BindSP(this, &FKeyerColorStructCustomization::OnSetColorFromColorPicker);
		
		TSharedRef<SButton> EyeDropperButton = SColorPicker::CreateStandaloneEyeDropperButton(ButtonArgs);
		EyeDropperButton->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder"));
		
		Row.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(ContentWidth)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				ColorWidget.ToSharedRef()
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				EyeDropperButton
			]
		];
	}

private:
	// Copy and hide the following original FColorStructCustomization methods with slight variations that allow the plate texture preview to be displayed or hidden
	// whenever the color picker is open or closed.

	TSharedRef<SWidget> CreateColorWidget(TWeakPtr<IPropertyHandle> StructWeakHandlePtr)
	{
		return
			SNew(SBox)
			.Padding(FMargin(0,0,4.0f,0.0f))
			.VAlign(VAlign_Center)
			[
				SAssignNew(ColorWidgetBackgroundBorder, SBorder)
				.Padding(1)
				.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RoundedSolidBackground"))
				.BorderBackgroundColor(this, &FKeyerColorStructCustomization::GetColorWidgetBorderColor)
				.VAlign(VAlign_Center)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SAssignNew(ColorPickerParentWidget, SColorBlock)
						.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
						.Color(this, &FKeyerColorStructCustomization::OnGetColorForColorBlock)
						.ShowBackgroundForAlpha(true)
						.AlphaDisplayMode(bIgnoreAlpha ? EColorBlockAlphaDisplayMode::Ignore : EColorBlockAlphaDisplayMode::Separate)
						.OnMouseButtonDown(this, &FKeyerColorStructCustomization::OnMouseButtonDownColorBlock)
						.Size(FVector2D(70.0f, 20.0f))
						.CornerRadius(FVector4(4.0f,4.0f,4.0f,4.0f))
						.IsEnabled(this, &FKeyerColorStructCustomization::IsValueEnabled, StructWeakHandlePtr)
					]
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.Visibility(this, &FKeyerColorStructCustomization::GetMultipleValuesTextVisibility)
						.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.MultipleValuesBackground"))
						.VAlign(VAlign_Center)
						.ForegroundColor(FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").ForegroundColor)
						.Padding(FMargin(12.0f, 2.0f))
						[
							SNew(STextBlock)
							.Text(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				]
			];
	}

	FReply OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
		{
			return FReply::Unhandled();
		}
	
		bool CanShowColorPicker = true;
		if (StructPropertyHandle.IsValid() && StructPropertyHandle->GetProperty() != nullptr)
		{
			CanShowColorPicker = !StructPropertyHandle->IsEditConst();
		}
		if (CanShowColorPicker)
		{
			CreateColorPicker(true /*bUseAlpha*/);
		}

		return FReply::Handled();
	}
	
	void CreateColorPicker(bool bUseAlpha)
	{
		TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("SetColorProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName()));

		GatherSavedPreColorPickerColors();

		FLinearColor InitialColor;
		GetColorAsLinear(InitialColor);

		const bool bRefreshOnlyOnOk = bDontUpdateWhileEditing || StructPropertyHandle->HasMetaData("DontUpdateWhileEditing");
		const bool bOnlyRefreshOnMouseUp = StructPropertyHandle->HasMetaData("OnlyUpdateOnInteractionEnd");

		FColorPickerArgs PickerArgs;
		{
			PickerArgs.bUseAlpha = !bIgnoreAlpha;
			PickerArgs.bOnlyRefreshOnMouseUp = bOnlyRefreshOnMouseUp;
			PickerArgs.bOnlyRefreshOnOk = bRefreshOnlyOnOk;
			PickerArgs.sRGBOverride = sRGBOverride;
			PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
			PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FKeyerColorStructCustomization::OnSetColorFromColorPicker);
			PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FKeyerColorStructCustomization::OnColorPickerCancelled);
			PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &FKeyerColorStructCustomization::OnColorPickerWindowClosed);
			PickerArgs.OnInteractivePickBegin = FSimpleDelegate::CreateSP(this, &FKeyerColorStructCustomization::OnColorPickerInteractiveBegin);
			PickerArgs.OnInteractivePickEnd = FSimpleDelegate::CreateSP(this, &FKeyerColorStructCustomization::OnColorPickerInteractiveEnd);
			PickerArgs.InitialColor = InitialColor;
			PickerArgs.ParentWidget = ColorPickerParentWidget;
			PickerArgs.OptionalOwningDetailsView = ColorPickerParentWidget;
			FWidgetPath ParentWidgetPath;
			if (FSlateApplication::Get().FindPathToWidget(ColorPickerParentWidget.ToSharedRef(), ParentWidgetPath))
			{
				PickerArgs.bOpenAsMenu = FSlateApplication::Get().FindMenuInWidgetPath(ParentWidgetPath).IsValid();
			}
		}

		if (OpenColorPicker(PickerArgs))
		{
			if (TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyer.Pin())
			{
				if (ACompositeActor* CompositeActor = PinnedKeyer->GetTypedOuter<ACompositeActor>())
				{
					CompositeActor->ShowPlatePreview(PinnedKeyer.Get());
				}
			}
		}
	}

	void OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window)
	{
		// Transact only at the end to avoid opening a lingering transaction. Reset value before transacting.
		if (!LastPickerColorString.IsEmpty())
		{
			StructPropertyHandle->SetValueFromFormattedString(LastPickerColorString);
		}

		GEditor->EndTransaction();
		TransactionIndex.Reset();

		if (TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyer.Pin())
		{
			if (ACompositeActor* CompositeActor = PinnedKeyer->GetTypedOuter<ACompositeActor>())
			{
				CompositeActor->HidePlatePreview();
			}
		}
	}

	/** Raised when the inline eye dropper button is clicked */
	void OnBeginEyeDropper()
	{
		TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("SetColorProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName()));
		
		if (TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyer.Pin())
		{
			if (ACompositeActor* CompositeActor = PinnedKeyer->GetTypedOuter<ACompositeActor>())
			{
				CompositeActor->ShowPlatePreview(PinnedKeyer.Get());
			}
		}
	}

	/** Raised when the inline eye dropper is completed, either by a color being selected or cancelled */
	void OnEndEyeDropper(bool bCancelled)
	{
		if (!bCancelled)
		{
			GEditor->EndTransaction();
			TransactionIndex.Reset();
		}
		else
		{
			GEditor->CancelTransaction(0);
			TransactionIndex.Reset();
		}
		
		if (TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyer.Pin())
		{
			if (ACompositeActor* CompositeActor = PinnedKeyer->GetTypedOuter<ACompositeActor>())
			{
				CompositeActor->HidePlatePreview();
			}
		}
	}

private:
	/** Reference to the keyer pass whose keyer color property is being customized */
	TWeakObjectPtr<UCompositePassColorKeyer> Keyer = nullptr;
};

void FCompositePassColorKeyerCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		NAME_LinearColor,
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FKeyerColorStructCustomization>();
		}));
	
	IDetailCategoryBuilder& DefaultCategory = DetailBuilder.EditCategory("Composite", UCompositePassColorKeyer::StaticClass()->GetDisplayNameText());

	// 'Customize' properties that should be above the 'Keyer Settings' custom group
	DefaultCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, DisplayName), UCompositePassBase::StaticClass()));
	CustomizeIsEnabledProperty(DetailBuilder, false);
	DefaultCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, ScreenType)));

	KeyerSourceHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, KeyerSource));

	// Add keyer color and clean plate properties to custom 'Keyer Settings' group
	IDetailGroup& KeyerSettingsGroup = DefaultCategory.AddGroup(TEXT("KeyerSettings"), LOCTEXT("KeyerSettingsGroupName", "Keyer Settings"), false, true);
	KeyerSettingsGroup.AddPropertyRow(KeyerSourceHandle.ToSharedRef());
	KeyerSettingsGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, KeyColor)));

	TSharedRef<IPropertyHandle> CleanPlateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, CleanPlate));
	IDetailPropertyRow& CleanPlateRow = KeyerSettingsGroup.AddPropertyRow(CleanPlateHandle);
	CleanPlateRow.CustomWidget()
	.NameContent()
	[
		CleanPlateHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SMediaProfileSourceTexturePicker)
		.TexturePropertyHandle(CleanPlateHandle)
		.ThumbnailPool(CachedDetailBuilder.IsValid() ? CachedDetailBuilder.Pin()->GetThumbnailPool() : nullptr)
		.AllowedClass(UTexture::StaticClass())
		.OnShouldFilterAsset_Lambda([Handle = CleanPlateHandle](const FAssetData& AssetData)
		{
			return CompositeCustomizationHelpers::ShouldFilterAssetByAllowedClasses(AssetData, Handle);
		})
	];

	PreviewSizeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, PlateTexturePreviewSize));
	KeyerSettingsGroup.AddPropertyRow(PreviewSizeHandle.ToSharedRef());
	PreviewSizeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCompositePassColorKeyerCustomization::OnPreviewSizeChanged));
	
	CountdownHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, CleanPlateCountdown));
	KeyerSettingsGroup.AddPropertyRow(CountdownHandle.ToSharedRef());

	// Add custom 'Capture Clean Plate' button to 'Keyer Settings' group
	KeyerSettingsGroup.AddWidgetRow()
	.WholeRowContent()
	[
		CreateCaptureButtonWidget()
	]
	.Visibility(TAttribute<EVisibility>::CreateSP(this, &FCompositePassColorKeyerCustomization::GetCaptureButtonVisibility));

	RedWeightHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, RedWeight));
	GreenWeightHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, GreenWeight));
	BlueWeightHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, BlueWeight));

	// Create custom group for RGB weight properties, which will include a single row of numeric sliders for each component in the group's header row
	FText ColorWeightGroupName = LOCTEXT("ColorWeightsGroupName", "Weights");
	IDetailGroup& ColorWeightGroup = DefaultCategory.AddGroup(TEXT("ColorWeights"), ColorWeightGroupName);
	ColorWeightGroup.HeaderRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(ColorWeightGroupName)
	]
	.ValueContent()
	[
		CreateRGBNumericEntryWidget()
	];
	
	ColorWeightGroup.AddPropertyRow(RedWeightHandle.ToSharedRef());
	ColorWeightGroup.AddPropertyRow(GreenWeightHandle.ToSharedRef());
	ColorWeightGroup.AddPropertyRow(BlueWeightHandle.ToSharedRef());
}

void FCompositePassColorKeyerCustomization::CaptureMediaTexture()
{
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositePassColorKeyer>> Keyers = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositePassColorKeyer>();
	if (Keyers.Num() != 1)
	{
		return;
	}

	TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyers[0].Pin();
	if (!PinnedKeyer.IsValid())
	{
		return;
	}
	
	UCompositeLayerPlate* ParentPlate = PinnedKeyer->GetTypedOuter<UCompositeLayerPlate>();
	if (!ParentPlate)
	{
		return;
	}

	UMediaTexture* MediaTexture = Cast<UMediaTexture>(ParentPlate->Texture);
	if (!MediaTexture)
	{
		return;
	}

	// Set the media texture as a parameter on the capture material
	CaptureCleanPlateMID->SetTextureParameterValue(TEXT("MediaTexture"), MediaTexture);

	// Draw material to render target using an linear HDR format
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>();
	RenderTarget->InitCustomFormat(MediaTexture->GetWidth(), MediaTexture->GetHeight(), PF_FloatRGBA, true);
	
	UKismetRenderingLibrary::ClearRenderTarget2D(PinnedKeyer.Get(), RenderTarget, FLinearColor::Black);
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(PinnedKeyer.Get(), RenderTarget, CaptureCleanPlateMID.Get());
	RenderTarget->UpdateResourceImmediate(false);

	// Query user for location to save the texture asset
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveCleanPlateDialogTitle", "Save as Clean Plate Texture");
	SaveAssetDialogConfig.DefaultPath = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");
	SaveAssetDialogConfig.DefaultAssetName = TEXT("NewCleanPlate");
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
	SaveAssetDialogConfig.AssetClassNames.Add(UTexture2D::StaticClass()->GetClassPathName());

	FString TextureAssetPath = ContentBrowser.CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	if (!TextureAssetPath.IsEmpty())
	{
		const FString RequestedName = FPaths::GetBaseFilename(TextureAssetPath, true);
		const FString RequestedPath = FPaths::GetPath(TextureAssetPath);
		const FString RequestedFullPath = FPaths::Combine(RequestedPath, RequestedName);

		// If there already exists a texture asset at the user's chosen location, we will overwrite it instead of creating a new one
		UTexture2D* CleanPlate = nullptr;
		if (UEditorAssetLibrary::DoesAssetExist(RequestedFullPath))
		{
			CleanPlate = Cast<UTexture2D>(UEditorAssetLibrary::LoadAsset(RequestedFullPath));
			if (CleanPlate)
			{
				if (!RenderTarget->UpdateTexture(CleanPlate))
				{
					CleanPlate = nullptr;
				}
			}
			else
			{
				// Asset exists, but for some reason isn't a 2D texture. Abort
				UE_LOGF(LogCompositeEditor, Display, "Could not overwrite %ls with clean plate as it is not a valid texture asset", *RequestedFullPath);
				return;
			}
		}
		else
		{
			FString FinalPackageName, FinalAssetName;
			const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateUniqueAssetName(FPaths::Combine(RequestedPath, RequestedName), TEXT(""), FinalPackageName, FinalAssetName);

			UPackage* NewPackage = CreatePackage(*FinalPackageName);
			CleanPlate = RenderTarget->ConstructTexture2D(NewPackage, RequestedName, RF_Public | RF_Standalone | RF_Transactional);
			
			FAssetRegistryModule::AssetCreated(CleanPlate);
		}

		if (!CleanPlate)
		{
			UE_LOGF(LogCompositeEditor, Display, "Failed to create texture asset for clean plate at %ls", *RequestedFullPath);
			return;
		}
		
		CleanPlate->UpdateResource();
		CleanPlate->MarkPackageDirty();

		// Automatically save the newly created or overwritten texture asset, and update the UCompositePassColorKeyer's CleanPlate property with the new asset
		if (UEditorLoadingAndSavingUtils::SavePackages({ CleanPlate->GetPackage() }, true))
		{
			const FScopedTransaction Transaction(LOCTEXT("SetCleanPlateTransaction", "Set Clean Plate"));
			PinnedKeyer->Modify();

			FProperty* CleanPlateProperty = FindFProperty<FProperty>(UCompositePassColorKeyer::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositePassColorKeyer, CleanPlate));
			PinnedKeyer->PreEditChange(CleanPlateProperty);
			PinnedKeyer->CleanPlate = CleanPlate;

			FPropertyChangedEvent ChangedEvent(CleanPlateProperty, EPropertyChangeType::ValueSet, { PinnedKeyer.Get() });
			PinnedKeyer->PostEditChangeProperty(ChangedEvent);
		}
	}
}

void FCompositePassColorKeyerCustomization::OnPreviewSizeChanged()
{
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UCompositePassColorKeyer>> Keyers = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositePassColorKeyer>();
	if (Keyers.Num() != 1)
	{
		return;
	}

	TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyers[0].Pin();
	if (!PinnedKeyer.IsValid())
	{
		return;
	}

	ACompositeActor* CompositeActor = PinnedKeyer->GetTypedOuter<ACompositeActor>();
	if (!CompositeActor)
	{
		return;
	}

	if (CompositeActor->IsPlatePreviewActive())
	{
		float PreviewSize;
		if (PreviewSizeHandle->GetValue(PreviewSize) == FPropertyAccess::Success)
		{
			if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
			{
				ViewportSettings->CameraPreviewSize = FMath::Clamp(PreviewSize, 1.0f, 10.0f);
			}
		}
	}
}

TSharedRef<SWidget> FCompositePassColorKeyerCustomization::CreateRGBNumericEntryWidget()
{
	auto CreateNumericEntryBox = [this](const TSharedPtr<IPropertyHandle>& InPropertyHandle, FLinearColor InLabelColor)
	{
		return SNew(SNumericEntryBox<float>)
			.Value(this, &FCompositePassColorKeyerCustomization::GetRGBWeightValue, InPropertyHandle)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.AllowSpin(true)
			.IsEnabled(this, &FCompositePassColorKeyerCustomization::IsRGBWeightEditable, InPropertyHandle)
			.OnValueChanged(this, &FCompositePassColorKeyerCustomization::ChangeRGBWeightValue, InPropertyHandle)
			.OnValueCommitted(this, &FCompositePassColorKeyerCustomization::CommitRGBWeightValue, InPropertyHandle)
			.MinValue(this, &FCompositePassColorKeyerCustomization::GetRGBWeightMin, InPropertyHandle)
			.MinSliderValue(this, &FCompositePassColorKeyerCustomization::GetRGBWeightMin, InPropertyHandle)
			.MaxValue(this, &FCompositePassColorKeyerCustomization::GetRGBWeightMax, InPropertyHandle)
			.MaxSliderValue(this, &FCompositePassColorKeyerCustomization::GetRGBWeightMax, InPropertyHandle)
			.LabelPadding(FMargin(3.f))
			.LabelLocation(SNumericEntryBox<float>::ELabelLocation::Inside)
			.Label()
			[
				SNumericEntryBox<float>::BuildNarrowColorLabel(InLabelColor)
			];
	};
	
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			CreateNumericEntryBox(RedWeightHandle, AxisDisplayInfo::GetAxisColor(EAxisList::X))
		]

		+SHorizontalBox::Slot()
		[
			CreateNumericEntryBox(GreenWeightHandle, AxisDisplayInfo::GetAxisColor(EAxisList::Y))
		]

		+SHorizontalBox::Slot()
		[
			CreateNumericEntryBox(BlueWeightHandle, AxisDisplayInfo::GetAxisColor(EAxisList::Z))
		];
}

TSharedRef<SWidget> FCompositePassColorKeyerCustomization::CreateCaptureButtonWidget()
{
	return SNew(SDelayedActionButton)
		.IsEnabled(this, &FCompositePassColorKeyerCustomization::IsCaptureButtonEnabled)
		.ButtonContent()
		[
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FCompositeEditorStyle::Get().GetBrush("CompositeEditor.CaptureCleanPlate"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CaptureButtonLabel", "Capture Clean Plate Texture"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]
		.CancelContent()
		[
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FCompositeEditorStyle::Get().GetBrush("CompositeEditor.CancelCapture"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CancelCaptureButtonLabel", "Cancel Capture"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]
		.Delay(this, &FCompositePassColorKeyerCustomization::GetCaptureCountdown)
		.OnActivateAction(this, &FCompositePassColorKeyerCustomization::CaptureMediaTexture);
}

float FCompositePassColorKeyerCustomization::GetCaptureCountdown() const
{
	if (CountdownHandle.IsValid())
	{
		float Countdown;
		FPropertyAccess::Result Result = CountdownHandle->GetValue(Countdown);
		if (Result == FPropertyAccess::Success)
		{
			return Countdown;
		}
	}

	return 0.0;
}

TOptional<float> FCompositePassColorKeyerCustomization::GetRGBWeightValue(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	if (InPropertyHandle.IsValid())
	{
		float Value;
		FPropertyAccess::Result Result = InPropertyHandle->GetValue(Value);
		if (Result == FPropertyAccess::Success)
		{
			return Value;
		}
	}

	return TOptional<float>();
}

void FCompositePassColorKeyerCustomization::ChangeRGBWeightValue(float InNewValue, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (InPropertyHandle.IsValid())
	{
		InPropertyHandle->SetValue(InNewValue, EPropertyValueSetFlags::InteractiveChange);
	}
}

void FCompositePassColorKeyerCustomization::CommitRGBWeightValue(float InNewValue, ETextCommit::Type InCommitType, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (InPropertyHandle.IsValid())
	{
		InPropertyHandle->SetValue(InNewValue);
	}
}

bool FCompositePassColorKeyerCustomization::IsRGBWeightEditable(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	if (InPropertyHandle.IsValid())
	{
		return InPropertyHandle->IsEditable();
	}

	return false;
}

TOptional<float> FCompositePassColorKeyerCustomization::GetRGBWeightMin(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	TNumericPropertyParams<float> NumericPropertyParams(InPropertyHandle->GetProperty(), nullptr);
	return NumericPropertyParams.MinSliderValue;
}

TOptional<float> FCompositePassColorKeyerCustomization::GetRGBWeightMax(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	TNumericPropertyParams<float> NumericPropertyParams(InPropertyHandle->GetProperty(), nullptr);
	return NumericPropertyParams.MaxSliderValue;
}

EVisibility FCompositePassColorKeyerCustomization::GetCaptureButtonVisibility() const
{
	if (KeyerSourceHandle.IsValid() && KeyerSourceHandle->IsValidHandle())
	{
		uint8 KeyerSource;
		if (KeyerSourceHandle->GetValue(KeyerSource) == FPropertyAccess::Success)
		{
			return KeyerSource == (uint8)ECompositeKeyerSource::CleanPlate ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	return EVisibility::Collapsed;
}

bool FCompositePassColorKeyerCustomization::IsCaptureButtonEnabled() const
{
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin();
	if (!DetailBuilder.IsValid())
	{
		return false;
	}

	TArray<TWeakObjectPtr<UCompositePassColorKeyer>> Keyers = DetailBuilder->GetObjectsOfTypeBeingCustomized<UCompositePassColorKeyer>();
	if (Keyers.Num() != 1)
	{
		return false;
	}

	TStrongObjectPtr<UCompositePassColorKeyer> PinnedKeyer = Keyers[0].Pin();
	if (!PinnedKeyer.IsValid())
	{
		return false;
	}
	
	UCompositeLayerPlate* ParentPlate = PinnedKeyer->GetTypedOuter<UCompositeLayerPlate>();
	if (!ParentPlate)
	{
		return false;
	}

	UMediaTexture* MediaTexture = Cast<UMediaTexture>(ParentPlate->Texture);
	if (!MediaTexture)
	{
		return false;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
