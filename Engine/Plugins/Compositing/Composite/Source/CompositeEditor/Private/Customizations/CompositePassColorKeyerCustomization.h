// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBaseCustomization.h"
#include "Layout/Visibility.h"
#include "Types/SlateEnums.h"
#include "UObject/StrongObjectPtr.h"

class UMaterialInstanceDynamic;
class IPropertyHandle;
class SWidget;

/**
 * Customization for UCompositePassColorKeyer, which reorganizes some properties, and adds a 'Capture Clean Plate' button for simple clean plate capturing
 */
class FCompositePassColorKeyerCustomization : public FCompositePassBaseCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void CaptureMediaTexture();

	/** Raised when the PlateTexturePreviewSize property changes values */
	void OnPreviewSizeChanged();
	
	/** Creates the custom RGB single row widget, which contains color-coded numeric spinners for each component */
	TSharedRef<SWidget> CreateRGBNumericEntryWidget();
	
	/** Creates the clean plate capture button widget */
	TSharedRef<SWidget> CreateCaptureButtonWidget();

	/** Gets the capture countdown property value on the UCompositePassColorKeyer objects being customized */
	float GetCaptureCountdown() const;

	/** Gets the RGB weight value for the specified component of the RGB property */
	TOptional<float> GetRGBWeightValue(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Sets the RGB weight value for the specified component of the RGB property */
	void ChangeRGBWeightValue(float InNewValue, TSharedPtr<IPropertyHandle> InPropertyHandle);

	/** Commits the RGB weight value for the specified component of the RGB property */
	void CommitRGBWeightValue(float InNewValue, ETextCommit::Type InCommitType, TSharedPtr<IPropertyHandle> InPropertyHandle);

	/** Gets whether the RGB weight is editable for the  specified component of the RGB property */
	bool IsRGBWeightEditable(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Gets the minimum RGB weight from the property metadata for the  specified component of the RGB property */
	TOptional<float> GetRGBWeightMin(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Gets the maximum RGB weight from the property metadata for the  specified component of the RGB property */
	TOptional<float> GetRGBWeightMax(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Gets the visibility state of the 'Capture Clean Plate' button */
	EVisibility GetCaptureButtonVisibility() const;

	/** Gets whether the 'Capture Clean Plate' button is enabled or not */
	bool IsCaptureButtonEnabled() const;
	
private:
	/** Cached detail layout builder pointer, used to manually refresh the details panel */
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;

	/** Property handle for UCompositePassColorKeyer.KeyerSource property */
	TSharedPtr<IPropertyHandle> KeyerSourceHandle;

	/** Property handle for UCompositePassColorKeyer.PlateTexturePreviewSize property */
	TSharedPtr<IPropertyHandle> PreviewSizeHandle;
	
	/** Property handle for UCompositePassKeyer.CleanPlateCountdown property */
	TSharedPtr<IPropertyHandle> CountdownHandle;

	/** Property handle for R component of the UCompositePassKeyer.RGBWeight property */
	TSharedPtr<IPropertyHandle> RedWeightHandle;

	/** Property handle for G component of the UCompositePassKeyer.RGBWeight property */
	TSharedPtr<IPropertyHandle> GreenWeightHandle;

	/** Property handle for B component of the UCompositePassKeyer.RGBWeight property */
	TSharedPtr<IPropertyHandle> BlueWeightHandle;

	/** Transient dynamic material instance used to capture a clean plate texture from the keyer's media source */
	TStrongObjectPtr<UMaterialInstanceDynamic> CaptureCleanPlateMID;
};
