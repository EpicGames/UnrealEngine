// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePassMaskingCustomization.h"

#include "CompositeCustomizationHelpers.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Texture.h"
#include "Passes/CompositePassMasking.h"
#include "PropertyHandle.h"
#include "SMediaProfileSourceTexturePicker.h"

#define LOCTEXT_NAMESPACE "FCompositePassMaskingCustomization"

TSharedRef<IDetailCustomization> FCompositePassMaskingCustomization::MakeInstance()
{
	return MakeShared<FCompositePassMaskingCustomization>();
}

void FCompositePassMaskingCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	IDetailCustomization::CustomizeDetails(DetailBuilder);
}

void FCompositePassMaskingCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FCompositePassBaseCustomization::CustomizeDetails(DetailLayout);

	IDetailCategoryBuilder& CompositeCategory = DetailLayout.EditCategory("Composite", DetailLayout.GetBaseClass()->GetDisplayNameText());

	const FName MaskTexturePropertyName = GET_MEMBER_NAME_CHECKED(UCompositePassMasking, MaskTexture);

	TSharedRef<IPropertyHandle> MaskTextureHandle = DetailLayout.GetProperty(MaskTexturePropertyName);
	DetailLayout.HideProperty(MaskTextureHandle);

	IDetailPropertyRow& MaskTextureRow = CompositeCategory.AddProperty(MaskTextureHandle);
	CustomizeMaskTexturePropertyRow(MaskTextureHandle, MaskTextureRow);
}

void FCompositePassMaskingCustomization::CustomizeMaskTexturePropertyRow(const TSharedPtr<IPropertyHandle>& InPropertyHandle, IDetailPropertyRow& InPropertyRow)
{
	const TSharedPtr<IDetailLayoutBuilder> PinnedDetailBuilder = CachedDetailBuilder.Pin();
	const TSharedPtr<FAssetThumbnailPool> ThumbnailPool = PinnedDetailBuilder.IsValid() ? PinnedDetailBuilder->GetThumbnailPool() : nullptr;

	InPropertyRow.CustomWidget()
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SMediaProfileSourceTexturePicker)
		.TexturePropertyHandle(InPropertyHandle)
		.ThumbnailPool(ThumbnailPool)
		.AllowedClass(UTexture::StaticClass())
		.OnShouldFilterAsset_Lambda([Handle = InPropertyHandle.ToSharedRef()](const FAssetData& AssetData)
		{
			return CompositeCustomizationHelpers::ShouldFilterAssetByAllowedClasses(AssetData, Handle);
		})
	];
}

#undef LOCTEXT_NAMESPACE
