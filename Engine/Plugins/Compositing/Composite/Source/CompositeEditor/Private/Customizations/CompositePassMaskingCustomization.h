// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositePassBaseCustomization.h"

class IDetailPropertyRow;
class IPropertyHandle;

/**
 * Customization for UCompositePassMasking that swaps the MaskTexture property's value widget for a media profile source texture picker.
 */
class FCompositePassMaskingCustomization : public FCompositePassBaseCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	void CustomizeMaskTexturePropertyRow(const TSharedPtr<IPropertyHandle>& InPropertyHandle, IDetailPropertyRow& InPropertyRow);

	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;
};
