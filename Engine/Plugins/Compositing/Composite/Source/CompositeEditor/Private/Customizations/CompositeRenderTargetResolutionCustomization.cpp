// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeRenderTargetResolutionCustomization.h"

#include "DetailLayoutBuilder.h"
#include "Math/UnrealMathUtility.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "CompositeResolutionCustomization"

TArray<TSharedPtr<FResolutionTypeCustomization::FNamedResolution>> FResolutionTypeCustomization::NamedResolutions =
{
	MakeShared<FNamedResolution>(LOCTEXT("540pResolutionName", "540p (qHD) - 960x540"), FIntPoint(960, 540), LOCTEXT("Resolution540pToolTip", "Quarter High Definition")),
	MakeShared<FNamedResolution>(LOCTEXT("720pResolutionName", "720p (HD) - 1280x720"), FIntPoint(1280, 720), LOCTEXT("Resolution720pToolTip", "High Definition")),
	MakeShared<FNamedResolution>(LOCTEXT("1080pResolutionName", "1080p (FHD) - 1920x1080"), FIntPoint(1920, 1080), LOCTEXT("Resolution1080pToolTip", "Full high definition")),
	MakeShared<FNamedResolution>(LOCTEXT("1440pResolutionName", "1440p (QHD) - 2560x1440"), FIntPoint(2560, 1440), LOCTEXT("Resolution1440pToolTip", "Quad high definition")),
	MakeShared<FNamedResolution>(LOCTEXT("2160pResolutionName", "2160p (4K UHD) - 3840x2160"), FIntPoint(3840, 2160), LOCTEXT("Resolution2160pToolTip", "Ultra high definition (4k)")),
	MakeShared<FNamedResolution>(LOCTEXT("CustomResolutionName", "Custom"), FIntPoint::ZeroValue, FText::GetEmpty())
};

void FResolutionTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	XPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, X));
	YPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, Y));

	FIntPoint InitialResolution = FIntPoint::ZeroValue;
	FPropertyAccess::Result XResult = XPropertyHandle->GetValue(InitialResolution.X);
	FPropertyAccess::Result YResult = YPropertyHandle->GetValue(InitialResolution.Y);

	int32 InitialResolutionIndex = NamedResolutions.IndexOfByPredicate([&InitialResolution](const FNamedResolutionPtr& NamedResolution)
	{
		return NamedResolution->Resolution == InitialResolution;
	});

	if (InitialResolutionIndex == INDEX_NONE)
	{
		InitialResolutionIndex = NamedResolutions.Num() - 1;
	}

	InHeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SAssignNew(ResolutionComboBox, SComboBox<FNamedResolutionPtr>)
		.OptionsSource(&NamedResolutions)
		.OnGenerateWidget_Lambda([](FNamedResolutionPtr InItem)
		{
			return SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(InItem->Name)
				.ToolTipText(InItem->ToolTip);
		})
		.OnSelectionChanged(this, &FResolutionTypeCustomization::SetResolution)
		.InitiallySelectedItem(NamedResolutions[InitialResolutionIndex])
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FResolutionTypeCustomization::GetResolutionText)
		]
	];

	InHeaderRow.OverrideResetToDefault(FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle>)
		{
			return ResolutionComboBox->GetSelectedItem() != NamedResolutions[0];
		}), FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle>)
		{
			ResolutionComboBox->SetSelectedItem(NamedResolutions[0]);
		})));
}

void FResolutionTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InChildBuilder.AddCustomRow(InPropertyHandle->GetPropertyDisplayName())
	.NameContent()
	[
		XPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		XPropertyHandle->CreatePropertyValueWidget()
	]
	.ExtensionContent()
	[
		SNew(SButton)
		.OnClicked_Lambda([this]()
		{
			if (LockedAspectRatio.IsSet())
			{
				LockedAspectRatio.Reset();
			}
			else
			{
				double AspectRatio;
				if (GetAspectRatio(AspectRatio))
				{
					LockedAspectRatio = AspectRatio;
				}
			}

			return FReply::Handled();
		})
		.ContentPadding(FMargin(0, 0, 4, 0))
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image_Lambda([this]
			{
				return LockedAspectRatio.IsSet() ? FAppStyle::GetBrush(TEXT("Icons.Link")) : FAppStyle::GetBrush(TEXT("Icons.Unlink"));
			})
			.ToolTipText_Lambda([this]
			{
				return LockedAspectRatio.IsSet() ?
				FText::Format(LOCTEXT("LockedAspectRatioTooltipFormat", "Click to unlock Aspect Ratio ({0})"), FText::AsNumber(LockedAspectRatio.GetValue())) :
				LOCTEXT("UnlockedAspectRatioTooltip", "Click to lock the Aspect Ratio");
			})
		]
	];

	InChildBuilder.AddCustomRow(InPropertyHandle->GetPropertyDisplayName())
	.NameContent()
	[
		YPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		YPropertyHandle->CreatePropertyValueWidget()
	];

	XPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]
	{
		if (!bSettingNamedResolution)
		{
			ResolutionComboBox->SetSelectedItem(NamedResolutions.Last());
		}

		if (LockedAspectRatio.IsSet())
		{
			const double AspectRatio = LockedAspectRatio.GetValue();
			
			if (AspectRatio > UE_SMALL_NUMBER)
			{
				SetDimensionFromAspectRatio(XPropertyHandle, YPropertyHandle, 1.0 / AspectRatio);
			}
		}
	}));

	YPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]
	{
		if (!bSettingNamedResolution)
		{
			ResolutionComboBox->SetSelectedItem(NamedResolutions.Last());
		}

		if (LockedAspectRatio.IsSet())
		{
			SetDimensionFromAspectRatio(YPropertyHandle, XPropertyHandle, LockedAspectRatio.GetValue());
		}
	}));
}

FText FResolutionTypeCustomization::GetResolutionText() const
{
	FIntPoint Resolution;
	FPropertyAccess::Result XResult = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, X))->GetValue(Resolution.X);
	FPropertyAccess::Result YResult = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, Y))->GetValue(Resolution.Y);

	if (XResult == FPropertyAccess::Fail || YResult == FPropertyAccess::Fail)
	{
		return FText::GetEmpty();
	}

	if (XResult == FPropertyAccess::MultipleValues || YResult == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	FNamedResolutionPtr SelectedResolution = ResolutionComboBox->GetSelectedItem();
	if (SelectedResolution->Resolution != FIntPoint::ZeroValue)
	{
		return SelectedResolution->Name;
	}

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.UseGrouping = false;
	return FText::Format(LOCTEXT("CustomResolutionTextFormat", "Custom - {0}x{1}"),
		FText::AsNumber(Resolution.X, &FormattingOptions),
		FText::AsNumber(Resolution.Y, &FormattingOptions));
}

void FResolutionTypeCustomization::SetResolution(FNamedResolutionPtr InNamedResolution, ESelectInfo::Type SelectInfo)
{
	if (InNamedResolution->Resolution != FIntPoint::ZeroValue)
	{
		TGuardValue<bool> SettingNamedResolution(bSettingNamedResolution, true);
		PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, X))->SetValue(InNamedResolution->Resolution.X);
		PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, Y))->SetValue(InNamedResolution->Resolution.Y);
	}
}

bool FResolutionTypeCustomization::GetAspectRatio(double& OutAspectRatio) const
{
	int32 X = 0;
	int32 Y = 0;

	if (XPropertyHandle->GetValue(X) != FPropertyAccess::Success)
	{
		return false;
	}

	if (YPropertyHandle->GetValue(Y) != FPropertyAccess::Success)
	{
		return false;
	}

	if (Y == 0)
	{
		return false;
	}

	OutAspectRatio = (double)X / (double)Y;
	return true;
}

void FResolutionTypeCustomization::SetDimensionFromAspectRatio(const TSharedPtr<IPropertyHandle>& SrcDimHandle, const TSharedPtr<IPropertyHandle>& DestDimHandle, double AspectRatio)
{
	if (bAspectRatioRecursionGuard)
	{
		return;
	}

	TGuardValue<bool> RecursionGuard(bAspectRatioRecursionGuard, true);

	int32 SrcDim;
	if (SrcDimHandle->GetValue(SrcDim) != FPropertyAccess::Success)
	{
		return;
	}

	const double ScaledSrcDim = SrcDim * AspectRatio;

	// Round to the nearest even number
	const int32 DestDim = FMath::RoundToInt(ScaledSrcDim / 2.0) * 2;

	DestDimHandle->SetValue(DestDim);
}

#undef LOCTEXT_NAMESPACE
