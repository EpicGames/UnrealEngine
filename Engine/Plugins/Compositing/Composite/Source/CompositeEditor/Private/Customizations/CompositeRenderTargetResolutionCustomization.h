// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SComboBox.h"

/**
 * Property type customization for FIntPoint render target resolution properties.
 * Displays a resolution combobox with named presets and an aspect ratio lock.
 */
class FResolutionTypeCustomization : public IPropertyTypeCustomization
{
public:
	struct FNamedResolution
	{
		FText Name = FText::GetEmpty();
		FIntPoint Resolution = FIntPoint::ZeroValue;
		FText ToolTip = FText::GetEmpty();

		FNamedResolution(const FText& InName, const FIntPoint& InResolution, const FText& InToolTip)
			: Name(InName)
			, Resolution(InResolution)
			, ToolTip(InToolTip)
		{ }
	};
	using FNamedResolutionPtr = TSharedPtr<FNamedResolution>;

	static TArray<FNamedResolutionPtr> NamedResolutions;

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

private:
	FText GetResolutionText() const;
	void SetResolution(FNamedResolutionPtr InNamedResolution, ESelectInfo::Type SelectInfo);
	bool GetAspectRatio(double& OutAspectRatio) const;
	void SetDimensionFromAspectRatio(const TSharedPtr<IPropertyHandle>& SrcDimHandle, const TSharedPtr<IPropertyHandle>& DestDimHandle, double AspectRatio);

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> XPropertyHandle;
	TSharedPtr<IPropertyHandle> YPropertyHandle;

	TSharedPtr<SComboBox<FNamedResolutionPtr>> ResolutionComboBox;

	TOptional<double> LockedAspectRatio = TOptional<double>();

	bool bAspectRatioRecursionGuard = false;
	bool bSettingNamedResolution = false;
};

/**
 * Property type identifier that matches FIntPoint properties named "RenderTargetResolution",
 * selecting them for FResolutionTypeCustomization.
 */
class FResolutionPropertyIdentifier : public IPropertyTypeIdentifier
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
	{
		return PropertyHandle.GetProperty()->GetNameCPP() == TEXT("RenderTargetResolution");
	}
};

