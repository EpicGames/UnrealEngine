// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeSkySphereActorCustomization.h"

#include "CompositeCustomizationHelpers.h"
#include "CompositeSkySphereActor.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Texture.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SMediaProfileSourceTexturePicker.h"

#define LOCTEXT_NAMESPACE "FCompositeSkySphereActorCustomization"

TSharedRef<IDetailCustomization> FCompositeSkySphereActorCustomization::MakeInstance()
{
	return MakeShared<FCompositeSkySphereActorCustomization>();
}

void FCompositeSkySphereActorCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	IDetailCustomization::CustomizeDetails(DetailBuilder);
}

void FCompositeSkySphereActorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	CustomizeSections();

	// Texture category: pinned at Important priority.
	IDetailCategoryBuilder& TextureCategory = DetailLayout.EditCategory(
		TEXT("Texture"), FText::GetEmpty(), ECategoryPriority::Important);

	// Texture property — use SMediaProfileSourceTexturePicker with our broader allowed types.
	TSharedRef<IPropertyHandle> TextureHandle =
		DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositeSkySphereActor, Texture));
	TextureCategory.AddProperty(TextureHandle).CustomWidget()
	.NameContent()
	[
		TextureHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SMediaProfileSourceTexturePicker)
		.TexturePropertyHandle(TextureHandle)
		.ThumbnailPool(CachedDetailBuilder.IsValid() ? CachedDetailBuilder.Pin()->GetThumbnailPool() : nullptr)
		.AllowedClass(UTexture::StaticClass())
		.OnShouldFilterAsset_Lambda([Handle = TextureHandle](const FAssetData& AssetData)
		{
			return CompositeCustomizationHelpers::ShouldFilterAssetByAllowedClasses(AssetData, Handle);
		})
	];

	TextureCategory.AddProperty(
		DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositeSkySphereActor, TextureParameterName)));
}

void FCompositeSkySphereActorCustomization::CustomizeSections()
{
	FPropertyEditorModule& PropertyModule =
		FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	const FName ClassName = ACompositeSkySphereActor::StaticClass()->GetFName();

	// Custom pills — registered before the suppress blocks so they appear first.
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("General"), LOCTEXT("SectionGeneral", "General"));
		Section->AddCategory(TEXT("Texture"));
		Section->AddCategory(TEXT("Ray Tracing"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Light"), LOCTEXT("SectionLight", "Light"));
		Section->AddCategory(TEXT("Light"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Materials"), LOCTEXT("SectionMaterials", "Materials"));
		Section->AddCategory(TEXT("Materials"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Ray Tracing"), LOCTEXT("SectionRayTracing", "Ray Tracing"));
		Section->AddCategory(TEXT("Ray Tracing"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Static Mesh"), LOCTEXT("SectionStaticMesh", "Static Mesh"));
		Section->AddCategory(TEXT("Static Mesh"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Texture"), LOCTEXT("SectionTexture", "Texture"));
		Section->AddCategory(TEXT("Texture"));
	}

	// Suppress pills irrelevant for a sky sphere actor.
	// Removing all categories from a section makes its pill disappear.
	// Displaced categories remain accessible via "All".
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Actor"), LOCTEXT("SectionActorSuppress", "Actor"));
		Section->RemoveCategory(TEXT("Actor"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Misc"), LOCTEXT("SectionMiscSuppress", "Misc"));
		Section->RemoveCategory(TEXT("Asset User Data"));
		Section->RemoveCategory(TEXT("Cooking"));
		Section->RemoveCategory(TEXT("Input"));
		Section->RemoveCategory(TEXT("Navigation"));
		Section->RemoveCategory(TEXT("Performance"));
		Section->RemoveCategory(TEXT("Replication"));
		Section->RemoveCategory(TEXT("Tags"));
	}
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Streaming"), LOCTEXT("SectionStreamingSuppress", "Streaming"));
		Section->RemoveCategory(TEXT("Data Layers"));
		Section->RemoveCategory(TEXT("HLOD"));
		Section->RemoveCategory(TEXT("World Partition"));
	}
	// Rendering: individual categories are exposed via dedicated pills (Materials, Ray Tracing).
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Rendering"), LOCTEXT("SectionRenderingSuppress", "Rendering"));
		Section->RemoveCategory(TEXT("Lighting"));
		Section->RemoveCategory(TEXT("Lightmass"));
		Section->RemoveCategory(TEXT("Materials"));
		Section->RemoveCategory(TEXT("Mobile"));
		Section->RemoveCategory(TEXT("Ray Tracing"));
		Section->RemoveCategory(TEXT("Path Tracing"));
		Section->RemoveCategory(TEXT("Rendering"));
		Section->RemoveCategory(TEXT("Texture Streaming"));
		Section->RemoveCategory(TEXT("Virtual Texture"));
		Section->RemoveCategory(TEXT("Material Parameters"));
		Section->RemoveCategory(TEXT("Mesh Painting"));
		// From SkyLightComponent:
		Section->RemoveCategory(TEXT("Light"));
		Section->RemoveCategory(TEXT("Atmosphere and Cloud"));
		Section->RemoveCategory(TEXT("Distance Field Ambient Occlusion"));
	}
	// LOD: sky sphere is a full-screen sphere — LOD settings are not meaningful.
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("LOD"), LOCTEXT("SectionLODSuppress", "LOD"));
		Section->RemoveCategory(TEXT("HLOD"));
		Section->RemoveCategory(TEXT("LOD"));
	}
	// Physics: collision is explicitly disabled in the constructor.
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			ClassName, TEXT("Physics"), LOCTEXT("SectionPhysicsSuppress", "Physics"));
		Section->RemoveCategory(TEXT("Collision"));
		Section->RemoveCategory(TEXT("Physics"));
	}
}

#undef LOCTEXT_NAMESPACE
