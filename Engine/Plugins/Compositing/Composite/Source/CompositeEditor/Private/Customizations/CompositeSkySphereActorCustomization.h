// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailPropertyRow;
class IPropertyHandle;
class SWidget;

/**
 * Details customization for ACompositeSkySphereActor.
 *
 * Registers section filter pills (General | Light | Materials | Ray Tracing | Static Mesh | Texture | All)
 * via FPropertyEditorModule::FindOrCreateSection so they appear in the native UE5 pill row at the top of
 * the Details panel.
 *
 * Also adds a media profile source picker to the Texture property, matching the plate layer UX.
 */
class FCompositeSkySphereActorCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of the details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ End IDetailCustomization Interface

private:
	/** Registers section filter pills via FPropertyEditorModule::FindOrCreateSection. */
	void CustomizeSections();

	/** Cached layout builder for thumbnail pool access. */
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;
};
