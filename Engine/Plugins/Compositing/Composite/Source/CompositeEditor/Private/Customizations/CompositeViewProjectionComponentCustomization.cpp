// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeViewProjectionComponentCustomization.h"

#include "CompositeCustomizationHelpers.h"
#include "Components/CompositeViewProjectionComponent.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "FCompositeViewProjectionComponentCustomization"

TSharedRef<IDetailCustomization> FCompositeViewProjectionComponentCustomization::MakeInstance()
{
	return MakeShared<FCompositeViewProjectionComponentCustomization>();
}

void FCompositeViewProjectionComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TSharedPtr<IPropertyHandle> CameraActorHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCompositeViewProjectionComponent, CameraActor));
	CompositeCustomizationHelpers::CustomizeCameraActorProperty(DetailLayout, CameraActorHandle);
}

#undef LOCTEXT_NAMESPACE
