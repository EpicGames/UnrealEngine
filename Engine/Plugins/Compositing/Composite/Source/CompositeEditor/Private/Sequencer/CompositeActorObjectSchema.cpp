// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeActorObjectSchema.h"

#include "CompositeActor.h"
#include "Layers/CompositeLayerBase.h"
#include "Passes/CompositePassBase.h"

#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "UObject/UnrealType.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "CompositeActorObjectSchema"

namespace UE::Sequencer
{

UObject* FCompositeActorObjectSchema::GetParentObject(UObject* Object) const
{
	if (!Object)
	{
		return nullptr;
	}

	// Component -> Actor (handle components so they still work under our schema)
	if (UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		return Component->GetOwner();
	}

	// Layer -> Actor (check layer first since UCompositeLayerBase inherits UCompositePassBase)
	if (Object->IsA<UCompositeLayerBase>())
	{
		return Object->GetTypedOuter<ACompositeActor>();
	}

	// Pass -> Layer
	if (Object->IsA<UCompositePassBase>())
	{
		return Object->GetTypedOuter<UCompositeLayerBase>();
	}

	return nullptr;
}

FObjectSchemaRelevancy FCompositeActorObjectSchema::GetRelevancy(const UObject* InObject) const
{
	if (!InObject)
	{
		return FObjectSchemaRelevancy();
	}

	// Claim ACompositeActor with its specific class — beats FActorSchema's AActor::StaticClass()
	// because ACompositeActor::IsChildOf(AActor) is true (more derived wins).
	if (InObject->IsA<ACompositeActor>())
	{
		return FObjectSchemaRelevancy(ACompositeActor::StaticClass());
	}

	// Also claim components owned by a CompositeActor so they nest correctly under our actor schema.
	if (const UActorComponent* Component = Cast<UActorComponent>(InObject))
	{
		if (Component->GetOwner() && Component->GetOwner()->IsA<ACompositeActor>())
		{
			return FObjectSchemaRelevancy(UActorComponent::StaticClass(), 1);
		}
	}

	if (InObject->IsA<UCompositeLayerBase>())
	{
		return FObjectSchemaRelevancy(UCompositeLayerBase::StaticClass(), 1);
	}

	if (InObject->IsA<UCompositePassBase>())
	{
		return FObjectSchemaRelevancy(UCompositePassBase::StaticClass(), 1);
	}

	return FObjectSchemaRelevancy();
}

FText FCompositeActorObjectSchema::GetPrettyName(const UObject* Object) const
{
	if (!Object)
	{
		return FText::GetEmpty();
	}

	if (const AActor* Actor = Cast<AActor>(Object))
	{
		return FText::FromString(Actor->GetActorLabel());
	}

	// Note: UCompositeLayerBase inherits from UCompositePassBase now.
	if (const UCompositePassBase* Pass = Cast<UCompositePassBase>(Object))
	{
		const FString Name = Pass->GetDisplayName();
		if (!Name.IsEmpty())
		{
			return FText::FromString(Name);
		}
	}

	return Object->GetClass()->GetDisplayNameText();
}

TSharedPtr<FExtender> FCompositeActorObjectSchema::ExtendObjectBindingMenu(
	TSharedRef<FUICommandList> CommandList,
	TWeakPtr<ISequencer> WeakSequencer,
	TArrayView<UObject* const> ContextSensitiveObjects) const
{
	// Actor context: show layers and components
	TArray<ACompositeActor*> Actors;
	for (UObject* Object : ContextSensitiveObjects)
	{
		if (ACompositeActor* Actor = Cast<ACompositeActor>(Object))
		{
			Actors.Add(Actor);
		}
	}

	if (Actors.Num() > 0)
	{
		TSharedRef<FExtender> Extender = MakeShared<FExtender>();
		Extender->AddMenuExtension(
			SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection,
			EExtensionHook::Before,
			CommandList,
			FMenuExtensionDelegate::CreateRaw(this, &FCompositeActorObjectSchema::HandleActorTrackMenu, WeakSequencer, Actors));
		
		return Extender;
	}

	// Layer context: show passes
	TArray<UCompositeLayerBase*> Layers;
	for (UObject* Object : ContextSensitiveObjects)
	{
		if (UCompositeLayerBase* Layer = Cast<UCompositeLayerBase>(Object))
		{
			Layers.Add(Layer);
		}
	}

	if (Layers.Num() > 0)
	{
		TSharedRef<FExtender> Extender = MakeShared<FExtender>();
		Extender->AddMenuExtension(
			SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection,
			EExtensionHook::Before,
			CommandList,
			FMenuExtensionDelegate::CreateRaw(this, &FCompositeActorObjectSchema::HandleLayerTrackMenu, WeakSequencer, Layers));
		
		return Extender;
	}

	return nullptr;
}

void FCompositeActorObjectSchema::HandleActorTrackMenu(
	FMenuBuilder& MenuBuilder,
	TWeakPtr<ISequencer> WeakSequencer,
	TArray<ACompositeActor*> Actors) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	// --- Composite Layers section ---
	MenuBuilder.BeginSection("CompositeLayers", LOCTEXT("CompositeLayersSection", "Composite Layers"));
	{
		for (ACompositeActor* Actor : Actors)
		{
			TArray<UCompositeLayerBase*> Layers = Actor->GetCompositeLayers();
			for (UCompositeLayerBase* Layer : Layers)
			{
				if (!Layer)
				{
					continue;
				}

				if (Sequencer->GetHandleToObject(Layer, false).IsValid())
				{
					continue;
				}

				FText DisplayName;
				const FString LayerName = Layer->GetDisplayName();
				if (!LayerName.IsEmpty())
				{
					DisplayName = FText::FromString(LayerName);
				}
				else
				{
					DisplayName = Layer->GetClass()->GetDisplayNameText();
				}

				FUIAction Action(FExecuteAction::CreateRaw(
					this, &FCompositeActorObjectSchema::HandleAddLayerExecute,
					TWeakObjectPtr<UCompositeLayerBase>(Layer), WeakSequencer));

				MenuBuilder.AddMenuEntry(
					DisplayName,
					FText::Format(LOCTEXT("AddLayerTooltip", "Add {0} to Sequencer"), DisplayName),
					FSlateIcon(),
					Action);
			}
		}
	}
	MenuBuilder.EndSection();

	// --- Components section (replicate FActorSchema behavior) ---
	MenuBuilder.BeginSection("Components", LOCTEXT("ComponentsSection", "Components"));
	{
		struct FComponentEntry
		{
			FString DisplayString;
			UActorComponent* Component;
		};

		TArray<FComponentEntry> ComponentEntries;

		for (ACompositeActor* Actor : Actors)
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (!Component || Component->IsVisualizationComponent())
				{
					continue;
				}

				if (Sequencer->GetHandleToObject(Component, false).IsValid())
				{
					continue;
				}

				ComponentEntries.Add({ Component->GetName(), Component });
			}
		}

		ComponentEntries.Sort([](const FComponentEntry& A, const FComponentEntry& B)
		{
			return A.DisplayString < B.DisplayString;
		});

		for (const FComponentEntry& Entry : ComponentEntries)
		{
			FUIAction Action(FExecuteAction::CreateRaw(
				this, &FCompositeActorObjectSchema::HandleAddComponentExecute,
				TWeakObjectPtr<UActorComponent>(Entry.Component), WeakSequencer));

			FText Label = FText::FromString(Entry.DisplayString);
			MenuBuilder.AddMenuEntry(
				Label,
				FText::Format(LOCTEXT("AddComponentTooltip", "Add {0}"), Label),
				FSlateIcon(),
				Action);
		}
	}
	MenuBuilder.EndSection();
}

void FCompositeActorObjectSchema::HandleLayerTrackMenu(
	FMenuBuilder& MenuBuilder,
	TWeakPtr<ISequencer> WeakSequencer,
	TArray<UCompositeLayerBase*> Layers) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	MenuBuilder.BeginSection("CompositePasses", LOCTEXT("CompositePassesSection", "Composite Passes"));
	{
		for (UCompositeLayerBase* Layer : Layers)
		{
			for (TFieldIterator<FArrayProperty> PropIt(Layer->GetClass()); PropIt; ++PropIt)
			{
				FArrayProperty* ArrayProp = *PropIt;
				FObjectPropertyBase* InnerProp = CastField<FObjectPropertyBase>(ArrayProp->Inner);

				if (!InnerProp || !InnerProp->PropertyClass->IsChildOf(UCompositePassBase::StaticClass()))
				{
					continue;
				}

				if (InnerProp->PropertyClass->IsChildOf(UCompositeLayerBase::StaticClass()))
				{
					continue;
				}

				FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(Layer));

				for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
				{
					UObject* Element = InnerProp->GetObjectPropertyValue(ArrayHelper.GetElementPtr(Index));
					UCompositePassBase* Pass = Cast<UCompositePassBase>(Element);

					if (!Pass || Pass->IsA<UCompositeLayerBase>())
					{
						continue;
					}

					if (Sequencer->GetHandleToObject(Pass, false).IsValid())
					{
						continue;
					}

					FText DisplayName;
					const FString PassName = Pass->GetDisplayName();
					if (!PassName.IsEmpty())
					{
						DisplayName = FText::FromString(PassName);
					}
					else
					{
						DisplayName = Pass->GetClass()->GetDisplayNameText();
					}

					FUIAction Action(FExecuteAction::CreateRaw(
						this, &FCompositeActorObjectSchema::HandleAddPassExecute,
						TWeakObjectPtr<UCompositePassBase>(Pass), WeakSequencer));

					MenuBuilder.AddMenuEntry(
						DisplayName,
						FText::Format(LOCTEXT("AddPassTooltip", "Add {0} to Sequencer"), DisplayName),
						FSlateIcon(),
						Action);
				}
			}
		}
	}
	MenuBuilder.EndSection();
}

void FCompositeActorObjectSchema::HandleAddLayerExecute(
	TWeakObjectPtr<UCompositeLayerBase> Layer,
	TWeakPtr<ISequencer> WeakSequencer) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer || !Layer.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddCompositeLayer", "Add Composite Layer"));
	Sequencer->GetHandleToObject(Layer.Get());
}

void FCompositeActorObjectSchema::HandleAddPassExecute(
	TWeakObjectPtr<UCompositePassBase> Pass,
	TWeakPtr<ISequencer> WeakSequencer) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer || !Pass.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddCompositePass", "Add Composite Pass"));
	Sequencer->GetHandleToObject(Pass.Get());
}

void FCompositeActorObjectSchema::HandleAddComponentExecute(
	TWeakObjectPtr<UActorComponent> Component,
	TWeakPtr<ISequencer> WeakSequencer) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer || !Component.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddComponent", "Add Component"));
	Sequencer->GetHandleToObject(Component.Get());
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
