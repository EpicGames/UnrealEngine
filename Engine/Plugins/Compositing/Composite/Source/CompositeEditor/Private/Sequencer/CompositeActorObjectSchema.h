// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerObjectSchema.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ACompositeActor;
class FMenuBuilder;
class UActorComponent;
class UCompositeLayerBase;
class UCompositePassBase;

namespace UE::Sequencer
{

/**
 * Object schema that exposes ACompositeActor layers and their passes
 * as possessable sub-objects in the Sequencer outliner.
 *
 * Handles two nesting levels:
 *   Actor -> Layers  (UCompositeLayerBase children of ACompositeActor)
 *   Layer -> Passes  (UCompositePassBase children of UCompositeLayerBase)
 */
class FCompositeActorObjectSchema : public IObjectSchema
{
public:
	//~ Begin IObjectSchema
	virtual UObject* GetParentObject(UObject* Object) const override;
	virtual FObjectSchemaRelevancy GetRelevancy(const UObject* InObject) const override;
	virtual TSharedPtr<FExtender> ExtendObjectBindingMenu(TSharedRef<FUICommandList> CommandList, TWeakPtr<ISequencer> WeakSequencer, TArrayView<UObject* const> ContextSensitiveObjects) const override;
	virtual FText GetPrettyName(const UObject* Object) const override;
	//~ End IObjectSchema

private:
	/** Populates the actor-level track menu with composite layers and actor components not yet bound in Sequencer. */
	void HandleActorTrackMenu(FMenuBuilder& MenuBuilder, TWeakPtr<ISequencer> WeakSequencer, TArray<ACompositeActor*> Actors) const;

	/** Populates the layer-level track menu with composite passes (from all pass arrays) not yet bound in Sequencer. */
	void HandleLayerTrackMenu(FMenuBuilder& MenuBuilder, TWeakPtr<ISequencer> WeakSequencer, TArray<UCompositeLayerBase*> Layers) const;

	/** Creates a possessable binding for a composite layer in a scoped undo transaction. */
	void HandleAddLayerExecute(TWeakObjectPtr<UCompositeLayerBase> Layer, TWeakPtr<ISequencer> WeakSequencer) const;

	/** Creates a possessable binding for a composite pass in a scoped undo transaction. */
	void HandleAddPassExecute(TWeakObjectPtr<UCompositePassBase> Pass, TWeakPtr<ISequencer> WeakSequencer) const;

	/** Creates a possessable binding for an actor component in a scoped undo transaction. */
	void HandleAddComponentExecute(TWeakObjectPtr<UActorComponent> Component, TWeakPtr<ISequencer> WeakSequencer) const;
};

} // namespace UE::Sequencer
