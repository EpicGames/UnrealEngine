// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeDetailKeyframeHandler.h"

#include "PropertyHandle.h"
#include "ISequencer.h"
#include "LevelEditorSequencerIntegration.h"
#include "MovieScene.h"
#include "PropertyPath.h"

TArray<TWeakPtr<ISequencer>> FCompositeDetailKeyframeHandler::GetSequencers() const
{
	return FLevelEditorSequencerIntegration::Get().GetSequencers();
}

bool FCompositeDetailKeyframeHandler::IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const
{
	FCanKeyPropertyParams CanKeyPropertyParams(InObjectClass, PropertyHandle);

	for (const TWeakPtr<ISequencer>& WeakSequencer : GetSequencers())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid() && Sequencer->CanKeyProperty(CanKeyPropertyParams) && !Sequencer->IsReadOnly())
		{
			return true;
		}
	}

	return false;
}

bool FCompositeDetailKeyframeHandler::IsPropertyKeyingEnabled() const
{
	for (const TWeakPtr<ISequencer>& WeakSequencer : GetSequencers())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly)
		{
			return true;
		}
	}
	
	return false;
}

bool FCompositeDetailKeyframeHandler::IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject* ParentObject) const
{
	for (const TWeakPtr<ISequencer>& WeakSequencer : GetSequencers())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence())
		{
			constexpr bool bCreateHandleIfMissing = false;
			FGuid ObjectHandle = Sequencer->GetHandleToObject(ParentObject, bCreateHandleIfMissing);
			if (ObjectHandle.IsValid())
			{
				UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

				/**
				 * Build the dot-separated property path that Sequencer uses for track lookup.
				 * Walk parent handles to produce the full path, e.g. "StructName.MemberName" for nested properties.
				 */
				FProperty* LeafProperty = PropertyHandle.GetProperty();
				if (!LeafProperty)
				{
					continue;
				}

				TArray<FProperty*> PropertyChain;
				PropertyChain.Add(LeafProperty);
				for (TSharedPtr<IPropertyHandle> Current = PropertyHandle.GetParentHandle(); Current.IsValid() && Current->GetProperty(); Current = Current->GetParentHandle())
				{
					PropertyChain.Add(Current->GetProperty());
				}

				TSharedRef<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();
				for (int32 Index = PropertyChain.Num() - 1; Index >= 0; --Index)
				{
					PropertyPath->AddProperty(FPropertyInfo(PropertyChain[Index]));
				}
				FName PropertyName(*PropertyPath->ToString(TEXT(".")));

				if (MovieScene->FindTrack(UMovieSceneTrack::StaticClass(), ObjectHandle, PropertyName) != nullptr)
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

void FCompositeDetailKeyframeHandler::OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle)
{
	TArray<UObject*> Objects;
	KeyedPropertyHandle.GetOuterObjects(Objects);

	TArray<UObject*> EachObject;
	EachObject.SetNum(1);
	for (const TWeakPtr<ISequencer>& WeakSequencer : GetSequencers())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid())
		{
			for (UObject* Object : Objects)
			{
				EachObject[0] = Object;
				FKeyPropertyParams KeyPropertyParams(EachObject, KeyedPropertyHandle, ESequencerKeyMode::ManualKeyForced);
				Sequencer->KeyProperty(KeyPropertyParams);
			}
		}
	}
}

EPropertyKeyedStatus FCompositeDetailKeyframeHandler::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;
	for (const TWeakPtr<ISequencer>& WeakSequencer : GetSequencers())
	{
		if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			EPropertyKeyedStatus NewKeyedStatus = Sequencer->GetPropertyKeyedStatus(PropertyHandle);
			KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);
		}
	}
	
	return KeyedStatus;
}
