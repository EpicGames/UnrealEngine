// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailKeyframeHandler.h"

class ISequencer;

/**
 * Keyframe handler for the Composite editor panel's DetailsView.
 * Delegates to active Sequencer instances obtained from FLevelEditorSequencerIntegration.
 * Stateless — queries live sequencer state on each call so it handles Sequencer open/close automatically.
 */
class FCompositeDetailKeyframeHandler : public IDetailKeyframeHandler
{
public:
	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override;
	virtual bool IsPropertyKeyingEnabled() const override;
	virtual bool IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject* ParentObject) const override;
	virtual void OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle) override;
	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const override;

private:
	TArray<TWeakPtr<ISequencer>> GetSequencers() const;
};
