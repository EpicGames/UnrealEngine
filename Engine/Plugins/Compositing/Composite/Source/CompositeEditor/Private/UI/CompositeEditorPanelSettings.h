// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "CompositeEditorPanelSettings.generated.h"

UENUM()
enum class ECompositeMeshAppliedMaterialType : uint8
{
	LitMaskedMaterial,
	UnlitAlphaMaterial,
	CustomMaterial
};

/** Mode controlling the composite mesh visualization overlay driven by the MPC_Composite "MeshVisualization" scalar. */
UENUM()
enum class ECompositeMeshVisualizationMode : uint8
{
	/** Visualization overlay is off. */
	Disabled,

	/** Overlay is shown only while the active composite actor's camera is not being piloted. */
	EnabledWhenNotPiloting,

	/** Overlay is always shown. */
	Enabled,
};

/** Settings for the Composure panel */
UCLASS(MinimalAPI, Config=EditorPerProjectUserSettings)
class UCompositeEditorPanelSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Whether the selected material is applied to actors that are added to the composite meshes list in the Plate layer */
	UPROPERTY(Config)
	bool bAutoApplyCompositeMeshMaterial = true;

	/** The material type to apply to actors that are added to the composite meshes list in the Plate layer */
	UPROPERTY(Config)
	ECompositeMeshAppliedMaterialType CompositeMeshMaterialType = ECompositeMeshAppliedMaterialType::LitMaskedMaterial;

	/** List of columns that have been hidden by the user in the composite meshes list in the Plate layer */
	UPROPERTY(Config)
	TArray<FName> CompositeMeshTableHiddenColumns = TArray<FName>();

	/** Indicates that actor pickers in the Composure panel will sync their selection with the editor selection */
	UPROPERTY(Config)
	bool bSyncActorSelection = true;

	/** Composite mesh visualization mode driving the MPC_Composite.MeshVisualization scalar. */
	UPROPERTY(Config)
	ECompositeMeshVisualizationMode MeshVisualizationMode = ECompositeMeshVisualizationMode::Disabled;
};
