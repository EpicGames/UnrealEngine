// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeLayerPassListOwner.h"

#include "Layers/CompositeLayerBase.h"
#include "Passes/CompositePassBase.h"
#include "PropertyEditorUtils.h"
#include "UObject/Package.h"

FCompositeLayerPassListOwner::FCompositeLayerPassListOwner(const TWeakObjectPtr<UCompositeLayerBase>& InLayer)
	: Layer(InLayer)
{
}

bool FCompositeLayerPassListOwner::IsObjectValid()
{
	return Layer.IsValid();
}

TStrongObjectPtr<UObject> FCompositeLayerPassListOwner::GetObject()
{
	return Layer.Pin();
}

bool FCompositeLayerPassListOwner::IsPassListPropertyName(const FName& InPropertyName)
{
	return InPropertyName == GET_MEMBER_NAME_CHECKED(UCompositeLayerBase, LayerPasses);
}

bool FCompositeLayerPassListOwner::IsValidPassIndex(int32 InGroupIndex, int32 InPassIndex)
{
	if (InGroupIndex != INDEX_NONE) { return false; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	return Pinned.IsValid() && Pinned->LayerPasses.IsValidIndex(InPassIndex);
}

UCompositePassBase* FCompositeLayerPassListOwner::GetPass(int32 InGroupIndex, int32 InPassIndex)
{
	if (InGroupIndex != INDEX_NONE) { return nullptr; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	if (!Pinned.IsValid() || !Pinned->LayerPasses.IsValidIndex(InPassIndex)) { return nullptr; }
	return Pinned->LayerPasses[InPassIndex];
}

TArray<TObjectPtr<UCompositePassBase>>& FCompositeLayerPassListOwner::GetPassesForGroup(int32 InGroupIndex)
{
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	check(Pinned.IsValid());
	return Pinned->LayerPasses;
}

bool FCompositeLayerPassListOwner::CanAddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) const
{
	if (InGroupIndex != INDEX_NONE) { return false; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	if (!Pinned.IsValid()) { return false; }

	FProperty* Prop = GetPassListProperty();
	TArray<const UClass*> Allowed, Disallowed;
	PropertyEditorUtils::GetAllowedAndDisallowedClasses({ Pinned.Get() }, *Prop, Allowed, Disallowed, false);
	return (Allowed.IsEmpty() || Allowed.Contains(InPassClass)) && !Disallowed.Contains(InPassClass);
}

int32 FCompositeLayerPassListOwner::AddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex)
{
	if (InGroupIndex != INDEX_NONE) { return INDEX_NONE; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	if (!Pinned.IsValid()) { return INDEX_NONE; }

	const int32 ClampedIndex = FMath::Clamp(InPassIndex, 0, Pinned->LayerPasses.Num());

	Pinned->Modify();
	UCompositePassBase* NewPass = NewObject<UCompositePassBase>(Pinned.Get(), InPassClass, NAME_None, RF_Transactional);

	FProperty* Prop = GetPassListProperty();
	Pinned->PreEditChange(Prop);
	int32 NewIndex = Pinned->LayerPasses.Insert(NewPass, ClampedIndex);
	FPropertyChangedEvent Event(Prop, EPropertyChangeType::ArrayAdd);
	Pinned->PostEditChangeProperty(Event);
	return NewIndex;
}

TArray<int32> FCompositeLayerPassListOwner::CopyPasses(const TArray<UCompositePassBase*>& InPassesToCopy, int32 InGroupIndex, int32 InPassIndex)
{
	TArray<int32> Out;
	if (InGroupIndex != INDEX_NONE) { return Out; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	if (!Pinned.IsValid()) { return Out; }

	Pinned->Modify();

	FProperty* Prop = GetPassListProperty();
	Pinned->PreEditChange(Prop);

	int32 Dest = FMath::Clamp(InPassIndex, 0, Pinned->LayerPasses.Num());
	for (UCompositePassBase* Pass : InPassesToCopy)
	{
		UCompositePassBase* Copy = DuplicateObject(Pass, Pinned.Get());
		Copy->SetFlags(RF_Transactional);
		Out.Add(Pinned->LayerPasses.Insert(Copy, Dest));
		++Dest;
	}

	FPropertyChangedEvent Event(Prop, EPropertyChangeType::ArrayAdd);
	Pinned->PostEditChangeProperty(Event);
	return Out;
}

void FCompositeLayerPassListOwner::MovePass(int32 InSourceGroupIndex, int32 InSourcePassIndex, int32 InDestGroupIndex, int32 InDestPassIndex)
{
	if (InSourceGroupIndex != INDEX_NONE || InDestGroupIndex != INDEX_NONE) { return; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	if (!Pinned.IsValid()) { return; }
	if (!Pinned->LayerPasses.IsValidIndex(InSourcePassIndex)) { return; }

	Pinned->Modify();
	FProperty* Prop = GetPassListProperty();
	UCompositePassBase* Pass = Pinned->LayerPasses[InSourcePassIndex];

	// InDestPassIndex is a post-removal index: the caller (SCompositePassTree) adjusts for the shift caused by RemoveAt.
	Pinned->PreEditChange(Prop);
	Pinned->LayerPasses.RemoveAt(InSourcePassIndex);
	Pinned->LayerPasses.Insert(Pass, FMath::Clamp(InDestPassIndex, 0, Pinned->LayerPasses.Num()));

	FPropertyChangedEvent Event(Prop, EPropertyChangeType::ArrayMove);
	Pinned->PostEditChangeProperty(Event);
}

void FCompositeLayerPassListOwner::RemovePasses(int32 InGroupIndex, const TArray<int32>& InPassIndices)
{
	if (InGroupIndex != INDEX_NONE) { return; }
	TStrongObjectPtr<UCompositeLayerBase> Pinned = Layer.Pin();
	if (!Pinned.IsValid()) { return; }

	Pinned->Modify();

	TArray<int32> Sorted = InPassIndices;
	Sorted.Sort();

	FProperty* Prop = GetPassListProperty();
	Pinned->PreEditChange(Prop);

	for (int32 i = Sorted.Num() - 1; i >= 0; --i)
	{
		if (!Pinned->LayerPasses.IsValidIndex(Sorted[i])) { continue; }
		TObjectPtr<UCompositePassBase>& PassToRemove = Pinned->LayerPasses[Sorted[i]];
		if (IsValid(PassToRemove))
		{
			PassToRemove->Modify();
			PassToRemove->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		}
		Pinned->LayerPasses.RemoveAt(Sorted[i], EAllowShrinking::No);
	}

	Pinned->LayerPasses.Shrink();
	FPropertyChangedEvent Event(Prop, EPropertyChangeType::ArrayRemove);
	Pinned->PostEditChangeProperty(Event);
}

FProperty* FCompositeLayerPassListOwner::GetPassListProperty()
{
	return FindFProperty<FProperty>(UCompositeLayerBase::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeLayerBase, LayerPasses));
}
