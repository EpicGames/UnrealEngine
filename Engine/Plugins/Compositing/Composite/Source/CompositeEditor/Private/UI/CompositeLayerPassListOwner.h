// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCompositePassTree.h"
#include "UObject/WeakObjectPtr.h"

class UCompositeLayerBase;
class UCompositePassBase;

/**
 * ICompositePassListOwner for any layer with a single LayerPasses array on UCompositeLayerBase.
 * Plate uses its own multi-group owner (FCompositePlatePassListOwner) since it has Media/Scene/Layer arrays.
 */
class FCompositeLayerPassListOwner : public ICompositePassListOwner
{
public:
	explicit FCompositeLayerPassListOwner(const TWeakObjectPtr<UCompositeLayerBase>& InLayer);

	virtual bool IsObjectValid() override;
	virtual TStrongObjectPtr<UObject> GetObject() override;
	virtual bool IsPassListPropertyName(const FName& InPropertyName) override;
	virtual bool IsValidPassIndex(int32 InGroupIndex, int32 InPassIndex) override;
	virtual UCompositePassBase* GetPass(int32 InGroupIndex, int32 InPassIndex) override;
	virtual int32 GetNumGroups() const override { return INDEX_NONE; }
	virtual TArray<TObjectPtr<UCompositePassBase>>& GetPassesForGroup(int32 InGroupIndex) override;

	virtual FString GetGroupFilterString(int32 InGroupIndex) override { return TEXT(""); }
	virtual const FSlateBrush* GetGroupIcon(int32 InGroupIndex) override { return nullptr; }
	virtual FText GetGroupDisplayName(int32 InGroupIndex) override { return FText::GetEmpty(); }
	virtual FGroupFilterConfig GetGroupFilterConfig(int32 InGroupIndex) override { return FGroupFilterConfig(); }
	virtual int32 GetDefaultGroupForNewPass(const UClass* InPassClass) const override { return INDEX_NONE; }

	virtual bool CanAddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) const override;
	virtual int32 AddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) override;
	virtual TArray<int32> CopyPasses(const TArray<UCompositePassBase*>& InPassesToCopy, int32 InGroupIndex, int32 InPassIndex) override;
	virtual void MovePass(int32 InSourceGroupIndex, int32 InSourcePassIndex, int32 InDestGroupIndex, int32 InDestPassIndex) override;
	virtual void RemovePasses(int32 InGroupIndex, const TArray<int32>& InPassIndices) override;

private:
	static FProperty* GetPassListProperty();

	TWeakObjectPtr<UCompositeLayerBase> Layer;
};
