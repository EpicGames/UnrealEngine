// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePlatePassListOwner.h"

#include "CompositeEditorStyle.h"
#include "PropertyEditorUtils.h"
#include "Layers/CompositeLayerBase.h"
#include "Layers/CompositeLayerPlate.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "FCompositePlatePassListOwner"

bool FCompositePlatePassListOwner::IsObjectValid()
{
	return Plate.IsValid();
}

TStrongObjectPtr<UObject> FCompositePlatePassListOwner::GetObject()
{
	return Plate.Pin();
}

bool FCompositePlatePassListOwner::IsPassListPropertyName(const FName& InPropertyName)
{
	return
		InPropertyName == GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, MediaPasses) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, LayerPasses);
}

bool FCompositePlatePassListOwner::IsValidPassIndex(int32 InGroupIndex, int32 InPassIndex)
{
	if (!Plate.IsValid())
	{
		return false;
	}
	
	TArray<TObjectPtr<UCompositePassBase>>& PassList = GetPassList((EPassType)InGroupIndex);
	return PassList.IsValidIndex(InPassIndex);
}

UCompositePassBase* FCompositePlatePassListOwner::GetPass(int32 InGroupIndex, int32 InPassIndex)
{
	if (!Plate.IsValid())
	{
		return nullptr;
	}
	
	TArray<TObjectPtr<UCompositePassBase>>& PassList = GetPassList((EPassType)InGroupIndex);

	if (!PassList.IsValidIndex(InPassIndex))
	{
		return nullptr;
	}
	
	return PassList[InPassIndex];
}

TArray<TObjectPtr<UCompositePassBase>>& FCompositePlatePassListOwner::GetPassesForGroup(int32 InGroupIndex)
{
	return GetPassList((EPassType)InGroupIndex);
}

FString FCompositePlatePassListOwner::GetGroupFilterString(int32 InGroupIndex)
{
	static FString PassTypeFilterStrings[] =
	{
		TEXT("Layer Passes"),
		TEXT("Media Passes")
	};

	return PassTypeFilterStrings[InGroupIndex];
}

const FSlateBrush* FCompositePlatePassListOwner::GetGroupIcon(int32 InGroupIndex)
{
	static const FSlateBrush* PassTypeIcons[] =
	{
		FCompositeEditorStyle::Get().GetBrush("CompositeEditor.Passes.Layer"),
		FCompositeEditorStyle::Get().GetBrush("CompositeEditor.Passes.Media")
	};
		
	return PassTypeIcons[InGroupIndex];
}

FText FCompositePlatePassListOwner::GetGroupDisplayName(int32 InGroupIndex)
{
	static FText PassTypeLabels[] =
	{
		LOCTEXT("LayerPassTypeLabel", "Layer Passes"),
		LOCTEXT("MediaPassTypeLabel", "Media Passes"),
	};
		
	return PassTypeLabels[InGroupIndex];
}

ICompositePassListOwner::FGroupFilterConfig FCompositePlatePassListOwner::GetGroupFilterConfig(int32 InGroupIndex)
{
	static TArray<FGroupFilterConfig> PassTypeFilterConfigs =
	{
		{ TEXT("LayerPassesFilter"), LOCTEXT("LayerPassesFilterName", "Layer Passes"),  LOCTEXT("LayerPassesFilterToolTip", "Only show layer passes")  },
		{ TEXT("MediaPassesFilter"), LOCTEXT("MediaPassesFilterName", "Media Passes"),  LOCTEXT("MediaPassesFilterToolTip", "Only show media passes") }
	};

	return PassTypeFilterConfigs[InGroupIndex];
}

int32 FCompositePlatePassListOwner::GetDefaultGroupForNewPass(const UClass* InPassClass) const
{
	return (int32)EPassType::Media;
}

bool FCompositePlatePassListOwner::CanAddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) const
{
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = Plate.Pin();
	if (!PinnedPlate.IsValid())
	{
		return false;
	}

	const EPassType DestPassType = (EPassType)InGroupIndex;
	FProperty* PassListProperty = GetPassListProperty(DestPassType);

	TArray<const UClass*> AllowedClassFilters;
	TArray<const UClass*> DisallowedClassFilters;
	PropertyEditorUtils::GetAllowedAndDisallowedClasses({ PinnedPlate.Get() }, *PassListProperty, AllowedClassFilters, DisallowedClassFilters, false);

	bool bIsAllowed = AllowedClassFilters.IsEmpty() || AllowedClassFilters.Contains(InPassClass);
	bool bIsDisallowed = DisallowedClassFilters.Contains(InPassClass);
	
	return bIsAllowed && !bIsDisallowed;
}

int32 FCompositePlatePassListOwner::AddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex)
{
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = Plate.Pin();
	if (!PinnedPlate.IsValid())
	{
		return INDEX_NONE;
	}
	
	PinnedPlate->Modify();

	const EPassType DestPassType = (EPassType)InGroupIndex;
	
	UCompositePassBase* NewPass = NewObject<UCompositePassBase>(PinnedPlate.Get(), InPassClass, NAME_None, RF_Transactional);
	TArray<TObjectPtr<UCompositePassBase>>& PassList = GetPassList(DestPassType);

	FProperty* PassListProperty = GetPassListProperty(DestPassType);
	PinnedPlate->PreEditChange(PassListProperty);
	
	int32 NewPassIndex = PassList.Insert(NewPass, InPassIndex);

	FPropertyChangedEvent PropertyChangedEvent(PassListProperty, EPropertyChangeType::ArrayAdd);
	PinnedPlate->PostEditChangeProperty(PropertyChangedEvent);

	return NewPassIndex;
}

TArray<int32> FCompositePlatePassListOwner::CopyPasses(const TArray<UCompositePassBase*>& InPassesToCopy, int32 InGroupIndex, int32 InPassIndex)
{
	TArray<int32> OutNewPassIndices;
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = Plate.Pin();
	if (!PinnedPlate.IsValid())
	{
		return OutNewPassIndices;
	}
	
	const EPassType PassTypeToCopyTo = (EPassType)InGroupIndex;
	TArray<TObjectPtr<UCompositePassBase>>& PassList = GetPassList(PassTypeToCopyTo);
	
	FProperty* PassListProperty = GetPassListProperty(PassTypeToCopyTo);
	PinnedPlate->PreEditChange(PassListProperty);

	int32 IndexToCopyTo = InPassIndex;
	for (UCompositePassBase* PassToPaste : InPassesToCopy)
	{
		UCompositePassBase* NewPass = DuplicateObject(PassToPaste, PinnedPlate.Get());
			
		const int32 NewPassIndex = PassList.Insert(NewPass, IndexToCopyTo);
		OutNewPassIndices.Add(NewPassIndex);
		++IndexToCopyTo;
	}

	FPropertyChangedEvent PropertyChangedEvent(PassListProperty, EPropertyChangeType::ArrayAdd);
	PinnedPlate->PostEditChangeProperty(PropertyChangedEvent);

	return OutNewPassIndices;
}

void FCompositePlatePassListOwner::MovePass(int32 InSourceGroupIndex, int32 InSourcePassIndex, int32 InDestGroupIndex, int32 InDestPassIndex)
{
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = Plate.Pin();
	if (!PinnedPlate.IsValid())
	{
		return;
	}

	PinnedPlate->Modify();

	const EPassType SourcePassType = (EPassType)InSourceGroupIndex;
	const EPassType DestPassType = (EPassType)InDestGroupIndex;
	
	TArray<TObjectPtr<UCompositePassBase>>& SourcePassList = GetPassList(SourcePassType);
	TArray<TObjectPtr<UCompositePassBase>>& DestPassList = GetPassList(DestPassType);

	FProperty* SourcePassListProperty = GetPassListProperty(SourcePassType);
	FProperty* DestPassListProperty = GetPassListProperty(DestPassType);
	
	UCompositePassBase* Pass = SourcePassList[InSourcePassIndex];
	
	PinnedPlate->PreEditChange(SourcePassListProperty);
	SourcePassList.RemoveAt(InSourcePassIndex);

	// If we aren't adding the item back to the same pass list, invoke a property changed event for its removal
	if (SourcePassType != DestPassType)
	{
		FPropertyChangedEvent SourceListChangedEvent(SourcePassListProperty, EPropertyChangeType::ArrayRemove);
		PinnedPlate->PostEditChangeProperty(SourceListChangedEvent);
		PinnedPlate->PreEditChange(DestPassListProperty);
	}
	
	DestPassList.Insert(Pass, InDestPassIndex);

	// If the item moved from one list to another, invoke a property changed event for the add; otherwise, invoke a move event
	if (SourcePassType != DestPassType)
	{
		FPropertyChangedEvent DestListChangedEvent(DestPassListProperty, EPropertyChangeType::ArrayAdd);
		PinnedPlate->PostEditChangeProperty(DestListChangedEvent);
	}
	else
	{
		FPropertyChangedEvent MoveChangedEvent(SourcePassListProperty, EPropertyChangeType::ArrayMove);
		PinnedPlate->PostEditChangeProperty(MoveChangedEvent);
	}
}

void FCompositePlatePassListOwner::RemovePasses(int32 InGroupIndex, const TArray<int32>& InPassIndices)
{
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = Plate.Pin();
	if (!PinnedPlate.IsValid())
	{
		return;
	}

	// Delete the passes in reverse order so that indices don't get messed up as we delete
	TArray<int32> SortedPassIndices = InPassIndices;
	SortedPassIndices.Sort();
	
	const EPassType PassType = (EPassType)InGroupIndex;
	TArray<TObjectPtr<UCompositePassBase>>& PassList = GetPassList(PassType);

	FProperty* PassListProperty = GetPassListProperty(PassType);
	PinnedPlate->PreEditChange(PassListProperty);

	for (int32 Index = SortedPassIndices.Num() - 1; Index >= 0; --Index)
	{
		const int32 PassIndex = SortedPassIndices[Index];

		// Here we replicate details panels instanced property array removal code.
		// The removed instanced pass objects must be moved to the transient package.
		TObjectPtr<UCompositePassBase>& PassToRemove = PassList[PassIndex];
		if (IsValid(PassToRemove))
		{
			PassToRemove->Modify();
			PassToRemove->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		}

		PassList.RemoveAt(PassIndex, EAllowShrinking::No);
	}

	PassList.Shrink();
			
	FPropertyChangedEvent PropertyChangedEvent(PassListProperty, EPropertyChangeType::ArrayRemove);
	PinnedPlate->PostEditChangeProperty(PropertyChangedEvent);
}

TArray<TObjectPtr<UCompositePassBase>>& FCompositePlatePassListOwner::GetPassList(EPassType InPassType)
{
	TStrongObjectPtr<UCompositeLayerPlate> PinnedPlate = Plate.Pin();
	check(PinnedPlate.IsValid());

	switch (InPassType)
	{
	case EPassType::Media:
		return PinnedPlate->MediaPasses;

	case EPassType::Layer:
	default:
		return PinnedPlate->LayerPasses;
	}
}

FProperty* FCompositePlatePassListOwner::GetPassListProperty(EPassType InPassType)
{
	switch (InPassType)
	{
	case EPassType::Media:
		return FindFProperty<FProperty>(UCompositeLayerPlate::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeLayerPlate, MediaPasses));

	case EPassType::Layer:
	default:
		// LayerPasses now lives on the base class; FindFProperty doesn't walk the inheritance chain.
		return FindFProperty<FProperty>(UCompositeLayerBase::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeLayerBase, LayerPasses));
	}
}

#undef LOCTEXT_NAMESPACE
