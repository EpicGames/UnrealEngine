// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SCompositePassTree.h"

/** ICompositePassListOwner implementation for UCompositeLayerPlate that manages its pass lists */
class FCompositePlatePassListOwner : public ICompositePassListOwner
{
private:
	/** Pass type which corresponds to which list in the plate layer a pass is stored in. The enum order determines display order — media passes are last so they appear at the bottom. */
	enum class EPassType : uint8
	{
		Layer,
		Media,
		MAX
	};

public:
	FCompositePlatePassListOwner(const TWeakObjectPtr<UCompositeLayerPlate>& InPlate)
		: Plate(InPlate)
	{ }

	//~ ICompositePassListOwner interface
	
public:
	virtual bool IsObjectValid() override;
	virtual TStrongObjectPtr<UObject> GetObject() override;
	virtual bool IsPassListPropertyName(const FName& InPropertyName) override;

	virtual  bool IsValidPassIndex(int32 InGroupIndex, int32 InPassIndex) override;
	virtual UCompositePassBase* GetPass(int32 InGroupIndex, int32 InPassIndex) override;
	virtual int32 GetNumGroups() const override { return (int32)EPassType::MAX; }
	virtual  TArray<TObjectPtr<UCompositePassBase>>& GetPassesForGroup(int32 InGroupIndex) override;
	
	virtual FString GetGroupFilterString(int32 InGroupIndex) override;
	virtual const FSlateBrush* GetGroupIcon(int32 InGroupIndex) override;
	virtual FText GetGroupDisplayName(int32 InGroupIndex) override;
	virtual FGroupFilterConfig GetGroupFilterConfig(int32 InGroupIndex) override;

	virtual int32 GetDefaultGroupForNewPass(const UClass* InPassClass) const override;
	virtual bool CanAddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) const override;
	virtual int32 AddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) override;
	virtual TArray<int32> CopyPasses(const TArray<UCompositePassBase*>& InPassesToCopy, int32 InGroupIndex, int32 InPassIndex) override;
	virtual void MovePass(int32 InSourceGroupIndex, int32 InSourcePassIndex, int32 InDestGroupIndex, int32 InDestPassIndex) override;
	virtual void RemovePasses(int32 InGroupIndex, const TArray<int32>& InPassIndices) override;
	
	//~ End ICompositePassListOwner interface
	
private:
	/** Gets a reference to the pass list that corresponds to the specified pass type in the plate. The plate pointer must be valid to call properly */
	TArray<TObjectPtr<UCompositePassBase>>& GetPassList(EPassType InPassType);

	/** Gets the FProperty object for the list that corresponds to the specified pass type in the plate */
	static FProperty* GetPassListProperty(EPassType InPassType);
	
private:
	/** The plate UObject whose pass lists are being provided */
	TWeakObjectPtr<UCompositeLayerPlate> Plate;
};
