// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompositeActorPickerSceneOutliner.h"

#include "ActorMode.h"
#include "ActorTreeItem.h"
#include "EngineUtils.h"
#include "ISceneOutlinerColumn.h"
#include "ISceneOutlinerMode.h"
#include "SceneOutlinerModule.h"
#include "ScopedTransaction.h"
#include "WorldTreeItem.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SCompositeActorPickerTable"

/** Custom column for the scene outliner that displays a checkbox to allow users to select multiple actors quickly */
class FCompositeActorPickerSelectedColumn : public ISceneOutlinerColumn
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetActorSelected, AActor*);
	DECLARE_DELEGATE_TwoParams(FOnActorSelected, AActor*, bool /* bIsSelected */);
	DECLARE_DELEGATE_TwoParams(FOnAllActorsSelected, const TArray<AActor*>& /* InActors */, bool /* bIsSelected */);
	
public:
	FCompositeActorPickerSelectedColumn(ISceneOutliner& InSceneOutliner)
	{
		SceneOutliner = StaticCastSharedRef<ISceneOutliner>(InSceneOutliner.AsShared());
	}

	static FName GetID() { return "IsSelected"; }

	// ISceneOutlinerColumn interface
	virtual FName GetColumnID() override
	{
		return GetID();
	}
	
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override
	{
		return SHeaderRow::Column(GetColumnID())
			.FixedWidth(24.0f)
			.DefaultLabel(FText::GetEmpty())
			.HAlignHeader(HAlign_Left)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked(this, &FCompositeActorPickerSelectedColumn::GetHeaderCheckState)
				.OnCheckStateChanged(this, &FCompositeActorPickerSelectedColumn::OnHeaderCheckStateChangedHandler)
			];
	}

	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef InTreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override
	{
		if (InTreeItem->IsA<FActorTreeItem>())
		{
			return SNew(SCheckBox)
				.IsChecked(this, &FCompositeActorPickerSelectedColumn::GetTreeItemSelectedCheckState, InTreeItem)
				.OnCheckStateChanged(this, &FCompositeActorPickerSelectedColumn::OnTreeItemSelectedCheckStateChanged, InTreeItem);
		}

		return SNullWidget::NullWidget;
	}
	// End of ISceneOutlinerColumn interface
	
private:
	/** Recursively visits all actor tree items in the scene outliner, calling the visitor for each valid actor. */
	void ForEachOutlinerActor(TFunctionRef<void(AActor*)> Visitor) const
	{
		TSharedPtr<ISceneOutliner> Outliner = SceneOutliner.Pin();
		if (!Outliner.IsValid())
		{
			return;
		}

		// Recursive lambda to walk the tree depth-first
		TFunction<void(const ISceneOutlinerTreeItem&)> VisitItem = [&](const ISceneOutlinerTreeItem& InItem)
		{
			if (const FActorTreeItem* ActorTreeItem = InItem.CastTo<FActorTreeItem>())
			{
				if (TStrongObjectPtr<AActor> PinnedActor = ActorTreeItem->Actor.Pin())
				{
					Visitor(PinnedActor.Get());
				}
			}

			for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildWeak : InItem.GetChildren())
			{
				if (TSharedPtr<ISceneOutlinerTreeItem> Child = ChildWeak.Pin())
				{
					VisitItem(*Child);
				}
			}
		};

		for (const FSceneOutlinerTreeItemPtr& RootItem : Outliner->GetTree().GetRootItems())
		{
			if (RootItem.IsValid())
			{
				VisitItem(*RootItem);
			}
		}
	}

	/** Returns the check state for the header checkbox based on all actors in the outliner: Checked if all are selected, Undetermined if mixed, Unchecked if none */
	ECheckBoxState GetHeaderCheckState() const
	{
		if (!OnGetActorSelected.IsBound())
		{
			return ECheckBoxState::Unchecked;
		}

		bool bAnySelected = false;
		bool bAnyUnselected = false;

		ForEachOutlinerActor([&](AActor* Actor)
		{
			if (OnGetActorSelected.Execute(Actor))
			{
				bAnySelected = true;
			}
			else
			{
				bAnyUnselected = true;
			}
		});

		if (bAnySelected)
		{
			return bAnyUnselected ? ECheckBoxState::Undetermined : ECheckBoxState::Checked;
		}

		return ECheckBoxState::Unchecked;
	}

	/** Toggles all actors in the outliner: selects all if any are unselected, deselects all if all are already selected */
	void OnHeaderCheckStateChangedHandler(ECheckBoxState InNewCheckState)
	{
		if (!OnAllActorsSelected.IsBound() || !OnGetActorSelected.IsBound())
		{
			return;
		}

		// Single pass: collect all actors and determine if we should select or deselect
		TArray<AActor*> Actors;
		bool bAllSelected = true;

		ForEachOutlinerActor([&](AActor* Actor)
		{
			Actors.Add(Actor);
			if (!OnGetActorSelected.Execute(Actor))
			{
				bAllSelected = false;
			}
		});

		if (!Actors.IsEmpty())
		{
			OnAllActorsSelected.Execute(Actors, !bAllSelected);
		}
	}

	ECheckBoxState GetTreeItemSelectedCheckState(FSceneOutlinerTreeItemRef InTreeItem) const
	{
		if (FActorTreeItem* ActorTreeItem = InTreeItem->CastTo<FActorTreeItem>())
		{
			if (!OnGetActorSelected.IsBound())
			{
				return ECheckBoxState::Unchecked;
			}
			
			if (TStrongObjectPtr<AActor> PinnedActor = ActorTreeItem->Actor.Pin())
			{
				return OnGetActorSelected.Execute(PinnedActor.Get()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		}
		
		return ECheckBoxState::Unchecked;
	}

	void OnTreeItemSelectedCheckStateChanged(ECheckBoxState InNewCheckState, FSceneOutlinerTreeItemRef InTreeItem)
	{
		if (FActorTreeItem* ActorTreeItem = InTreeItem->CastTo<FActorTreeItem>())
		{
			if (TStrongObjectPtr<AActor> PinnedActor = ActorTreeItem->Actor.Pin())
			{
				OnActorSelected.ExecuteIfBound(PinnedActor.Get(), InNewCheckState == ECheckBoxState::Checked);
			}
		}
	}
	
private:
	TWeakPtr<ISceneOutliner> SceneOutliner = nullptr;

public:
	FOnGetActorSelected OnGetActorSelected;
	FOnActorSelected OnActorSelected;
	FOnAllActorsSelected OnAllActorsSelected;
};

void SCompositeActorPickerSceneOutliner::Construct(const FArguments& InArgs, const FCompositeActorPickerListRef& InActorListRef)
{
	ActorListRef = InActorListRef;
	OnActorListChanged = InArgs._OnActorListChanged;
	
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	FSceneOutlinerInitializationOptions InitOptions;
	{
		InitOptions.bShowHeaderRow = true;
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		InitOptions.Filters = InArgs._SceneOutlinerFilters;

		// FActorMode::IsActorDisplayable hides RF_Transient actors by default, which would block
		// every sequencer spawnable from ever reaching our custom filter. Opt in to seeing them.
		InitOptions.bShowTransient = true;

		InitOptions.ColumnMap.Add(FCompositeActorPickerSelectedColumn::GetID(), FSceneOutlinerColumnInfo(
			ESceneOutlinerColumnVisibility::Visible,
			0,
			FCreateSceneOutlinerColumn::CreateLambda([this](ISceneOutliner& Outliner)
			{
				TSharedPtr<FCompositeActorPickerSelectedColumn> Column = MakeShared<FCompositeActorPickerSelectedColumn>(Outliner);
				Column->OnGetActorSelected.BindSP(this, &SCompositeActorPickerSceneOutliner::OnGetActorSelected);
				Column->OnActorSelected.BindSP(this, &SCompositeActorPickerSceneOutliner::OnActorSelected);
				Column->OnAllActorsSelected.BindSP(this, &SCompositeActorPickerSceneOutliner::OnAllActorsSelected);
				
				return Column.ToSharedRef();
			})
		));
			
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 1));
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 2));
	}
	
	ChildSlot
	[
		SNew(SBox)
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(
				InitOptions,
				FOnActorPicked::CreateLambda([&](AActor* Actor)
				{
					OnActorSelected(Actor, true);
					FSlateApplication::Get().DismissAllMenus();
				})
			)
		]
	];
}

bool SCompositeActorPickerSceneOutliner::OnGetActorSelected(AActor* InActor)
{
	if (!ActorListRef.IsValid())
	{
		return false;
	}

	return ActorListRef.ActorList->Contains(InActor);
}

void SCompositeActorPickerSceneOutliner::OnActorSelected(AActor* InActor, bool bIsSelected)
{
	if (!ActorListRef.IsValid())
	{
		return;
	}

	if (bIsSelected && !SCompositeActorPickerTable::IsAllowedActor(InActor))
	{
		return;
	}

	TStrongObjectPtr<UObject> PinnedListOwner = ActorListRef.ActorListOwner.Pin();

	const bool bIsAdding = bIsSelected && !ActorListRef.ActorList->Contains(InActor);
	const bool bIsRemoving = !bIsSelected && ActorListRef.ActorList->Contains(InActor);

	if (!bIsAdding && !bIsRemoving)
	{
		return;
	}
	
	FScopedTransaction SelectActorTransaction(LOCTEXT("SelectActorTransaction", "Select Actor"));
	PinnedListOwner->Modify();

	ActorListRef.NotifyPreEditChange();
	int32 LastModifiedIndex = INDEX_NONE;

	EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified;
	if (bIsAdding)
	{
		ActorListRef.ActorList->Add(InActor);
		LastModifiedIndex = ActorListRef.ActorList->Num() - 1;
		ChangeType = EPropertyChangeType::ArrayAdd;
	}
	else if (bIsRemoving)
	{
		ActorListRef.ActorList->Remove(InActor);
		ChangeType = EPropertyChangeType::ArrayRemove;
	}

	if (bIsAdding)
	{
		TArray<TSoftObjectPtr<AActor>> AddedActors;
		AddedActors.Add(InActor);

		ActorListRef.OnActorsAdded.ExecuteIfBound(AddedActors);
	}

	ActorListRef.NotifyPostEditChangeList(ChangeType, LastModifiedIndex);
	OnActorListChanged.ExecuteIfBound();
}

void SCompositeActorPickerSceneOutliner::OnAllActorsSelected(const TArray<AActor*>& InActors, bool bIsSelected)
{
	if (!ActorListRef.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> PinnedListOwner = ActorListRef.ActorListOwner.Pin();

	FScopedTransaction Transaction(bIsSelected
		? LOCTEXT("SelectAllActorsTransaction", "Select All Actors")
		: LOCTEXT("DeselectAllActorsTransaction", "Deselect All Actors"));
	PinnedListOwner->Modify();

	ActorListRef.NotifyPreEditChange();

	bool bModified = false;

	TArray<TSoftObjectPtr<AActor>> AddedActors;
	for (AActor* Actor : InActors)
	{
		if (!Actor)
		{
			continue;
		}

		if (bIsSelected && !SCompositeActorPickerTable::IsAllowedActor(Actor))
		{
			continue;
		}

		const bool bIsInList = ActorListRef.ActorList->Contains(Actor);

		if (bIsSelected && !bIsInList)
		{
			ActorListRef.ActorList->Add(Actor);
			AddedActors.Add(Actor);
			bModified = true;
		}
		else if (!bIsSelected && bIsInList)
		{
			ActorListRef.ActorList->Remove(Actor);
			bModified = true;
		}
	}

	if (bModified)
	{
		if (!AddedActors.IsEmpty())
		{
			ActorListRef.OnActorsAdded.ExecuteIfBound(AddedActors);
		}

		ActorListRef.NotifyPostEditChangeList(bIsSelected ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
		OnActorListChanged.ExecuteIfBound();
	}
	else
	{
		Transaction.Cancel();
	}
}

#undef LOCTEXT_NAMESPACE