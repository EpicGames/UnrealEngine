// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCompositeActorPickerTable.h"
#include "Widgets/SCompoundWidget.h"

struct FSceneOutlinerFilters;

/**
 * Scene Outliner used for picking multiple actors quickly for a composite actor list
 */
class SCompositeActorPickerSceneOutliner : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCompositeActorPickerSceneOutliner) { }
		SLATE_EVENT(FSimpleDelegate, OnActorListChanged)
		SLATE_ARGUMENT(TSharedPtr<FSceneOutlinerFilters>, SceneOutlinerFilters)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FCompositeActorPickerListRef& InActorListRef);

private:
	/** Raised when the selection state of a specific actor needs to be determined */
	bool OnGetActorSelected(AActor* InActor);

	/** Raised when the selection state of a specific actor has changed */
	void OnActorSelected(AActor* InActor, bool bIsSelected);

	/** Raised when the header checkbox toggles to select or deselect all actors at once */
	void OnAllActorsSelected(const TArray<AActor*>& InActors, bool bIsSelected);
	
private:
	/** Reference to the actor list being modified by this actor picker */
	FCompositeActorPickerListRef ActorListRef;

	/** Raised when the actor list has changed */
	FSimpleDelegate OnActorListChanged;
};
