// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompositeActorPickerTable.h"

#include "ActorTreeItem.h"
#include "CompositeCoreSettings.h"
#include "CompositeMeshActor.h"
#include "Components/PrimitiveComponent.h"
#include "SceneOutlinerFilters.h"
#include "CompositeEditorCommands.h"
#include "CompositeEditorPanelSettings.h"
#include "CompositeEditorStyle.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "IContentBrowserSingleton.h"
#include "SceneOutlinerDragDrop.h"
#include "SCompositeActorPickerSceneOutliner.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Components/VerticalBox.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "Editor/EditorWidgets/Public/SDropTarget.h"
#include "Filters/SFilterSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/TextFilter.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SCompositeActorPickerTable"

namespace CompositeActorPickerTable
{
	static FName ActorListColumn_ActorName = "ActorName";
	static FName ActorListColumn_Type = "Type";
}

/** Toolbar widget that contains filtering and an add button */
class SCompositeActorPickerToolbar : public SCompoundWidget
{
private:
	using FFilterType = SCompositeActorPickerTable::FActorListItemRef;
	using FTreeFilter =  TTextFilter<FFilterType>;
	using FOnExtendAddMenu = SCompositeActorPickerTable::FOnExtendAddMenu;
	
public:	
	SLATE_BEGIN_ARGS(SCompositeActorPickerToolbar) {}
		SLATE_ATTRIBUTE(TSharedPtr<FSceneOutlinerFilters>, SceneOutlinerFilters)
		SLATE_EVENT(FSimpleDelegate, OnFilterChanged)
		SLATE_EVENT(FSimpleDelegate, OnActorListChanged)
		SLATE_EVENT(FOnExtendAddMenu, OnExtendAddMenu)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FCompositeActorPickerListRef& InActorListRef)
	{
		ActorListRef = InActorListRef;
		SceneOutlinerFilters = InArgs._SceneOutlinerFilters;
		OnFilterChanged = InArgs._OnFilterChanged;
		OnActorListChanged = InArgs._OnActorListChanged;
		OnExtendAddMenu = InArgs._OnExtendAddMenu;

		InitializeFilters();

		// Generate a (multi-layered) icon for the "Add" menu
		const TSharedRef<SLayeredImage> AddIcon =
			SNew(SLayeredImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush("LevelEditor.OpenAddContent.Background"));
		AddIcon->AddLayer(FAppStyle::GetBrush("LevelEditor.OpenAddContent.Overlay"));
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SComboButton)
					.HasDownArrow(true)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.ToolTipText(LOCTEXT("AddActorButtonToolTip", "Add a level actor to the list of selected actors"))
					.VAlign(VAlign_Center)
					.OnGetMenuContent(this, &SCompositeActorPickerToolbar::GetAddActorMenuContent)
					.ButtonContent()
					[
						AddIcon
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SCheckBox)
					.Padding(4.0)
					.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
					.Type(ESlateCheckBoxType::ToggleButton)
					.ToolTipText(LOCTEXT("SyncSelectionToolTip", "Sync the world outliner selection with this table's selection"))
					.IsChecked_Lambda([]
					{
						if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
						{
							return Settings->bSyncActorSelection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}

						return ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckState)
					{
						if (UCompositeEditorPanelSettings* Settings = GetMutableDefault<UCompositeEditorPanelSettings>())
						{
							Settings->bSyncActorSelection = CheckState == ECheckBoxState::Checked;
							Settings->SaveConfig();
						}
					})
					[
						SNew(SImage)
						.Image(FCompositeEditorStyle::Get().GetBrush("CompositeEditor.SyncSelection"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(TextFilterSearchBox, SFilterSearchBox)
					.HintText(LOCTEXT("FilterSearch", "Search..."))
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search for specific actors"))
					.OnTextChanged_Lambda([this](const FText& InText)
					{
						TreeTextFilter->SetRawFilterText(InText);
					})
				]
			]
		];
	}

	bool ItemPassesFilters(const FFilterType& InItem) const
	{
		return TreeTextFilter->PassesFilter(InItem);
	}

	void ClearFilters()
	{
		TextFilterSearchBox->SetText(FText::GetEmpty());
	}
	
private:
	/** Creates all the filter bar filters and initializes the text filter */
	void InitializeFilters()
	{
		TreeTextFilter = MakeShared<FTreeFilter>(FTreeFilter::FItemToStringArray::CreateLambda([](const FFilterType& InItem, TArray<FString>& OutStrings)
		{
			if (InItem.IsValid() && InItem->Actor.IsValid())
			{
				OutStrings.Add(InItem->Actor->GetActorNameOrLabel());
				OutStrings.Add(InItem->Actor->GetClass()->GetDisplayNameText().ToString());
			}
		}));
		TreeTextFilter->OnChanged().AddSP(this, &SCompositeActorPickerToolbar::FilterChanged);
	}

	/** Creates the menu widget to display when the Add button is pressed */
	TSharedRef<SWidget> GetAddActorMenuContent()
	{
		constexpr bool bCloseMenuAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

		MenuBuilder.BeginSection("Add", LOCTEXT("AddSectionLabel", "Add Actor"));
		{
			MenuBuilder.AddMenuEntry(LOCTEXT("AddSelectedEntryLabel", "Add Selected in Outliner"),
				LOCTEXT("AddSelectionEntryToolTip", "Adds all actors currently selected in the level editor"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(),"FoliageEditMode.SetSelect"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SCompositeActorPickerToolbar::AddSelectedActors),
					FCanExecuteAction::CreateSP(this, &SCompositeActorPickerToolbar::CanAddSelectedActors))
			);

			OnExtendAddMenu.ExecuteIfBound(MenuBuilder);
		}
		MenuBuilder.EndSection();
		
		MenuBuilder.BeginSection("SceneOutliner", LOCTEXT("SceneOutlinerSectionLabel", "Browse"));
		{
			TSharedPtr<FSceneOutlinerFilters> Filters = SceneOutlinerFilters.Get(nullptr);
			if (!Filters)
			{
				Filters = SCompositeActorPickerTable::MakeDefaultSceneOutlinerFilters();
			}

			TSharedRef<SWidget> SceneOutliner =
				SNew(SCompositeActorPickerSceneOutliner, ActorListRef)
				.SceneOutlinerFilters(Filters)
				.OnActorListChanged(OnActorListChanged);
			
			MenuBuilder.AddWidget(SceneOutliner, FText::GetEmpty(), true);
		}
		MenuBuilder.EndSection();
		
		return MenuBuilder.MakeWidget();
	}

	/** Adds all actors selected in the level/outliner to the actor list */
	void AddSelectedActors()
	{
		if (!ActorListRef.IsValid())
		{
			return;
		}

		TStrongObjectPtr<UObject> PinnedListOwner = ActorListRef.ActorListOwner.Pin();
		
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

		if (SelectedActors.IsEmpty())
		{
			return;
		}

		TSharedPtr<FScopedTransaction> AddActorsTransaction = nullptr;
		int32 LastModifiedIndex = INDEX_NONE;

		TArray<TSoftObjectPtr<AActor>> AddedActors;
		for (AActor* SelectedActor : SelectedActors)
		{
			if (!SCompositeActorPickerTable::IsAllowedActor(SelectedActor))
			{
				continue;
			}

			if (ActorListRef.ActorList->Contains(SelectedActor))
			{
				continue;
			}

			if (!AddActorsTransaction.IsValid())
			{
				AddActorsTransaction = MakeShared<FScopedTransaction>(LOCTEXT("AddActorsTransaction", "Add Actors"));
				PinnedListOwner->Modify();

				ActorListRef.NotifyPreEditChange();
			}

			ActorListRef.ActorList->Add(SelectedActor);
			AddedActors.Add(SelectedActor);
			LastModifiedIndex = ActorListRef.ActorList->Num() - 1;
		}

		if (AddActorsTransaction.IsValid())
		{
			if (!AddedActors.IsEmpty())
			{
				ActorListRef.OnActorsAdded.ExecuteIfBound(AddedActors);
			}

			ActorListRef.NotifyPostEditChangeList(EPropertyChangeType::ArrayAdd, LastModifiedIndex);

			OnActorListChanged.ExecuteIfBound();
		}
	}

	/** Gets whether the selected level/outliner actors can be added to the actor list */
	bool CanAddSelectedActors()
	{
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
		
		return ActorListRef.IsValid() && SelectedActors.Num() > 0;
	}

	/** Raised when the actor list filter has been changed */
	void FilterChanged()
	{
		OnFilterChanged.ExecuteIfBound();
	}
	
private:
	/** A reference to the actor list being managed by the actor picker */
	FCompositeActorPickerListRef ActorListRef;
	
	TSharedPtr<FTreeFilter> TreeTextFilter;
	
	TSharedPtr<SFilterSearchBox> TextFilterSearchBox;

	/** Attribute to retrieve the filters to apply to the scene outliner actor picker when it is opened */
	TAttribute<TSharedPtr<FSceneOutlinerFilters>> SceneOutlinerFilters;

	FSimpleDelegate OnFilterChanged;
	FSimpleDelegate OnActorListChanged;
	FOnExtendAddMenu OnExtendAddMenu;
};

/** Table row to display an actor in the actor list view */
class SCompositeActorListItemRow : public SMultiColumnTableRow<SCompositeActorPickerTable::FActorListItemRef>
{
	using FActorListItemRef = SCompositeActorPickerTable::FActorListItemRef;
	using FOnGenerateColumnWidget = SCompositeActorPickerTable::FOnGenerateColumnWidget;

public:
	SLATE_BEGIN_ARGS(SCompositeActorListItemRow) { }
		SLATE_EVENT(FOnGenerateColumnWidget, OnGenerateColumnWidget)
		SLATE_ARGUMENT(const FCompositeSpawnableBindings*, SpawnableBindings)
		SLATE_ARGUMENT(TWeakObjectPtr<UObject>, BindingsOwner)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const FActorListItemRef& InActorItem)
	{
		ActorItem = InActorItem;
		OnGenerateColumnWidget = InArgs._OnGenerateColumnWidget;
		SpawnableBindings = InArgs._SpawnableBindings;
		BindingsOwner = InArgs._BindingsOwner;

		STableRow<FActorListItemRef>::FArguments Args = FSuperRowType::FArguments()
		   .Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));
		
		SMultiColumnTableRow::Construct(Args, InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == CompositeActorPickerTable::ActorListColumn_ActorName)
		{
			return SNew(SHorizontalBox)
			
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8.0f, 1.0f, 6.0f, 1.0f)
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					.HeightOverride(16.0f)
					[
						SNew(SImage).Image(this, &SCompositeActorListItemRow::GetActorIcon)
					]
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(this, &SCompositeActorListItemRow::GetActorName)
				];
		}
		else if (InColumnName == CompositeActorPickerTable::ActorListColumn_Type)
		{
			return SNew(STextBlock).Text(this, &SCompositeActorListItemRow::GetActorType);
		}
		else if (OnGenerateColumnWidget.IsBound() && ActorItem.IsValid() && ActorItem->Actor.IsValid())
		{
			return OnGenerateColumnWidget.Execute(ActorItem->Actor, InColumnName);
		}

		return SNullWidget::NullWidget;
	}

private:
	/** Gets the icon to display for the actor */
	const FSlateBrush* GetActorIcon() const
	{
		if (ActorItem.IsValid() && ActorItem->Actor.IsValid())
		{
			return FSlateIconFinder::FindIconForClass(ActorItem->Actor->GetClass()).GetIcon();
		}

		return nullptr;
	}

	/** Gets the name to display for the actor */
	FText GetActorName() const
	{
		if (ActorItem.IsValid() && ActorItem->Actor.IsValid())
		{
			return FText::FromString(ActorItem->Actor->GetActorNameOrLabel());
		}

		// Hint for entries that have a binding but no resolved actor yet (e.g. a sequencer
		// spawnable that hasn't materialized, or a Multi-User peer awaiting Tick re-resolve).
		// Hotfix: BindingsOwner gates the SpawnableBindings deref; the raw pointer aliases the owner
		// UObject's memory, and this row can tick once more after the owner is torn down on map switch.
		if (ActorItem.IsValid() && BindingsOwner.IsValid() && SpawnableBindings != nullptr
			&& SpawnableBindings->HasBindingAt(ActorItem->Index))
		{
			return LOCTEXT("SpawnableRef", "[Pending Spawnable]");
		}

		return LOCTEXT("NoneActorLabel", "None");
	}

	/** Gets the name of the actor's type to display */
	FText GetActorType() const
	{
		if (ActorItem.IsValid() && ActorItem->Actor.IsValid())
		{
			return ActorItem->Actor->GetClass()->GetDisplayNameText();
		}

		return LOCTEXT("InvalidActorTypeLabel", "-");
	}
	
private:
	FActorListItemRef ActorItem;
	FOnGenerateColumnWidget OnGenerateColumnWidget;
	const FCompositeSpawnableBindings* SpawnableBindings = nullptr;

	/** Weak handle to the UObject that owns SpawnableBindings; gates the raw-pointer deref so a row
	 *  ticked after the owner's destruction can't dangle. */
	TWeakObjectPtr<UObject> BindingsOwner;
};

FCompositeActorPickerListRef::FCompositeActorPickerListRef(const TWeakObjectPtr<UObject>& InActorListOwner, const FName& InActorListPropertyName, TArray<TSoftObjectPtr<AActor>>* InActorList, const FCompositeSpawnableBindings* InSpawnableBindings)
	: ActorListOwner(InActorListOwner)
	, ActorListPropertyName(InActorListPropertyName)
	, ActorList(InActorList)
	, SpawnableBindings(InSpawnableBindings)
{
	if (TStrongObjectPtr<UObject> PinnedOwner = ActorListOwner.Pin())
	{
		ActorListProperty = FindFProperty<FProperty>(PinnedOwner->GetClass(), ActorListPropertyName);
	}
}

bool FCompositeActorPickerListRef::IsValid() const
{
	return ActorListOwner.IsValid() && ActorListProperty != nullptr && ActorList != nullptr;
}

void FCompositeActorPickerListRef::NotifyPreEditChange()
{
	if (!IsValid())
	{
		return;
	}

	if (TStrongObjectPtr<UObject> PinnedOwner = ActorListOwner.Pin())
	{
		PinnedOwner->PreEditChange(ActorListProperty);
	}
}

void FCompositeActorPickerListRef::NotifyPostEditChangeList(EPropertyChangeType::Type InChangeType, int32 LastModifiedIndex)
{
	if (!IsValid())
	{
		return;
	}

	if (TStrongObjectPtr<UObject> PinnedOwner = ActorListOwner.Pin())
	{
		FPropertyChangedEvent ChangedEvent(ActorListProperty, InChangeType);
		
		// TODO: Full implementation would require an FPropertyChangedChainEvent

		TMap<FString, int32> ArrayIndexPerObject;
		ArrayIndexPerObject.Add(ActorListPropertyName.ToString(), LastModifiedIndex);
		ChangedEvent.ObjectIteratorIndex = 0;
		ChangedEvent.SetArrayIndexPerObject(MakeArrayView(&ArrayIndexPerObject, 1));

		PinnedOwner->PostEditChangeProperty(ChangedEvent);
	}
}

FText FCompositeActorPickerListRef::GetToolTipText(bool bShortTooltip) const
{
	return ActorListProperty ? ActorListProperty->GetToolTipText(bShortTooltip) : FText::GetEmpty();
}

TSharedPtr<FSceneOutlinerFilters> SCompositeActorPickerTable::MakeDefaultSceneOutlinerFilters(bool bExcludeCompositeMeshActors)
{
	TSharedPtr<FSceneOutlinerFilters> Filters = MakeShared<FSceneOutlinerFilters>();

	Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda([bExcludeCompositeMeshActors](const AActor* InActor)
	{
		if (!IsAllowedActor(const_cast<AActor*>(InActor)))
		{
			return false;
		}

		if (bExcludeCompositeMeshActors && InActor->IsA<ACompositeMeshActor>())
		{
			return false;
		}

		const UCompositeCorePluginSettings* Settings = GetDefault<UCompositeCorePluginSettings>();
		TInlineComponentArray<UPrimitiveComponent*> PrimComponents(InActor);

		for (const UPrimitiveComponent* PrimitiveComponent : PrimComponents)
		{
			if (Settings->IsAllowedPrimitiveClass(PrimitiveComponent))
			{
				return true;
			}
		}

		return false;
	}));

	return Filters;
}

bool SCompositeActorPickerTable::IsAllowedActor(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return false;
	}

	if (!InActor->HasAnyFlags(RF_Transient))
	{
		return true;
	}

	// Transient actor: accept only if it is a sequencer spawnable that the
	// FCompositeSpawnableBindings side-channel can resolve.
	return UE::Composite::DetectSpawnableBinding(InActor, InActor->GetWorld()).IsValid();
}

void SCompositeActorPickerTable::Construct(const FArguments& InArgs, const FCompositeActorPickerListRef& InActorListRef)
{
	ActorListRef = InActorListRef;
	OnExtendContextMenu = InArgs._OnExtendContextMenu;
	OnLayoutSizeChanged = InArgs._OnLayoutSizeChanged;
	bShowApplyMaterialSection = InArgs._ShowApplyMaterialSection;
	OnApplyMaterialToActor = InArgs._OnApplyMaterialToActor;
	
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SCompositeActorPickerTable::OnObjectPropertyChanged);
	USelection::SelectionChangedEvent.AddSP(this, &SCompositeActorPickerTable::OnLevelSelectionChanged);
	USelection::SelectObjectEvent.AddSP(this, &SCompositeActorPickerTable::OnLevelSelectionChanged);
	
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	CommandList = MakeShared<FUICommandList>();
	BindCommands();
	
	HeaderRow = 
		SNew(SHeaderRow)
		.CanSelectGeneratedColumn(true)
		.HiddenColumnsList(InArgs._HiddenColumnsList)
		.OnHiddenColumnsListChanged(InArgs._OnHiddenColumnsListChanged)

		+SHeaderRow::Column(CompositeActorPickerTable::ActorListColumn_ActorName)
		.FillWidth(0.6)
		.DefaultLabel(LOCTEXT("ActorNameColumnLabel", "Item Label"))

		+SHeaderRow::Column(CompositeActorPickerTable::ActorListColumn_Type)
		.FillWidth(0.4)
		.DefaultLabel(LOCTEXT("ActorTypeColumnLabel", "Type"));
	
	InArgs._OnGenerateHeaderColumns.ExecuteIfBound(HeaderRow.ToSharedRef());
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f)
		[
			SAssignNew(Toolbar, SCompositeActorPickerToolbar, ActorListRef)
			.SceneOutlinerFilters(InArgs._SceneOutlinerFilters)
			.OnFilterChanged(this, &SCompositeActorPickerTable::OnFilterChanged)
			.OnActorListChanged_Lambda([this]
			{
				FillActorList();
			})
			.OnExtendAddMenu(InArgs._OnExtendAddMenu)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SDropTarget)
			.OnAllowDrop(this, &SCompositeActorPickerTable::OnAllowListDrop)
			.OnDropped(this, &SCompositeActorPickerTable::OnListDropped)
			[
				SAssignNew(ListView, SListView<FActorListItemRef>)
				.HeaderRow(HeaderRow)
				.ListItemsSource(&FilteredActorListItems)
				.OnGenerateRow_Lambda([OnGenerateColumnWidget = InArgs._OnGenerateColumnWidget, SpawnableBindings = ActorListRef.SpawnableBindings, BindingsOwner = ActorListRef.ActorListOwner](FActorListItemRef InListItem, const TSharedRef<STableViewBase>& OwnerTable)
				{
					return SNew(SCompositeActorListItemRow, OwnerTable, InListItem)
						.OnGenerateColumnWidget(OnGenerateColumnWidget)
						.SpawnableBindings(SpawnableBindings)
						.BindingsOwner(BindingsOwner);
				})
				.OnContextMenuOpening(this, &SCompositeActorPickerTable::CreateListContextMenu)
				.OnSelectionChanged(this, &SCompositeActorPickerTable::OnActorSelectionChanged)
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.MinHeight(24.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SCompositeActorPickerTable::GetFilterStatusText)
			.ColorAndOpacity(this, &SCompositeActorPickerTable::GetFilterStatusTextColor)
		]
	];

	FillActorList();
}

SCompositeActorPickerTable::~SCompositeActorPickerTable()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
	
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
}

void SCompositeActorPickerTable::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Catches silent array mutations (e.g. TickResolveStale resolves) that bypass OnObjectPropertyChanged.
	FillActorList();

	// Keep track of when the list view has changed the number of widgets it is actively displaying.
	// This allows us to know if list items have been added or removed (as a list view's internal list is
	// not generated immediately when calling RefreshList), so that we can call OnLayoutSizeChanged at the right time
	const int32 ListViewNumChildren = ListView.IsValid() ? ListView->GetNumGeneratedChildren() : 0;
	if (ListViewNumChildren != CachedListViewNumChildren)
	{
		OnLayoutSizeChanged.ExecuteIfBound();
		CachedListViewNumChildren = ListViewNumChildren;
	}
}

FReply SCompositeActorPickerTable::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SCompositeActorPickerTable::PostUndo(bool bSuccess)
{
	FillActorList();
}

void SCompositeActorPickerTable::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

TArray<FName> SCompositeActorPickerTable::GetHiddenColumnsList() const
{
	if (HeaderRow.IsValid())
	{
		return HeaderRow->GetHiddenColumnIds();
	}
	
	return TArray<FName>();
}

void SCompositeActorPickerTable::BindCommands()
{
	CommandList->MapAction(FCompositeEditorCommands::Get().RemoveActor,
		FExecuteAction::CreateSP(this, &SCompositeActorPickerTable::RemoveActors),
		FCanExecuteAction::CreateSP(this, &SCompositeActorPickerTable::CanEditActors));
}

void SCompositeActorPickerTable::FillActorList()
{
	if (!ActorListRef.IsValid())
	{
		return;
	}

	// Check to see if a refresh is needed by comparing the list of actors being displayed to
	// the list of actors in the actor list reference
	bool bNeedsRefresh = false;
	if (ActorListItems.Num() != ActorListRef.ActorList->Num())
	{
		bNeedsRefresh = true;
	}
	else
	{
		for (int32 Index = 0; Index < ActorListItems.Num(); ++Index)
		{
			const FActorListItemRef& ActorListItem = ActorListItems[Index];
			if (!ActorListItem.IsValid() || ActorListItem->Actor != (*ActorListRef.ActorList)[Index])
			{
				bNeedsRefresh = true;
				break;
			}
		}
	}

	if (!bNeedsRefresh)
	{
		return;
	}

	ActorListItems.Empty();
	
	for (int32 Index = 0; Index < ActorListRef.ActorList->Num(); ++Index)
	{
		ActorListItems.Add(MakeShared<FActorListItem>((*ActorListRef.ActorList)[Index], Index));
	}

	FilterActorList();

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
		
		// If the selection should be synchronized with the editor's selection, select any actors that
		// are currently selected in the editor
		if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
		{
			if (Settings->bSyncActorSelection)
			{
				if (USelection* Selection = GEditor->GetSelectedActors())
				{
					TArray<AActor*> SelectedActors;	
					Selection->GetSelectedObjects<AActor>(SelectedActors);
			
					ListView->ClearSelection();
					for (const AActor* Actor : SelectedActors)
					{
						FActorListItemRef* ActorToSelect = FilteredActorListItems.FindByPredicate([&Actor](const FActorListItemRef& Item)
						{
							return Item.IsValid() && Item->Actor.IsValid() && Item->Actor.Get() == Actor;
						});

						if (ActorToSelect)
						{
							ListView->SetItemSelection(*ActorToSelect, true);
						}
					}
				}
			}
		}
	}
}

void SCompositeActorPickerTable::FilterActorList()
{
	FilteredActorListItems.Empty();

	for (const FActorListItemRef& Actor : ActorListItems)
	{
		const bool bPassesFilter = Toolbar.IsValid() ? Toolbar->ItemPassesFilters(Actor) : true;
		if (bPassesFilter)
		{
			FilteredActorListItems.Add(Actor);
		}
	}
}

void SCompositeActorPickerTable::OnFilterChanged()
{
	FilterActorList();
	
	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

TSharedPtr<SWidget> SCompositeActorPickerTable::CreateListContextMenu()
{
	if (!ListView.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	const int32 NumItems = ListView->GetNumItemsSelected();
	if (NumItems >= 1)
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);
		
		if (bShowApplyMaterialSection.Get())
		{
			MenuBuilder.BeginSection("ApplyMaterial", LOCTEXT("ApplyMaterial", "Apply Material"));

			const FString LitMaterialPath = TEXT("/Composite/Materials/M_CompositeMesh_Lit_Masked.M_CompositeMesh_Lit_Masked");
			const FString UnlitMaterialPath = TEXT("/Composite/Materials/M_CompositeMesh_Unlit_AlphaComposite.M_CompositeMesh_Unlit_AlphaComposite");
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ApplyLitMaskedMaterial", "Apply Lit Masked Material To Selected Actors"),
				LOCTEXT("ApplyLitMaskedMaterialToolTip", "Apply the plugin default lit masked material to selected actors: best for catching shadows and reflections."),
				FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.LitMaterial"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SCompositeActorPickerTable::ApplyMaterialToActors, LitMaterialPath),
					FCanExecuteAction::CreateSP(this, &SCompositeActorPickerTable::CanEditActors)
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ApplyUnlitAlphaMaterial", "Apply Unlit Alpha Material To Selected Actors"),
				LOCTEXT("ApplyUnlitAlphaMaterialToolTip", "Apply the plugin default unlit alpha material to selected actors: best for keying media."),
				FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.UnlitMaterial"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SCompositeActorPickerTable::ApplyMaterialToActors, UnlitMaterialPath),
					FCanExecuteAction::CreateSP(this, &SCompositeActorPickerTable::CanEditActors)
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ApplyCustomMaterial", "Apply Custom Material To Selected Actors"),
				LOCTEXT("ApplyCustomMaterialToolTip", "Apply a custom material to selected actors"),
				FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.CustomMaterial"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SCompositeActorPickerTable::ApplyMaterialToActors, FString()),
					FCanExecuteAction::CreateSP(this, &SCompositeActorPickerTable::CanApplyCustomMaterial)
				)
			);

			MenuBuilder.AddSeparator();

			MenuBuilder.AddSubMenu(
				TAttribute<FText>::CreateSP(this, &SCompositeActorPickerTable::GetCustomMaterialSubMenuLabel),
				FText::GetEmpty(),
				FNewMenuDelegate::CreateSP(this, &SCompositeActorPickerTable::CreateCustomMaterialSubMenu),
				false,
				FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.CustomMaterial"),
				false
			);

			MenuBuilder.EndSection();
		}
		
		MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
		MenuBuilder.AddMenuEntry(FCompositeEditorCommands::Get().RemoveActor);
		MenuBuilder.EndSection();
		
		if (OnExtendContextMenu.IsBound())
		{
			TArray<TSoftObjectPtr<AActor>> SelectedActors;
			TArray<FActorListItemRef> SelectedItems = ListView->GetSelectedItems();
			Algo::Transform(SelectedItems, SelectedActors, [](const FActorListItemRef& Item)
			{
				return Item->Actor;
			});
			
			OnExtendContextMenu.ExecuteIfBound(MenuBuilder, SelectedActors);
		}
		
		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void SCompositeActorPickerTable::OnActorSelectionChanged(FActorListItemRef ActorListItem, ESelectInfo::Type InSelectType)
{
	if (InSelectType == ESelectInfo::Direct)
	{
		return;
	}
	
	if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
	{
		if (Settings->bSyncActorSelection)
		{
			TArray<FActorListItemRef> SelectedActors = ListView->GetSelectedItems();

			TArray<AActor*> ValidActorsToSelect;
			ValidActorsToSelect.Reserve(SelectedActors.Num());
			for (const FActorListItemRef& Actor : SelectedActors)
			{
				if (Actor.IsValid() && Actor->Actor.IsValid())
				{
					ValidActorsToSelect.Add(Actor->Actor.Get());
				}
			}

			// If only invalid (e.g. dangling "None") entries are selected, leave the editor selection alone.
			// Otherwise the round-trip through OnLevelSelectionChanged would immediately clear our row selection.
			if (ValidActorsToSelect.IsEmpty())
			{
				return;
			}

			GEditor->GetSelectedActors()->Modify();

			GEditor->GetSelectedActors()->BeginBatchSelectOperation();
			GEditor->GetSelectedActors()->DeselectAll();

			for (AActor* Actor : ValidActorsToSelect)
			{
				constexpr bool bIsSelected = true;
				constexpr bool bNotifyAfterSelect = false;
				constexpr bool bSelectEvenIfHidden = true;
				GEditor->SelectActor(Actor, bIsSelected, bNotifyAfterSelect, bSelectEvenIfHidden);
			}

			constexpr bool bNotify = false;
			GEditor->GetSelectedActors()->EndBatchSelectOperation(bNotify);

			GEditor->NoteSelectionChange();
		}
	}
}

void SCompositeActorPickerTable::OnLevelSelectionChanged(UObject* NewSelection)
{
	USelection* Selection = Cast<USelection>(NewSelection);
	if (!Selection)
	{
		return;
	}

	if (!ListView.IsValid())
	{
		return;
	}

	if (const UCompositeEditorPanelSettings* Settings = GetDefault<UCompositeEditorPanelSettings>())
	{
		if (Settings->bSyncActorSelection)
		{
			TArray<AActor*> SelectedActors;	
			Selection->GetSelectedObjects<AActor>(SelectedActors);
			
			ListView->ClearSelection();
			for (const AActor* Actor : SelectedActors)
			{
				FActorListItemRef* ActorToSelect = FilteredActorListItems.FindByPredicate([&Actor](const FActorListItemRef& Item)
				{
					return Item.IsValid() && Item->Actor.IsValid() && Item->Actor.Get() == Actor;
				});

				if (ActorToSelect)
				{
					ListView->SetItemSelection(*ActorToSelect, true);
				}
			}
		}
	}
}

bool SCompositeActorPickerTable::OnAllowListDrop(TSharedPtr<FDragDropOperation> InDragDropOperation)
{
	// Support dragging both actors and folders from the Outliner (dragging a folder will add all actor types in the folder)
	return InDragDropOperation->IsOfType<FActorDragDropGraphEdOp>() || InDragDropOperation->IsOfType<FSceneOutlinerDragDropOp>();
}

FReply SCompositeActorPickerTable::OnListDropped(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
{
	if (!ActorListRef.IsValid())
	{
		return FReply::Unhandled();
	}

	TStrongObjectPtr<UObject> PinnedListOwner = ActorListRef.ActorListOwner.Pin();
	
	TArray<AActor*> DroppedActors;
	if (const TSharedPtr<FActorDragDropGraphEdOp> ActorOperation = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>())
	{
		Algo::TransformIf(ActorOperation->Actors, DroppedActors,
			[](const TWeakObjectPtr<AActor>& Actor) { return IsAllowedActor(Actor.Get()); },
			[](const TWeakObjectPtr<AActor>& Actor) { return Actor.Get(); });
	}
	else if (const TSharedPtr<FSceneOutlinerDragDropOp> SceneOperation = DragDropEvent.GetOperationAs<FSceneOutlinerDragDropOp>())
	{
		if (const TSharedPtr<FFolderDragDropOp> FolderOp = SceneOperation->GetSubOp<FFolderDragDropOp>())
		{
			FActorFolders::GetActorsFromFolders(*FolderOp->World.Get(), FolderOp->Folders, DroppedActors);
			DroppedActors.RemoveAll([](AActor* Actor) { return !IsAllowedActor(Actor); });
		}
	}
			
	TSharedPtr<FScopedTransaction> AddActorsTransaction = nullptr;
	int32 LastModifiedIndex = INDEX_NONE;
	TArray<TSoftObjectPtr<AActor>> AddedActors;
	for (AActor* DroppedActor : DroppedActors)
	{
		if (ActorListRef.ActorList->Contains(DroppedActor))
		{
			continue;
		}

		if (!AddActorsTransaction.IsValid())
		{
			AddActorsTransaction = MakeShared<FScopedTransaction>(LOCTEXT("AddActorsTransaction", "Add Actors"));
			PinnedListOwner->Modify();

			ActorListRef.NotifyPreEditChange();
		}

		ActorListRef.ActorList->Add(DroppedActor);
		AddedActors.Add(DroppedActor);
		LastModifiedIndex = ActorListRef.ActorList->Num() - 1;
	}

	if (AddActorsTransaction.IsValid())
	{
		if (!AddedActors.IsEmpty())
		{
			ActorListRef.OnActorsAdded.ExecuteIfBound(AddedActors);
		}

		ActorListRef.NotifyPostEditChangeList(EPropertyChangeType::ArrayAdd, LastModifiedIndex);

		FillActorList();
	}

	return FReply::Handled();
}

FText SCompositeActorPickerTable::GetFilterStatusText() const
{
	const int32 NumActors = ActorListItems.Num();
	const int32 NumFiltered = FilteredActorListItems.Num();
	const int32 NumSelected = ListView.IsValid() ? ListView->GetNumItemsSelected() : 0;
	
	const FText ActorLabel = NumActors > 1 ? LOCTEXT("ActorPlural", "Actors") : LOCTEXT("ActorSingular", "Actor");
	if (NumActors > 0 && NumActors == NumFiltered)
	{
		if (NumSelected > 0)
		{
			return FText::Format(LOCTEXT("NumActorsAndSelectedTextFormat", "{0} {1}, {2} Selected"), FText::AsNumber(NumActors), ActorLabel, FText::AsNumber(NumSelected));
		}
		else
		{
			return FText::Format(LOCTEXT("NumActorsTextFormat", "{0} {1}"), FText::AsNumber(NumActors), ActorLabel);
		}
	}
	else if (NumFiltered > 0)
	{
		if (NumSelected > 0)
		{
			return FText::Format(LOCTEXT("NumActorsWithFilteredAndSelectedTextFormat", "Showing {0} of {1} {2}, {3} Selected"),
				FText::AsNumber(NumFiltered),
				FText::AsNumber(NumActors),
				ActorLabel,
				FText::AsNumber(NumSelected));
		}
		else
		{
			return FText::Format(LOCTEXT("NumActorsWithFilteredTextFormat", "Showing {0} of {1} {2}"),
				FText::AsNumber(NumFiltered),
				FText::AsNumber(NumActors),
				ActorLabel);
		}
	}
	else
	{
		if (NumActors > 0)
		{
			return FText::Format(LOCTEXT("NoMatchingActorsTextFormat", "No matching actors ({0} {1})"), FText::AsNumber(NumActors), ActorLabel);
		}
		else
		{
			return LOCTEXT("NoActorsFoundLabel", "0 Actors");
		}
	}
}

FSlateColor SCompositeActorPickerTable::GetFilterStatusTextColor() const
{
	const int32 NumActors = ActorListItems.Num();
	const int32 NumFiltered = FilteredActorListItems.Num();
	
	if (NumActors > 0 && NumFiltered == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	
	return FSlateColor::UseForeground();
}

void SCompositeActorPickerTable::RemoveActors()
{
	TArray<FActorListItemRef> SelectedActors = ListView.IsValid() ? ListView->GetSelectedItems() : TArray<FActorListItemRef>();
	if (SelectedActors.IsEmpty())
	{
		return;
	}

	if (!ActorListRef.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> PinnedListOwner = ActorListRef.ActorListOwner.Pin();

	TArray<int32> ActorIndicesToRemove;
	for (const FActorListItemRef& ActorItem : SelectedActors)
	{
		if (!ActorListRef.ActorList->IsValidIndex(ActorItem->Index))
		{
			return;
		}

		ActorIndicesToRemove.Add(ActorItem->Index);
	}

	if (ActorIndicesToRemove.IsEmpty())
	{
		return;
	}

	// Remove in reverse order so that indices don't get messed up as actors are removed
	ActorIndicesToRemove.Sort();

	FScopedTransaction RemoveActorsTransaction(LOCTEXT("RemoveActorsTransaction", "Remove Actors"));
	PinnedListOwner->Modify();
	ActorListRef.NotifyPreEditChange();
	
	for (int32 Index = ActorIndicesToRemove.Num() - 1; Index >= 0; --Index)
	{
		ActorListRef.ActorList->RemoveAt(ActorIndicesToRemove[Index]);
	}

	ActorListRef.NotifyPostEditChangeList(EPropertyChangeType::ArrayRemove);
	FillActorList();
}

void SCompositeActorPickerTable::ApplyMaterialToActors(FString InMaterialPath)
{
	UMaterialInterface* LoadedMaterial = nullptr;

	if (InMaterialPath.IsEmpty())
	{
		if (CustomMaterialAsset.IsValid() && CustomMaterialAsset.IsInstanceOf(UMaterialInterface::StaticClass()))
		{
			LoadedMaterial = Cast<UMaterialInterface>(CustomMaterialAsset.GetAsset());
		}
	}
	else
	{
		LoadedMaterial = LoadObject<UMaterialInterface>(nullptr, InMaterialPath);
	}

	if (LoadedMaterial == nullptr)
	{
		return;
	}

	TArray<FActorListItemRef> SelectedActors = ListView.IsValid() ? ListView->GetSelectedItems() : TArray<FActorListItemRef>();
	if (SelectedActors.IsEmpty())
	{
		return;
	}

	if (!ActorListRef.IsValid())
	{
		return;
	}

	TArray<int32> ActorIndicesToEdit;
	for (const FActorListItemRef& ActorItem : SelectedActors)
	{
		if (!ActorListRef.ActorList->IsValidIndex(ActorItem->Index))
		{
			return;
		}

		ActorIndicesToEdit.Add(ActorItem->Index);
	}

	if (ActorIndicesToEdit.IsEmpty())
	{
		return;
	}

	TStrongObjectPtr<UObject> PinnedListOwner = ActorListRef.ActorListOwner.Pin();

	FScopedTransaction UpdateActorsMaterialTransaction(LOCTEXT("UpdateActorsMaterial", "Update Actors Material"));
	PinnedListOwner->Modify();
	ActorListRef.NotifyPreEditChange();

	for (int32 Index : ActorIndicesToEdit)
	{
		TSoftObjectPtr<AActor>& Actor = (*ActorListRef.ActorList)[Index];
		if (!Actor.IsValid())
		{
			continue;
		}
		
		// If the OnApplyMaterialToActor handler is bound, call that to allow custom apply logic. Otherwise, apply the material to 
		// all the actor's primitive components
		if (OnApplyMaterialToActor.IsBound())
		{
			OnApplyMaterialToActor.Execute(Actor, LoadedMaterial);
		}
		else
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
				{
					PrimitiveComponent->Modify();
					const int32 MaterialCount = PrimitiveComponent->GetNumMaterials();
					for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; MaterialIndex++)
					{
						PrimitiveComponent->SetMaterial(MaterialIndex, LoadedMaterial);
					}
				}
			}
		}
	}
	// Rely on the global UpdateCompositeMeshes() post-edit call to update materials into MIDs
	ActorListRef.NotifyPostEditChangeList(EPropertyChangeType::Unspecified);
}

bool SCompositeActorPickerTable::CanEditActors() const
{
	const int32 NumSelected = ListView.IsValid() ? ListView->GetNumItemsSelected() : 0;
	if (NumSelected == 0)
	{
		return false;
	}

	if (!ActorListRef.IsValid())
	{
		return false;
	}

	return true;
}

bool SCompositeActorPickerTable::CanApplyCustomMaterial() const
{
	return CanEditActors() && CustomMaterialAsset.IsValid() && CustomMaterialAsset.IsInstanceOf(UMaterialInterface::StaticClass());
}

FText SCompositeActorPickerTable::GetCustomMaterialSubMenuLabel() const
{
	if (CustomMaterialAsset.IsValid() && CustomMaterialAsset.IsInstanceOf(UMaterialInterface::StaticClass()))
	{
		return FText::FromName(CustomMaterialAsset.AssetName);
	}
	
	return LOCTEXT("SelectCustomMaterial", "Select a Custom Material");
}

void SCompositeActorPickerTable::CreateCustomMaterialSubMenu(FMenuBuilder& InMenuBuilder)
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UMaterialInstance::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SCompositeActorPickerTable::OnCustomMaterialSelected);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.InitialAssetSelection = CustomMaterialAsset;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	InMenuBuilder.AddWidget(
		SNew(SBox)
		.MaxDesiredHeight(600.0f)
		.WidthOverride(256.0f)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		], 
		FText(),
		true
	);
}

void SCompositeActorPickerTable::OnCustomMaterialSelected(const FAssetData& InAssetData)
{
	CustomMaterialAsset = InAssetData;
	FSlateApplication::Get().DismissAllMenus();
}

void SCompositeActorPickerTable::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!ActorListRef.IsValid())
	{
		return;
	}

	if (InObject != ActorListRef.ActorListOwner.Get())
	{
		return;
	}

	if (InPropertyChangedEvent.GetPropertyName() == ActorListRef.ActorListPropertyName)
	{
		FillActorList();
	}
}

#undef LOCTEXT_NAMESPACE
