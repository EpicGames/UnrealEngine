// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositeSpawnableBinding.h"
#include "EditorUndoClient.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "Widgets/SCompoundWidget.h"

class UMaterialInterface;
class UMaterial;
struct FSceneOutlinerFilters;
class SCompositeActorPickerToolbar;
class FMenuBuilder;
class FUICommandList;
template<typename T> class SListView;
class SHeaderRow;

/** Stores a pointer reference to a list of actors on a UObject to be managed by the actor picker */
struct FCompositeActorPickerListRef
{
	DECLARE_DELEGATE_OneParam(FOnActorsAdded, TArray<TSoftObjectPtr<AActor>>&);

public:
	FCompositeActorPickerListRef(const TWeakObjectPtr<UObject>& InActorListOwner, const FName& InActorListPropertyName, TArray<TSoftObjectPtr<AActor>>* InActorList, const FCompositeSpawnableBindings* InSpawnableBindings = nullptr);

	FCompositeActorPickerListRef() { }
	
	/** Gets whether the pointer to the actor list is valid */
	bool IsValid() const;

	/** Raises the owner's PreEditChange event for the actor list property */
	void NotifyPreEditChange();

	/** Raises the owner's PostEditchange event for the actor list property */
	void NotifyPostEditChangeList(EPropertyChangeType::Type InChangeType, int32 LastModifiedIndex = INDEX_NONE);

	/** Finds the localized tooltip or native tooltip as a fallback. */
	FText GetToolTipText(bool bShortTooltip = false) const;
	
public:
	/** Reference to the UObject that owns the actor list being edited by this widget */
	TWeakObjectPtr<UObject> ActorListOwner = nullptr;

	/** The name of the actor list property within ActorListOwner */
	FName ActorListPropertyName = NAME_None;

	/** The property in ActorListOwner of the actor list */
	FProperty* ActorListProperty = nullptr;
	
	/** Pointer to the actor list being edited by this widget */
	TArray<TSoftObjectPtr<AActor>>* ActorList = nullptr;

	/** Raised when any actors are added to the actor list */
	FOnActorsAdded OnActorsAdded;

	/** Optional read-only view of the layer's spawnable side-channel; when non-null the widget shows [Pending Spawnable] for stale entries. */
	const FCompositeSpawnableBindings* SpawnableBindings = nullptr;
};

/**
 * A table that displays a list of selected actors as well as a picker to easily add or remove actors from the current level
 */
class SCompositeActorPickerTable : public SCompoundWidget, public FEditorUndoClient
{
private:
	struct FActorListItem
	{
		/** Soft reference to the actor this item represents */
		TSoftObjectPtr<AActor> Actor;

		/** The index of the actor in the list */
		int32 Index;

		FActorListItem(const TSoftObjectPtr<AActor>& InActor, int32 InIndex)
			: Actor(InActor)
			, Index(InIndex)
		{ }
	};
	using FActorListItemRef = TSharedPtr<FActorListItem>;

public:
	DECLARE_DELEGATE_OneParam(FOnGenerateHeaderColumns, TSharedRef<SHeaderRow> /* InHeaderRow */)
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOnGenerateColumnWidget, const TSoftObjectPtr<AActor>& /* InActor */, const FName& /* ColumnName */)
	DECLARE_DELEGATE_OneParam(FOnExtendAddMenu, FMenuBuilder&);
	DECLARE_DELEGATE_TwoParams(FOnExtendContextMenu, FMenuBuilder&, TArray<TSoftObjectPtr<AActor>>& /*SelectedActors*/);
	DECLARE_DELEGATE_TwoParams(FOnApplyMaterialToActor, TSoftObjectPtr<AActor>& /* InActor */, UMaterialInterface* /* InMaterial */);

public:
	/** Returns default scene outliner filters that check IsAllowedPrimitiveClass and optionally exclude ACompositeMeshActor. */
	static TSharedPtr<FSceneOutlinerFilters> MakeDefaultSceneOutlinerFilters(bool bExcludeCompositeMeshActors = true);

	/**
	 * True if InActor is eligible to be added to a Composite layer/pass actor list.
	 * Non-transient actors always qualify. Transient actors only qualify when they are
	 * sequencer spawnables — those resolve through the parallel FCompositeSpawnableBindings
	 * side-channel. Other transients (editor previews, runtime-only spawns, drag-preview
	 * duplicates) are rejected.
	 */
	static bool IsAllowedActor(AActor* InActor);

	SLATE_BEGIN_ARGS(SCompositeActorPickerTable) : _ShowApplyMaterialSection(false) { }
		SLATE_ATTRIBUTE(TSharedPtr<FSceneOutlinerFilters>, SceneOutlinerFilters)
		SLATE_ARGUMENT(TArray<FName>, HiddenColumnsList)
		SLATE_EVENT(FSimpleDelegate, OnHiddenColumnsListChanged)
		SLATE_EVENT(FOnGenerateHeaderColumns, OnGenerateHeaderColumns)
		SLATE_EVENT(FOnGenerateColumnWidget, OnGenerateColumnWidget)
		SLATE_EVENT(FOnExtendAddMenu, OnExtendAddMenu)
		SLATE_EVENT(FOnExtendContextMenu, OnExtendContextMenu)
		SLATE_EVENT(FSimpleDelegate, OnLayoutSizeChanged)
		SLATE_ATTRIBUTE(bool, ShowApplyMaterialSection)
		SLATE_EVENT(FOnApplyMaterialToActor, OnApplyMaterialToActor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FCompositeActorPickerListRef& InActorListRef);

	virtual ~SCompositeActorPickerTable() override;

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// End of SWidget interface
	
	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** Gets the list of columns that are currently hidden in the actor table */
	TArray<FName> GetHiddenColumnsList() const;

private:
	/** Binds any commands to callbacks for the actor list view */
	void BindCommands();
	
	/** Fills the list view's source list with the valid actors from the actor list and applies any active filters */
	void FillActorList();

	/** Filters the actor list using any active filters */
	void FilterActorList();

	/** Raised when the toolbar filter has changed */
	void OnFilterChanged();

	/** Create the right click context menu for the actor list view */
	TSharedPtr<SWidget> CreateListContextMenu();

	/** Raised when an actor item is selected in the list view */
	void OnActorSelectionChanged(FActorListItemRef ActorListItem, ESelectInfo::Type InSelectType);

	/** Raised when the editor selection is changed */
	void OnLevelSelectionChanged(UObject* NewSelection);

	/** Raised when a drag drop operation is trying to be dropped on the actor list */
	bool OnAllowListDrop(TSharedPtr<FDragDropOperation> InDragDropOperation);

	/** Raised when a drag drop operation is being dropped on the actor list */
	FReply OnListDropped(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent);
	
	/** Gets the filter status text */
	FText GetFilterStatusText() const;

	/** Gets the color for the filter status text */
	FSlateColor GetFilterStatusTextColor() const;
	
	/** Removes the selected actors from the list */
	void RemoveActors();

	/** Apply the specified material to the the selected actors from the list */
	void ApplyMaterialToActors(FString InMaterialPath);

	/** Gets whether the contextual menu commands can be used */
	bool CanEditActors() const;
	
	/** Gets whether a custom material can be applied to the selected actors */
	bool CanApplyCustomMaterial() const;

	/** Gets the label for the custom material sub-menu in the context menu */
	FText GetCustomMaterialSubMenuLabel() const;

	/** Creates the custom material sub-menu in the context menu */
	void CreateCustomMaterialSubMenu(FMenuBuilder& InMenuBuilder);

	/** Raised when a new material is selected in the material browser in the custom material sub-menu */
	void OnCustomMaterialSelected(const FAssetData& InAssetData);
	
	/** Raised when a property on the object has changed */
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	
private:
	/** List view that displays the list of selected actors */
	TSharedPtr<SListView<FActorListItemRef>> ListView;

	/** Header row of the list view that displays the columns of the list */
	TSharedPtr<SHeaderRow> HeaderRow;

	/** The toolbar that contains the Add button and the filter text box */
	TSharedPtr<SCompositeActorPickerToolbar> Toolbar;
	
	/** The reference to the actor list being managed by this actor picker */
	FCompositeActorPickerListRef ActorListRef;

	/** The list of actors to display in the list view */
	TArray<FActorListItemRef> ActorListItems;

	/** Filtered list of actors to display in the list view */
	TArray<FActorListItemRef> FilteredActorListItems;

	/** The command list for commands related to the actor list view */
	TSharedPtr<FUICommandList> CommandList;

	/** Stores the last observed number of widgets the list view is actually displaying, used to determine if the list view's layout has changed */
	int32 CachedListViewNumChildren = 0;
	
	/** Raised when building the Add menu, allows the menu contents to be extended with additional entries */
	FOnExtendAddMenu OnExtendAddMenu;

	/** Raised when building the context menu for entries in the table, allows the context menu contents to be extended with additional entries */
	FOnExtendContextMenu OnExtendContextMenu;

	/** Raised when the number of items in the list view changes */
	FSimpleDelegate OnLayoutSizeChanged;

	/** Flag to enable the material application section for selected actors. */
	TAttribute<bool> bShowApplyMaterialSection;

	/** Selected custom material to apply to selected actors */
	FAssetData CustomMaterialAsset = FAssetData();

	/** Raised when a material is being applied to an actor. Allows for custom logic on how the material is applied */
	FOnApplyMaterialToActor OnApplyMaterialToActor;

	friend class SCompositeActorPickerToolbar;
	friend class SCompositeActorListItemRow;
};
