// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompositeEditorPanel.h"

#include "CompositeActor.h"
#include "CompositeEditorStyle.h"
#include "Components/CompositeViewProjectionComponent.h"
#include "Customizations/CompositeCustomizationHelpers.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomization.h"
#include "LevelEditor.h"
#include "SCompositePanelLayerTree.h"
#include "SCompositePassTreePanel.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Customizations/CompositeLayerCustomization.h"
#include "Customizations/CompositeLayerPlanarReflectionCustomization.h"
#include "Customizations/CompositeLayerPlateCustomization.h"
#include "Customizations/CompositeLayerSceneCaptureCustomization.h"
#include "Customizations/CompositeLayerShadowReflectionCustomization.h"
#include "Customizations/CompositeLayerSimplePassesCustomization.h"
#include "Customizations/CompositeLayerSingleLightShadowCustomization.h"
#include "Sequencer/CompositeDetailKeyframeHandler.h"
#include "Layers/CompositeLayerBase.h"
#include "Layers/CompositeLayerMainRender.h"
#include "Layers/CompositeLayerPlanarReflection.h"
#include "Layers/CompositeLayerPlate.h"
#include "Layers/CompositeLayerProcessing.h"
#include "Layers/CompositeLayerSceneCapture.h"
#include "Layers/CompositeLayerShadowReflection.h"
#include "Layers/CompositeLayerSingleLightShadow.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SCompositeEditorPanel"

namespace CompositingEditorPanel
{
	static const FName CompositeEditorTabName = "CompositeEditorTab";

	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& InArgs)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("PanelTabLabel", "Composure"))
			.TabRole(ETabRole::PanelTab)
			[
				SNew(SCompositeEditorPanel)
			];
	}
}

class FCompositeActorPanelDetailCustomization : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		
		// Hide all categories besides the Composite category
		static const FName CompositeCategoryName = "Composite";
		
		TArray<FName> CategoryNames;
		DetailBuilder.GetCategoryNames(CategoryNames);

		for (const FName& CategoryName : CategoryNames)
		{
			if (CategoryName == CompositeCategoryName)
			{
				continue;
			}
			
			DetailBuilder.HideCategory(CategoryName);
		}

		// TransformCommon is a custom category that doesn't get returned by GetCategoryNames that also needs to be hidden
		DetailBuilder.HideCategory(TEXT("TransformCommon"));

		IDetailCategoryBuilder& CompositeCategory = DetailBuilder.EditCategory(CompositeCategoryName, LOCTEXT("CompositeActorCategoryName", "Composite Actor"));

		CompositeCategory.AddCustomRow(LOCTEXT("CompositeActorNameFilterText", "Name"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CompositeActorNameProperty", "Name"))
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			]
			.ValueContent()
			[
				SNew(SEditableTextBox)
				.Text_Lambda([this]
				{
					if (ObjectsBeingCustomized.Num() == 1 && ObjectsBeingCustomized[0].IsValid() && ObjectsBeingCustomized[0]->IsA<AActor>())
					{
						return FText::FromString(Cast<AActor>(ObjectsBeingCustomized[0])->GetActorLabel());
					}
					if (ObjectsBeingCustomized.Num() > 1)
					{
						return LOCTEXT("MultipleValuesLabel", "Multiple Values");
					}

					return FText::GetEmpty();
				})
				.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
				{
					if (ObjectsBeingCustomized.Num() == 1 && ObjectsBeingCustomized[0].IsValid() && ObjectsBeingCustomized[0]->IsA<AActor>())
					{
						FScopedTransaction RenameActorTransaction(LOCTEXT("RenameActorTransaction", "Rename Composite Actor"));
						FActorLabelUtilities::RenameExistingActor(Cast<AActor>(ObjectsBeingCustomized[0]), InText.ToString(), true);
					}
				})
				.IsEnabled_Lambda([this]
				{
					return ObjectsBeingCustomized.Num() == 1;
				})
			];
		
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ACompositeActor, bIsActive));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));

		TSharedPtr<IPropertyHandle> CameraActorHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositeActor, CameraActor));
		CompositeCustomizationHelpers::CustomizeCameraActorProperty(DetailBuilder, CameraActorHandle);

		// Surface the view projection component's bIsEnabled at the top of the Composite advanced section
		// so users can toggle it without drilling into the component.
		CompositeCategory.AddCustomRow(LOCTEXT("EnableCameraProjectionFilterText", "Enable Camera Projection"), /*bForAdvanced=*/ true)
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("EnableCameraProjectionLabel", "Enable Camera Projection"))
				.ToolTipText(LOCTEXT("EnableCameraProjectionTooltip", "Convenience toggle that drives the enabled state of the Composite Actor's Camera View Projection component.\nDisabling it detaches the projection source from the camera, freezing the Material Parameter Collection at its last value."))
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FCompositeActorPanelDetailCustomization::GetCameraProjectionEnabledState)
				.OnCheckStateChanged(this, &FCompositeActorPanelDetailCustomization::OnCameraProjectionEnabledChanged)
			];
	}

private:
	ECheckBoxState GetCameraProjectionEnabledState() const
	{
		TOptional<bool> Aggregate;
		for (const TWeakObjectPtr<UObject>& WeakObject : ObjectsBeingCustomized)
		{
			ACompositeActor* Actor = Cast<ACompositeActor>(WeakObject.Get());
			if (!Actor)
			{
				continue;
			}

			UCompositeViewProjectionComponent* Component = Actor->GetComponentByClass<UCompositeViewProjectionComponent>();
			if (!Component)
			{
				continue;
			}

			if (!Aggregate.IsSet())
			{
				Aggregate = Component->bIsEnabled;
			}
			else if (Aggregate.GetValue() != Component->bIsEnabled)
			{
				return ECheckBoxState::Undetermined;
			}
		}

		if (!Aggregate.IsSet())
		{
			return ECheckBoxState::Unchecked;
		}
		return Aggregate.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnCameraProjectionEnabledChanged(ECheckBoxState NewState)
	{
		if (NewState == ECheckBoxState::Undetermined || GetCameraProjectionEnabledState() == NewState)
		{
			return;
		}

		const bool bNewValue = (NewState == ECheckBoxState::Checked);

		FProperty* Property = FindFProperty<FProperty>(UCompositeViewProjectionComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeViewProjectionComponent, bIsEnabled));
		FPropertyChangedEvent PropertyEvent(Property, EPropertyChangeType::ValueSet);

		FScopedTransaction Transaction(LOCTEXT("SetEnableCameraProjection", "Set Enable Camera Projection"));

		for (const TWeakObjectPtr<UObject>& WeakObject : ObjectsBeingCustomized)
		{
			ACompositeActor* Actor = Cast<ACompositeActor>(WeakObject.Get());
			if (!Actor)
			{
				continue;
			}

			UCompositeViewProjectionComponent* Component = Actor->GetComponentByClass<UCompositeViewProjectionComponent>();
			if (!Component || Component->bIsEnabled == bNewValue)
			{
				continue;
			}

			Component->PreEditChange(Property);
			Component->bIsEnabled = bNewValue;
			Component->PostEditChangeProperty(PropertyEvent);
		}
	}

private:
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
};

void SCompositeEditorPanel::RegisterTabSpawner()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
	{
		LevelEditorTabManager->RegisterTabSpawner(CompositingEditorPanel::CompositeEditorTabName, FOnSpawnTab::CreateStatic(&CompositingEditorPanel::CreateTab))
			.SetDisplayName(LOCTEXT("PanelDisplayName", "Composure"))
			.SetTooltipText(LOCTEXT("PanelTooltipText", "Panel for viewing and editing compositing actors in the current level"))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory())
			.SetIcon(FSlateIcon(FCompositeEditorStyle::Get().GetStyleSetName(), "CompositeEditor.Composure"));
	}
}

void SCompositeEditorPanel::UnregisterTabSpawner()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
	{
		LevelEditorTabManager->UnregisterTabSpawner(CompositingEditorPanel::CompositeEditorTabName);
	}
}

void SCompositeEditorPanel::UpdateActivePanelSelection(const TArray<UObject*>& InSelection)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
	{
		if (TSharedPtr<SDockTab> Tab = LevelEditorTabManager->FindExistingLiveTab(CompositingEditorPanel::CompositeEditorTabName))
		{
			if (TSharedPtr<SCompositeEditorPanel> CompositePanel = StaticCastSharedPtr<SCompositeEditorPanel>(Tab->GetContent().ToSharedPtr()))
			{
				CompositePanel->SelectObjects(InSelection);

				// If the composure panel has a valid selection after selecting the objects, draw attention to the panel so it gets put into view
				if (CompositePanel->GetSelectedObjects().Num() > 0)
				{
					LevelEditorTabManager->DrawAttention(Tab.ToSharedRef());
				}
			}
		}
	}
}

TArray<UObject*> SCompositeEditorPanel::GetActivePanelSelection()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
	{
		if (TSharedPtr<SDockTab> Tab = LevelEditorTabManager->FindExistingLiveTab(CompositingEditorPanel::CompositeEditorTabName))
		{
			if (TSharedPtr<SCompositeEditorPanel> CompositePanel = StaticCastSharedPtr<SCompositeEditorPanel>(Tab->GetContent().ToSharedPtr()))
			{
				return CompositePanel->GetSelectedObjects();
			}
		}
	}

	return { };
}

void SCompositeEditorPanel::Construct(const FArguments& InArgs)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	
	DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	DetailsView->SetKeyframeHandler(MakeShared<FCompositeDetailKeyframeHandler>());
	DetailsView->RegisterInstancedCustomPropertyLayout(ACompositeActor::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([]
	{
		return MakeShared<FCompositeActorPanelDetailCustomization>();
	}));

	// Create instanced versions of all layer customizations so that pointers to them can be stored
	// in InstancedCustomizations and accessed later for selected-pass aggregation in the panel.
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerPlate::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerPlate, FCompositeLayerPlateCustomization>));
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerSceneCapture::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerSceneCapture, FCompositeLayerSceneCaptureCustomization>));
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerPlanarReflection::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerPlanarReflection, FCompositeLayerPlanarReflectionCustomization>));
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerShadowReflection::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerShadowReflection, FCompositeLayerShadowReflectionCustomization>));
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerSingleLightShadow::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerSingleLightShadow, FCompositeLayerSingleLightShadowCustomization>));
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerMainRender::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerMainRender, FCompositeLayerSimplePassesCustomization>));
	DetailsView->RegisterInstancedCustomPropertyLayout(UCompositeLayerProcessing::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateSP(this, &SCompositeEditorPanel::CreateCustomization<UCompositeLayerProcessing, FCompositeLayerSimplePassesCustomization>));
	
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(EOrientation::Orient_Vertical)

		+SSplitter::Slot()
		.Value(0.25f)
		.MinSize(this, &SCompositeEditorPanel::GetLayerTreeMinHeight)
		[
			SAssignNew(LayerTree, SCompositePanelLayerTree)
			.OnSelectionChanged(this, &SCompositeEditorPanel::OnLayerSelectionChanged)
			.OnLayerStateChanged(this, &SCompositeEditorPanel::OnLayerStateChanged)
		]

		+SSplitter::Slot()
		.Value(0.75f)
		[
			DetailsView.ToSharedRef()
		]
	];
}

void SCompositeEditorPanel::SelectCompositeActors(const TArray<TWeakObjectPtr<ACompositeActor>>& InCompositeActors)
{
	if (LayerTree.IsValid())
	{
		LayerTree->SelectCompositeActors(InCompositeActors);
	}
}

void SCompositeEditorPanel::SelectObjects(const TArray<UObject*>& InSelection)
{
	TArray<UObject*> CompositeActorsAndLayers;
	TMap<UCompositeLayerBase*, TArray<UCompositePassBase*>> CompositePasses;

	for (UObject* Object : InSelection)
	{
		if (!Object)
		{
			continue;
		}
		
		if (Object->IsA<ACompositeActor>() || Object->IsA<UCompositeLayerBase>())
		{
			CompositeActorsAndLayers.Add(Object);
		}

		UCompositePassBase* Pass = Cast<UCompositePassBase>(Object);
		if (Pass && !Pass->IsA<UCompositeLayerBase>())
		{
			if (UCompositeLayerBase* ParentLayer = Pass->GetTypedOuter<UCompositeLayerBase>())
			{
				CompositeActorsAndLayers.Add(ParentLayer);

				if (!CompositePasses.Contains(ParentLayer))
				{
					CompositePasses.Add(ParentLayer);
				}
				
				CompositePasses[ParentLayer].Add(Pass);
			}
		}
	}
	
	if (LayerTree.IsValid())
	{
		LayerTree->SelectObjects(CompositeActorsAndLayers);
	}

	for (const TTuple<UClass*, TArray<TWeakPtr<IDetailCustomization>>>& CustomizationsForClass : InstancedCustomizations)
	{
		UClass* LayerClass = CustomizationsForClass.Key;
		if (!LayerClass || !LayerClass->IsChildOf(UCompositeLayerBase::StaticClass()))
		{
			continue;
		}

		for (const TPair<UCompositeLayerBase*, TArray<UCompositePassBase*>>& PassesForLayer : CompositePasses)
		{
			UCompositeLayerBase* Layer = PassesForLayer.Key;
			if (!Layer || !Layer->IsA(LayerClass))
			{
				continue;
			}

			TArray<UCompositePassBase*> Passes = PassesForLayer.Value;

			for (const TWeakPtr<IDetailCustomization>& WeakCustomization : CustomizationsForClass.Value)
			{
				TSharedPtr<IDetailCustomization> Pinned = WeakCustomization.Pin();
				if (!Pinned.IsValid())
				{
					continue;
				}

				// Customizations registered on UCompositeLayerBase-derived classes are guaranteed
				// to derive from FCompositeLayerCustomization, so this static_cast is well-defined.
				FCompositeLayerCustomization* Host = static_cast<FCompositeLayerCustomization*>(Pinned.Get());
				if (!Host->IsCustomizingObject(Layer))
				{
					continue;
				}

				if (TSharedPtr<SCompositePassTreePanel> PassPanel = Host->GetPassPanelWidget())
				{
					PassPanel->SelectPasses(Passes);
				}
			}
		}
	}
}

TArray<UObject*> SCompositeEditorPanel::GetSelectedObjects() const
{
	TArray<UObject*> SelectedObjects;
	if (LayerTree.IsValid())
	{
		SelectedObjects.Append(LayerTree->GetSelectedObjects());
	}

	for (const TTuple<UClass*, TArray<TWeakPtr<IDetailCustomization>>>& CustomizationsForClass : InstancedCustomizations)
	{
		UClass* LayerClass = CustomizationsForClass.Key;
		if (!LayerClass || !LayerClass->IsChildOf(UCompositeLayerBase::StaticClass()))
		{
			continue;
		}

		for (const TWeakPtr<IDetailCustomization>& WeakCustomization : CustomizationsForClass.Value)
		{
			TSharedPtr<IDetailCustomization> Pinned = WeakCustomization.Pin();
			if (!Pinned.IsValid())
			{
				continue;
			}

			// Customizations registered on UCompositeLayerBase-derived classes are guaranteed
			// to derive from FCompositeLayerCustomization, so this static_cast is well-defined.
			FCompositeLayerCustomization* Host = static_cast<FCompositeLayerCustomization*>(Pinned.Get());
			if (TSharedPtr<SCompositePassTreePanel> PassPanel = Host->GetPassPanelWidget())
			{
				SelectedObjects.Append(PassPanel->GetSelectedPasses());
			}
		}
	}

	return SelectedObjects;
}

void SCompositeEditorPanel::OnLayerSelectionChanged(const TArray<UObject*>& SelectedLayers)
{
	if (DetailsView.IsValid())
	{
		// Clear all stored customizations before changing the details view's objects, as it will recreate any customizations it needs
		InstancedCustomizations.Empty();

		DetailsView->SetObjects(SelectedLayers);
	}

	GetOnSelectionChanged().Broadcast();
}

void SCompositeEditorPanel::OnLayerStateChanged()
{
	if (DetailsView.IsValid())
	{
		DetailsView->ForceRefresh();
	}
}

float SCompositeEditorPanel::GetLayerTreeMinHeight() const
{
	if (LayerTree.IsValid())
	{
		const float PanelHeight = GetCachedGeometry().GetAbsoluteSize().Y;
		const float LayerTreeHeight = LayerTree->GetMinimumHeight();

		// Splitter panels get confused if a slot has a minimum size larger than the splitter, so only
		// return a non-zero minimum when whole panel is larger than the minimum
		if (PanelHeight > LayerTreeHeight)
		{
			return LayerTreeHeight;
		}
	}

	return 0.0f;
}

#undef LOCTEXT_NAMESPACE
