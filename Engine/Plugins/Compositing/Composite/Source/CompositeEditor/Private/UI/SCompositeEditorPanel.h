// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UCompositePassBase;
class IDetailCustomization;
class ACompositeActor;
class IDetailsView;
class SCompositePanelLayerTree;

/**
 * Editor that displays the compositing actors in the level and displays the layer tree and layer properties for them
 */
class SCompositeEditorPanel : public SCompoundWidget
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged)
	
	/** Registers the tab spawner for the panel with the editor */
	static void RegisterTabSpawner();

	/** Unregisters the tab spawner for the panel with the editor */
	static void UnregisterTabSpawner();

	/** Updates the selected composite actors, layers, and passes in the active Composure panel, if one is open */
	static void UpdateActivePanelSelection(const TArray<UObject*>& InSelection);

	/** Gets the currently selected composite actors, layers, and passes in the active Composure panel, if one is open */
	static TArray<UObject*> GetActivePanelSelection();

	/** Static delegate that is raised whenever the selection is changed in the active Composure panel */
	static FOnSelectionChanged& GetOnSelectionChanged() { static FOnSelectionChanged Instance; return Instance; }
	
public:
	SLATE_BEGIN_ARGS(SCompositeEditorPanel) { }
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	/** Selects the specified composite actors in the layer tree */
	void SelectCompositeActors(const TArray<TWeakObjectPtr<ACompositeActor>>& InCompositeActors);
	
private:
	/** Selects any composite actors, layers, and passes in the specified list of objects */
	void SelectObjects(const TArray<UObject*>& InSelection);

	/** Gets the actively selected composite actors, layers, and passes in this panel */
	TArray<UObject*> GetSelectedObjects() const;

	/** Raised when the selected layers in the tree view have changed */
	void OnLayerSelectionChanged(const TArray<UObject*>& SelectedLayers);

	/** Raised when a layer's editable state changes (e.g. Solo toggled); forces details view to re-evaluate CanEditChange */
	void OnLayerStateChanged();

	/** Gets the minimum height to display the full layer tree */
	float GetLayerTreeMinHeight() const;

	/**
	 * Generic callback to create a customization of the specified type for the specified object type being customized.
	 * A weak reference to the customization will be stored in the InstancedCustomizations map keyed under the type being customized
	 */
	template<typename TClass, typename TCustomization>
	TSharedRef<IDetailCustomization> CreateCustomization()
	{
		TSharedRef<IDetailCustomization> Customization = TCustomization::MakeInstance();

		if (!InstancedCustomizations.Contains(TClass::StaticClass()))
		{
			InstancedCustomizations.Add(TClass::StaticClass(), { });
		}

		InstancedCustomizations[TClass::StaticClass()].Add(Customization);
	
		return Customization;
	}
	
private:
	/** The tree view widget that displays the composite layers */
	TSharedPtr<SCompositePanelLayerTree> LayerTree;

	/** The details view that displays the properties of the selected layers and actors */
	TSharedPtr<IDetailsView> DetailsView;

	/** A list of pointers to details customizations in use by the details panel, indexed by class being customized */
	TMap<UClass*, TArray<TWeakPtr<IDetailCustomization>>> InstancedCustomizations;
};
