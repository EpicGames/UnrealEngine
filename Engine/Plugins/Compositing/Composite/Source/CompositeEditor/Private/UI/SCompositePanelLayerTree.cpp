// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompositePanelLayerTree.h"

#include "ActorFactories/ActorFactory.h"
#include "CompositeActor.h"
#include "CompositeEditorCommands.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Factories.h"
#include "Kismet/KismetMaterialLibrary.h"
#include "LevelEditor.h"
#include "LevelEditorSubsystem.h"
#include "Materials/MaterialParameterCollection.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "UnrealExporter.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Exporters/Exporter.h"
#include "Filters/SBasicFilterBar.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "CompositeAnalytics.h"
#include "CompositeEditorStyle.h"
#include "MediaProfileMenus.h"
#include "Layers/CompositeLayerBase.h"
#include "SLevelViewport.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "UObject/Class.h"
#include "UObject/SoftObjectPath.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SCompositePanelLayerTree"

namespace CompositePanelLayerTree
{
	static const FName LayerTreeColumn_Active = "Active";
	static const FName LayerTreeColumn_Enabled = "Enabled";
	static const FName LayerTreeColumn_Solo = "Solo";
	static const FName LayerTreeColumn_Name = "Name";
	static const FName LayerTreeColumn_Type = "Type";
	static const FName LayerTreeColumn_Operation = "Operation";

	static const FLazyName MeshVisualizationParameterName(TEXT("MeshVisualization"));
	static const FSoftObjectPath MPCCompositePath(TEXT("/Composite/Materials/MPC_Composite.MPC_Composite"));
}

/** Toolbar combo button that opens a radio menu of ECompositeMeshVisualizationMode entries. */
class SCompositeMeshVisualizationToolBarButton : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FOnGetMenuContent)

	SLATE_BEGIN_ARGS(SCompositeMeshVisualizationToolBarButton) {}
		SLATE_ARGUMENT(FText, ToolTipText)
		SLATE_ARGUMENT(const FSlateBrush*, IconBrush)
		SLATE_ATTRIBUTE(FSlateColor, IconColor)
		SLATE_EVENT(FOnGetMenuContent, OnGetMenuContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnGetMenuContent = InArgs._OnGetMenuContent;

		SetToolTipText(InArgs._ToolTipText);

		// Compose a combo style whose outer button uses the raised (transparent-normal) button style so the
		// background brightness matches the pilot toggle next to it.
		const FToolBarStyle& RaisedToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("ViewportToolbar.Raised");
		ComboStyle = FComboButtonStyle(RaisedToolbarStyle.ComboButtonStyle)
			.SetButtonStyle(RaisedToolbarStyle.ButtonStyle);

		ChildSlot
		[
			SNew(SComboButton)
			.ContentPadding(FMargin(2.0f, 4.0f))
			.ComboButtonStyle(&ComboStyle)
			.HasDownArrow(true)
			.OnGetMenuContent(this, &SCompositeMeshVisualizationToolBarButton::HandleGetMenuContent)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(InArgs._IconBrush)
				.ColorAndOpacity(InArgs._IconColor)
			]
		];
	}

private:
	TSharedRef<SWidget> HandleGetMenuContent()
	{
		return OnGetMenuContent.IsBound() ? OnGetMenuContent.Execute() : SNullWidget::NullWidget;
	}

	FOnGetMenuContent OnGetMenuContent;

	/** Composed combo style with the raised ButtonStyle grafted onto the raised ComboButtonStyle; storage must outlive the SComboButton. */
	FComboButtonStyle ComboStyle;
};

/** Toolbar widget that contains filtering, the add button, and the settings button */
class SCompositePanelLayerTreeToolbar : public SCompoundWidget
{
private:
	using FFilterType = SCompositePanelLayerTree::FCompositeActorTreeItemPtr;
	using FCompositeLayersFilter = TTextFilter<FFilterType>;
	
public:
	DECLARE_DELEGATE_OneParam(FOnPilotCameraToggled, bool);
	DECLARE_DELEGATE_OneParam(FOnLayerAdded, const UClass*)
	DECLARE_DELEGATE_OneParam(FOnMeshVisualizationModeChanged, ECompositeMeshVisualizationMode)

	SLATE_BEGIN_ARGS(SCompositePanelLayerTreeToolbar) {}
		SLATE_ATTRIBUTE(bool, CanAddLayer)
		SLATE_ATTRIBUTE(bool, CanPilotCamera)
		SLATE_ATTRIBUTE(bool, IsPilotingCamera)
		SLATE_ATTRIBUTE(ECompositeMeshVisualizationMode, MeshVisualizationMode)
		SLATE_EVENT(FOnPilotCameraToggled, OnPilotCameraToggled)
		SLATE_EVENT(FOnMeshVisualizationModeChanged, OnMeshVisualizationModeChanged)
		SLATE_EVENT(FSimpleDelegate, OnFilterChanged)
		SLATE_EVENT(FSimpleDelegate, OnCompositeActorAdded)
		SLATE_EVENT(FOnLayerAdded, OnLayerAdded)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		IsPilotingCamera = InArgs._IsPilotingCamera;
		CanAddLayer = InArgs._CanAddLayer;
		MeshVisualizationMode = InArgs._MeshVisualizationMode;
		OnPilotCameraToggled = InArgs._OnPilotCameraToggled;
		OnMeshVisualizationModeChanged = InArgs._OnMeshVisualizationModeChanged;
		OnFilterChanged = InArgs._OnFilterChanged;
		OnCompositeActorAdded = InArgs._OnCompositeActorAdded;
		OnLayerAdded = InArgs._OnLayerAdded;

		InitializeFilters();
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SPositiveActionButton)
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Text(LOCTEXT("AddMediaButtonLabel", "Add"))
					.ToolTipText(LOCTEXT("AddMediaButtonToolTip", "Add new composite layer to the selected composite actor"))
					.OnGetMenuContent(this, &SCompositePanelLayerTreeToolbar::GetAddMenuContent)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 0.f, 4.f, 0.f)
				[
					SNew(SCheckBox)
					.Padding(FMargin(8.0f, 8.0f))
					.Style(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("ViewportToolbar.Raised").ToggleButton)
					.Type(ESlateCheckBoxType::ToggleButton)
					.ToolTipText(LOCTEXT("PilotButtonToolTip", "Pilot the camera actor associated with the selected composite actor"))
					.IsEnabled(InArgs._CanPilotCamera)
					.IsChecked_Lambda([this]
					{
						return IsPilotingCamera.Get(false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckState)
					{
						OnPilotCameraToggled.ExecuteIfBound(CheckState == ECheckBoxState::Checked);
					})
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("LevelViewport.PilotSelectedActor"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(SCompositeMeshVisualizationToolBarButton)
					.ToolTipText(LOCTEXT("MeshVisualizationButtonToolTip",
						"Composite mesh visualization mode.\n\n"
						"Drives the scalar parameter \"MeshVisualization\" on the material parameter collection MPC_Composite (0 = off, 1 = on).\n\n"
						"The default plugin composite-mesh materials sample this parameter to overlay a debug UV grid. "
						"Custom materials can opt in by sampling the same parameter on MPC_Composite and using the "
						"MF_CompositeMeshVizGrid material function to generate the grid mask."))
					.IconBrush(FCompositeEditorStyle::Get().GetBrush("CompositeEditor.CompositeMeshVisualization"))
					.IconColor_Lambda([this]() -> FSlateColor
					{
						return MeshVisualizationMode.Get(ECompositeMeshVisualizationMode::Disabled) != ECompositeMeshVisualizationMode::Disabled
							? FSlateColor(FStyleColors::AccentBlue)
							: FSlateColor::UseForeground();
					})
					.OnGetMenuContent(this, &SCompositePanelLayerTreeToolbar::GetMeshVisualizationMenuContent)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					UE::MediaProfile::Menus::CreateMediaProfileToolBarButton(LOCTEXT("MediaProfileToolBarButtonTooltip", "Open or select a media profile"))
				]
				
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SFilterSearchBox)
					.HintText(LOCTEXT("FilterSearch", "Search..."))
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search for specific composite layers"))
					.OnTextChanged_Lambda([this](const FText& InText)
					{
						TextFilter->SetRawFilterText(InText);
					})
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]
				]
			]
		];
	}

	/** Gets whether the specified tree item passes any filters enabled in the toolbar */
	bool ItemPassesFilters(const FFilterType& InItem) const
	{
		return TextFilter->PassesFilter(InItem);
	}

	/** Clears the toolbar filters */
	void ClearFilter()
	{
		TextFilter->SetRawFilterText(FText::GetEmpty());
	}
	
private:
	/** Configures any filters for the toolbar */
	void InitializeFilters()
	{
		TextFilter = MakeShared<FCompositeLayersFilter>(FCompositeLayersFilter::FItemToStringArray::CreateLambda([](const FFilterType& InItem, TArray<FString>& OutStrings)
		{
			if (!InItem.IsValid())
			{
				return;
			}

			if (!InItem->HasValidCompositeActor())
			{
				return;
			}

			OutStrings.Add(InItem->CompositeActor->GetActorNameOrLabel());
			OutStrings.Add(InItem->CompositeActor->GetClass()->GetDisplayNameText().ToString());
				
			if (UCompositeLayerBase* CompositeLayer = InItem->GetCompositeLayer())
			{
				OutStrings.Add(CompositeLayer->GetDisplayName());
				OutStrings.Add(CompositeLayer->GetClass()->GetDisplayNameText().ToString());
			}
		}));
		
		TextFilter->OnChanged().AddSP(this, &SCompositePanelLayerTreeToolbar::FilterChanged);
	}

	/** Gets the menu content of the Add button, which will allow users to pick which kind of layer they want to add */
	TSharedRef<SWidget> GetAddMenuContent()
	{
		constexpr bool bCloseMenuAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

		MenuBuilder.BeginSection("Actor", LOCTEXT("AddActorSectionLabel", "Actor"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PlaceCompositeActorLabel", "Place Composite Actor"),
				LOCTEXT("PlaceCompositeActorToolTip", "Add a new Composite Actor to the level"),
				FSlateIconFinder::FindIconForClass(ACompositeActor::StaticClass()),
				FUIAction(FExecuteAction::CreateSP(this, &SCompositePanelLayerTreeToolbar::AddActor)));
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Actor", LOCTEXT("AddLayerSectionLabel", "Layer"));
		
		TArray<UClass*> BaseLayerTypes;
		const bool bRecursive = false;
		GetDerivedClasses(UCompositeLayerBase::StaticClass(), BaseLayerTypes, bRecursive);

		auto ShouldSkipClass = [](const UClass* InClass)
		{
			constexpr EClassFlags InvalidClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract;
			return InClass->HasAnyClassFlags(InvalidClassFlags) ||
				InClass->GetName().StartsWith(TEXT("SKEL_")) ||
				InClass->GetName().StartsWith(TEXT("REINST_"));
		};
		
		// Add the native child classes of UCompositeLayerBase to the menu first
		for (const UClass* BaseLayerType : BaseLayerTypes)
		{
			if (ShouldSkipClass(BaseLayerType))
			{
				continue;
			}
			
			MenuBuilder.AddMenuEntry(
				BaseLayerType->GetDisplayNameText(),
				BaseLayerType->GetToolTipText(),
				FSlateIconFinder::FindIconForClass(BaseLayerType),
				FUIAction(
					FExecuteAction::CreateSP(this, &SCompositePanelLayerTreeToolbar::AddLayer, BaseLayerType),
					FCanExecuteAction::CreateLambda([this] { return CanAddLayer.Get(true); })));
		}
				
		// Add any blueprint derived classes in a new section of the menu
		bool bHasAddedSeparator = false;
		for (const UClass* BaseLayerType : BaseLayerTypes)
		{
			TArray<UClass*> ChildLayerTypes;
			GetDerivedClasses(BaseLayerType, ChildLayerTypes);

			for (const UClass* ChildLayerType : ChildLayerTypes)
			{
				if (ShouldSkipClass(ChildLayerType))
				{
					continue;
				}
				
				if (!bHasAddedSeparator)
				{
					MenuBuilder.AddSeparator();
					bHasAddedSeparator = true;
				}

				MenuBuilder.AddMenuEntry(
					ChildLayerType->GetDisplayNameText(),
					ChildLayerType->GetToolTipText(),
					FSlateIconFinder::FindIconForClass(ChildLayerType),
					FUIAction(
						FExecuteAction::CreateSP(this, &SCompositePanelLayerTreeToolbar::AddLayer, ChildLayerType),
						FCanExecuteAction::CreateLambda([this] { return CanAddLayer.Get(true); })));
			}
		}

		MenuBuilder.EndSection();
		
		return MenuBuilder.MakeWidget();
	}

	/** Builds the composite mesh visualization radio menu. */
	TSharedRef<SWidget> GetMeshVisualizationMenuContent()
	{
		constexpr bool bCloseMenuAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

		MenuBuilder.BeginSection("CompositeMeshVisualization", LOCTEXT("MeshVisualizationSectionLabel", "Composite Mesh Visualization"));
		{
			auto MakeEntry = [this, &MenuBuilder](ECompositeMeshVisualizationMode InMode, const FText& Label, const FText& ToolTip)
			{
				MenuBuilder.AddMenuEntry(
					Label,
					ToolTip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, InMode]
						{
							OnMeshVisualizationModeChanged.ExecuteIfBound(InMode);
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([this, InMode]
						{
							return MeshVisualizationMode.Get(ECompositeMeshVisualizationMode::Disabled) == InMode;
						})),
					NAME_None,
					EUserInterfaceActionType::RadioButton);
			};

			MakeEntry(
				ECompositeMeshVisualizationMode::Disabled,
				LOCTEXT("MeshVisualizationDisabled", "Disable Visualization"),
				LOCTEXT("MeshVisualizationDisabledToolTip", "Composite mesh visualization is off."));

			MakeEntry(
				ECompositeMeshVisualizationMode::EnabledWhenNotPiloting,
				LOCTEXT("MeshVisualizationEnabledWhenNotPiloting", "Visualize When Not Piloting"),
				LOCTEXT("MeshVisualizationEnabledWhenNotPilotingToolTip", "Show the composite mesh debug grid only when the active composite actor's camera is not being piloted."));

			MakeEntry(
				ECompositeMeshVisualizationMode::Enabled,
				LOCTEXT("MeshVisualizationEnabled", "Always Visualize"),
				LOCTEXT("MeshVisualizationEnabledToolTip", "Always show the composite mesh debug grid."));
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	/** Callback raised when 'Place Composite Actor' is selected in the Add dropdown menu */
	void AddActor()
	{
		OnCompositeActorAdded.ExecuteIfBound();
	}
	
	/** Callback raised when a layer type is selected in the Add dropdown menu */
	void AddLayer(const UClass* InLayerType)
	{
		OnLayerAdded.ExecuteIfBound(InLayerType);
	}

	/** Callback raised when the toolbar's active filter has changed */
	void FilterChanged()
	{
		OnFilterChanged.ExecuteIfBound();
	}
	
private:
	/** Attribute used to query if a camera actor is actively being piloted in the editor */
	TAttribute<bool> IsPilotingCamera;

	/** Gets whether there is a valid composite actor that a layer can be added to */
	TAttribute<bool> CanAddLayer;

	/** Attribute used to query the current composite mesh visualization mode from the editor settings */
	TAttribute<ECompositeMeshVisualizationMode> MeshVisualizationMode;

	/** Manages the text filter that has been entered into the filter text box */
	TSharedPtr<FCompositeLayersFilter> TextFilter;

	FOnPilotCameraToggled OnPilotCameraToggled;
	FOnMeshVisualizationModeChanged OnMeshVisualizationModeChanged;
	FSimpleDelegate OnFilterChanged;
	FSimpleDelegate OnCompositeActorAdded;
	FOnLayerAdded OnLayerAdded;
};

/** Row widget used to display a tree item in the composite layer tree */
class SCompositePanelLayerTreeItemRow : public SMultiColumnTableRow<SCompositePanelLayerTree::FCompositeActorTreeItemPtr>
{
	using FTreeItem = SCompositePanelLayerTree::FCompositeActorTreeItem;
	using FTreeItemPtr = SCompositePanelLayerTree::FCompositeActorTreeItemPtr;
	using FTreeItemWeakPtr = TWeakPtr<FTreeItem>;
	
public:
	DECLARE_DELEGATE_OneParam(FOnCompositeActorActivated, const TStrongObjectPtr<ACompositeActor>&)
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnLayerSoloToggled, const FTreeItemPtr&)
	DECLARE_DELEGATE_TwoParams(FOnLayerMoved, const FTreeItemPtr& /* InItemToMove */, int32 /* InDestIndex */)

private:
	/** Drag/drop operation for changing the order of sources or outputs in the tree view */
	class FCompositeLayerDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FCompositeLayerDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FCompositeLayerDragDropOp> New(const FTreeItemPtr& InDraggedItem)
		{
			TSharedRef<FCompositeLayerDragDropOp> Operation = MakeShared<FCompositeLayerDragDropOp>();
			Operation->DraggedItem = InDraggedItem;
			Operation->Construct();
			return Operation;
		}

		virtual void Construct() override
		{
			if (FTreeItemPtr PinnedItem = DraggedItem.Pin())
			{
				if (UCompositeLayerBase* CompositeLayer = PinnedItem->GetCompositeLayer())
				{
					const FString LayerName = !CompositeLayer->GetDisplayName().IsEmpty() ? CompositeLayer->GetDisplayName() : CompositeLayer->GetName();
					const FSlateBrush* LayerIcon = FSlateIconFinder::FindIconForClass(CompositeLayer->GetClass()).GetIcon();
					SetToolTip(FText::FromString(LayerName), LayerIcon);
				}
			}
			
			FDecoratedDragDropOp::Construct();
		}

		FTreeItemWeakPtr DraggedItem;
	};
	
public:
	SLATE_BEGIN_ARGS(SCompositePanelLayerTreeItemRow) { }
		SLATE_EVENT(FOnCompositeActorActivated, OnCompositeActorActivated)
		SLATE_EVENT(FOnLayerSoloToggled, OnLayerSoloToggled)
		SLATE_EVENT(FOnLayerMoved, OnLayerMoved)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, FTreeItemPtr InTreeItem)
	{
		TreeItem = InTreeItem;
		OnCompositeActorActivated = InArgs._OnCompositeActorActivated;
		OnLayerSoloToggled = InArgs._OnLayerSoloToggled;
		OnLayerMoved = InArgs._OnLayerMoved;

		STableRow<FTreeItemPtr>::FArguments Args = FSuperRowType::FArguments()
           .Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
           .OnDragDetected(this, &SCompositePanelLayerTreeItemRow::HandleDragDetected)
           .OnCanAcceptDrop(this, &SCompositePanelLayerTreeItemRow::HandleCanAcceptDrop)
           .OnAcceptDrop(this, &SCompositePanelLayerTreeItemRow::HandleAcceptDrop);
		
		SMultiColumnTableRow::Construct(Args, InOwnerTable);
	}

	/** Sets the name text block to be in edit mode for this row widget */
	void RequestRename()
	{
		if (NameTextBlock.IsValid())
		{
			NameTextBlock->EnterEditingMode();
		}
	}
	
private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		// Padding for column contents for the first column is the base value of 3.5 (based on SHeaderRow)
		// Subsequent columns need to account for the splitter size (which is default 1.0f)
		constexpr float FirstColumnHorizontalPadding = 3.5f;
		constexpr float SplitterSize = 1.0f;
		constexpr float ColumnHorizontalPadding = FirstColumnHorizontalPadding + SplitterSize;
		
		if (InColumnName == CompositePanelLayerTree::LayerTreeColumn_Active)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(FirstColumnHorizontalPadding, 0.0f)
				[
					SNew(SCheckBox)
						.Style(FAppStyle::Get(), "RadioButton")
						.ToolTipText(LOCTEXT("ItemActiveTooltip", "Activate this composite actor (only one can be active at a time)"))
						.Visibility(this, &SCompositePanelLayerTreeItemRow::IsItemActiveVisible)
						.IsChecked(this, &SCompositePanelLayerTreeItemRow::IsItemActive)
						.IsEnabled(this, &SCompositePanelLayerTreeItemRow::IsItemActiveEnabled)
						.OnCheckStateChanged(this, &SCompositePanelLayerTreeItemRow::OnIsItemActiveChanged)
				];
		}
		else if (InColumnName == CompositePanelLayerTree::LayerTreeColumn_Enabled)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(ColumnHorizontalPadding, 0.0f)
				[
					SNew(SCheckBox)
						.ToolTipText(LOCTEXT("ItemEnabledTooltip", "Enable rendering of this layer"))
						.IsChecked(this, &SCompositePanelLayerTreeItemRow::IsItemEnabled)
						.OnCheckStateChanged(this, &SCompositePanelLayerTreeItemRow::OnIsItemEnabledChanged)
				];
		}
		else if (InColumnName == CompositePanelLayerTree::LayerTreeColumn_Solo)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(ColumnHorizontalPadding, 0.0f)
				[
					SAssignNew(SoloButtonImage, SImage).Image(FAppStyle::GetBrush("MediaAsset.AssetActions.Solo.Small"))
					.ToolTipText(LOCTEXT("ItemSoloTooltip", "Solo this layer (hides all others)"))
					.ColorAndOpacity_Raw(this, &SCompositePanelLayerTreeItemRow::GetItemSoloIconColor)
					.OnMouseButtonDown(this, &SCompositePanelLayerTreeItemRow::OnItemSoloIconMouseButtonDown)
				];
		}
		else if (InColumnName == CompositePanelLayerTree::LayerTreeColumn_Name)
		{
			return SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(6.f, 0.f, 0.f, 0.f)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
				]
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 1.0f, 6.0f, 1.0f)
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					.HeightOverride(16.0f)
					[
						SNew(SImage).Image(this, &SCompositePanelLayerTreeItemRow::GetItemIcon)
					]
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(NameTextBlock, SInlineEditableTextBlock)
					.Text(this, &SCompositePanelLayerTreeItemRow::GetItemName)
					.ToolTipText(this, &SCompositePanelLayerTreeItemRow::GetItemToolTip)
					.OnTextCommitted(this, &SCompositePanelLayerTreeItemRow::OnItemNameChanged)
					.IsSelected(FIsSelected::CreateSP(this, &SCompositePanelLayerTreeItemRow::IsSelectedExclusively))
				];
		}
		else if (InColumnName == CompositePanelLayerTree::LayerTreeColumn_Type)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(ColumnHorizontalPadding, 0.0f)
				[
					SNew(STextBlock)
					.Text(this, &SCompositePanelLayerTreeItemRow::GetItemType)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
		}
		else if (InColumnName == CompositePanelLayerTree::LayerTreeColumn_Operation)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(ColumnHorizontalPadding, 0.0f)
				[
					SNew(STextBlock)
					.Text(this, &SCompositePanelLayerTreeItemRow::GetItemOperation)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
		}
		
		return SNullWidget::NullWidget;
	}

	TStrongObjectPtr<ACompositeActor> GetCompositeActor() const
	{
		return TreeItem->HasValidCompositeActor() ? TreeItem->CompositeActor.Pin() : nullptr;
	}

	/** Gets the visibility state for the row item's Active flag */
	EVisibility IsItemActiveVisible() const
	{
		// Only display active toggles on composite actor rows
		const bool bIsVisible = TreeItem->HasValidCompositeActor() && !TreeItem->HasValidCompositeLayer();

		return bIsVisible ? EVisibility::Visible : EVisibility::Hidden;
	}

	/** Gets the checkbox state for the row item's Active flag */
	ECheckBoxState IsItemActive() const
	{
		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			return CompositeActor->GetIsActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	/** Gets the checkbox enabled state for the row item's Active flag */
	bool IsItemActiveEnabled() const
	{
		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			return CompositeActor->IsInVPRole();
		}

		return true;
	}

	/** Raised when the row item's Active radiobox has changed state */
	void OnIsItemActiveChanged(ECheckBoxState InCheckBoxState)
	{
		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			const bool bIsActive = (InCheckBoxState == ECheckBoxState::Checked);
			// No transactions, the property is non-transactional
			CompositeActor->SetIsActive(bIsActive);
			OnCompositeActorActivated.ExecuteIfBound(bIsActive ? CompositeActor : nullptr);
		}
	}

	/** Gets the checkbox state for the row item's Enabled flag */
	ECheckBoxState IsItemEnabled() const
	{
		if (TreeItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
			{
				return CompositeLayer->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			
			// In the case where there is a valid layer index, but the layer itself is null
			return ECheckBoxState::Unchecked;
		}

		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			return CompositeActor->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		
		return ECheckBoxState::Unchecked;
	}

	/** Raised when the row item's Enabled checkbox has changed state */
	void OnIsItemEnabledChanged(ECheckBoxState InCheckBoxState)
	{
		if (TreeItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
			{
				FScopedTransaction SetEnabledTransaction(LOCTEXT("SetEnabledTransaction", "Set Enabled"));
				CompositeLayer->Modify();

				CompositeLayer->SetIsEnabled(InCheckBoxState == ECheckBoxState::Checked);
				return;
			}

			// In the case where there is a valid layer index, but the layer itself is null
			return;
		}

		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			FScopedTransaction SetEnabledTransaction(LOCTEXT("SetEnabledTransaction", "Set Enabled"));
			CompositeActor->Modify();

			CompositeActor->SetIsEnabled(InCheckBoxState == ECheckBoxState::Checked);
			return;
		}
	}

	/** Gets the color for the row item's Solo icon */
	FSlateColor GetItemSoloIconColor() const
	{
		bool bIsLayer = false;
		bool bIsLayerSolo = false;
		if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
		{
			bIsLayer = true;
			bIsLayerSolo = CompositeLayer->bIsSolo;
		}
		
		if (!bIsLayer)
		{
			return FLinearColor::Transparent;
		}

		if (SoloButtonImage->IsHovered())
		{
			return FStyleColors::ForegroundHover;
		}

		return (bIsLayerSolo || IsHovered()) ? FSlateColor::UseForeground() : FLinearColor::Transparent;
	}

	/** Raised when the Solo button has been pressed */
	FReply OnItemSoloIconMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
	{
		if (PointerEvent.GetEffectingButton() != EKeys::LeftMouseButton)
		{
			return FReply::Unhandled();
		}

		if (OnLayerSoloToggled.IsBound())
		{
			return OnLayerSoloToggled.Execute(TreeItem);
		}

		return FReply::Unhandled();
	}

	/** Gets the visibility of the row item's Solo button */
	EVisibility GetItemSoloVisibility() const
	{
		bool bIsLayerSolo = false;
		if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
		{
			bIsLayerSolo = CompositeLayer->bIsSolo;
		}
		
		return (IsHighlighted() || bIsLayerSolo) ? EVisibility::Visible : EVisibility::Hidden;
	}

	/** Gets the row item's icon to display next to its name */
	const FSlateBrush* GetItemIcon() const
	{
		if (TreeItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
			{
				return FSlateIconFinder::FindIconForClass(CompositeLayer->GetClass()).GetIcon();
			}
			
			// In the case where there is a valid layer index, but the layer itself is null
			return nullptr;
		}
		
		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			return FSlateIconFinder::FindIconForClass(CompositeActor->GetClass()).GetIcon(); 
		}

		return nullptr;
	}

	/** Gets the row item's display name */
	FText GetItemName() const
	{
		if (TreeItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
			{
				if (CompositeLayer->GetDisplayName().IsEmpty())
				{
					// Display the UObject name for the layer if its display name is empty.
					return FText::FromString(CompositeLayer->GetName());
				}
			
				return FText::FromString(CompositeLayer->GetDisplayName());
			}

			// In the case where there is a valid layer index, but the layer itself is null
			return LOCTEXT("LayerNotConfiguredLabel", "Layer not configured");
		}

		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			return FText::FromString(CompositeActor->GetActorNameOrLabel());
		}

		return FText::GetEmpty();
	}

	/** Raised when the row item's name has been changed */
	void OnItemNameChanged(const FText& InNewName, ETextCommit::Type InCommitType)
	{
		if (TreeItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
			{
				FScopedTransaction SetLayerNameTransaction(LOCTEXT("SetLayerNameTransaction", "Set Layer Name"));
				CompositeLayer->Modify();

				CompositeLayer->SetDisplayName(InNewName.ToString());
				return;
			}

			// In the case where there is a valid layer index, but the layer itself is null
			return;
		}

		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			FScopedTransaction RenameActorTransaction(LOCTEXT("RenameActorTransaction", "Rename Composite Actor"));
			FActorLabelUtilities::RenameExistingActor(CompositeActor.Get(), InNewName.ToString(), true);
		}
	}

	/** Gets the row item's tooltip, drawn from the layer or actor class's documentation. */
	FText GetItemToolTip() const
	{
		if (TreeItem.IsValid())
		{
			if (TreeItem->HasValidCompositeLayer())
			{
				if (const UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
				{
					return CompositeLayer->GetClass()->GetToolTipText();
				}
			}
			else if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
			{
				return CompositeActor->GetClass()->GetToolTipText();
			}
		}

		return FText::GetEmpty();
	}

	/** Gets the display text of the row item's type */
	FText GetItemType() const
	{
		if (TreeItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
			{
				return CompositeLayer->GetClass()->GetDisplayNameText();
			}

			return LOCTEXT("NoneTypeLabel", "None");
		}

		if (TStrongObjectPtr<ACompositeActor> CompositeActor = GetCompositeActor())
		{
			return CompositeActor->GetClass()->GetDisplayNameText();
		}

		return FText::GetEmpty();
	}

	/** Gets the display text of the row item's compositing operation */
	FText GetItemOperation() const
	{
		if (UCompositeLayerBase* CompositeLayer = TreeItem->GetCompositeLayer())
		{
			return StaticEnum<ECompositeCoreMergeOp>()->GetDisplayNameTextByValue((int64)CompositeLayer->Operation);
		}

		return LOCTEXT("NullOperationLabel", "-");
	}

	/** Raised when a drag start has been detected on this row item */
	FReply HandleDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		if (!TreeItem.IsValid())
		{
			return FReply::Unhandled();
		}

		if (!TreeItem->HasValidCompositeLayer())
		{
			return FReply::Unhandled();
		}
		
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TSharedPtr<FDragDropOperation> DragDropOp = FCompositeLayerDragDropOp::New(TreeItem);
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}

		return FReply::Unhandled();
	}

	/** Raised when the user is attempting to drop something onto this item */
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, FTreeItemPtr InTreeItem)
	{
		TSharedPtr<FCompositeLayerDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FCompositeLayerDragDropOp>();

		if (!DragDropOp.IsValid() || !DragDropOp->DraggedItem.IsValid())
		{
			return TOptional<EItemDropZone>();
		}

		FTreeItemPtr DraggedItem = DragDropOp->DraggedItem.Pin();
		if (DraggedItem->HasValidCompositeActor() && DraggedItem->CompositeActor.Get() == InTreeItem->CompositeActor.Get())
		{
			if (!InTreeItem->HasValidCompositeLayer())
			{
				return EItemDropZone::BelowItem;
			}

			return InItemDropZone == EItemDropZone::OntoItem ? EItemDropZone::AboveItem : InItemDropZone;
		}
		
		return TOptional<EItemDropZone>();
	}

	/** Raised when the user has dropped something onto this item */
	FReply HandleAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, FTreeItemPtr InTreeItem)
	{
		TSharedPtr<FCompositeLayerDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FCompositeLayerDragDropOp>();

		if (!DragDropOp.IsValid() || !DragDropOp->DraggedItem.IsValid())
		{
			return FReply::Unhandled();
		}

		if (OnLayerMoved.IsBound())
		{
			int32 DestIndex = InItemDropZone == EItemDropZone::AboveItem ? InTreeItem->LayerIndex: InTreeItem->LayerIndex + 1;

			// If the destination is further down than the item being moved, we must subtract one from the destination index
			// to account for the removal of the original item
			if (InTreeItem->LayerIndex > DragDropOp->DraggedItem.Pin()->LayerIndex)
			{
				--DestIndex;
			}
			
			OnLayerMoved.Execute(DragDropOp->DraggedItem.Pin(), DestIndex);
			return FReply::Handled();
		}
		
		return FReply::Unhandled();
	}
	
private:
	/** The tree item this widget is outputting */
	FTreeItemPtr TreeItem;

	/** Text block widget used to display the tree item's name */
	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;

	/** Image widget used for the 'Solo' option button */
	TSharedPtr<SImage> SoloButtonImage;

	/** Raised when a composite actor is set to the active composite actor */
	FOnCompositeActorActivated OnCompositeActorActivated;
	
	/** Callback that is raised when the layer's "solo" flag is enabled */
	FOnLayerSoloToggled OnLayerSoloToggled;

	/** Callback that is raised when a layer is moved via a drag and drop operation */
	FOnLayerMoved OnLayerMoved;
};

bool SCompositePanelLayerTree::FCompositeActorTreeItem::HasValidCompositeActor() const
{
	return CompositeActor.IsValid();
}

bool SCompositePanelLayerTree::FCompositeActorTreeItem::HasValidCompositeLayer() const
{
	return CompositeActor.IsValid() && CompositeActor->CompositeLayers.IsValidIndex(LayerIndex);
}

UCompositeLayerBase* SCompositePanelLayerTree::FCompositeActorTreeItem::GetCompositeLayer() const
{
	return HasValidCompositeLayer() ? CompositeActor->CompositeLayers[LayerIndex] : nullptr;
}

void SCompositePanelLayerTree::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	OnLayerStateChanged = InArgs._OnLayerStateChanged;

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	if (GEngine)
	{
		GEngine->OnLevelActorAdded().AddSP(this, &SCompositePanelLayerTree::OnActorAddedToLevel);
		GEngine->OnLevelActorListChanged().AddSP(this, &SCompositePanelLayerTree::OnLevelActorListChanged);
		GEngine->OnLevelActorDeleted().AddSP(this, &SCompositePanelLayerTree::OnActorRemovedFromLevel);
	}
	
	FEditorDelegates::MapChange.AddSP(this, &SCompositePanelLayerTree::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddSP(this, &SCompositePanelLayerTree::OnNewCurrentLevel);
	FWorldDelegates::LevelAddedToWorld.AddSP(this, &SCompositePanelLayerTree::OnLevelAdded);
	FWorldDelegates::LevelRemovedFromWorld.AddSP(this, &SCompositePanelLayerTree::OnLevelRemoved);
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SCompositePanelLayerTree::OnObjectReplaced);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SCompositePanelLayerTree::OnObjectPropertyChanged);
	
	CommandList = MakeShared<FUICommandList>();
	BindCommands();
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 8.0f, 8.0f, 4.0f)
		[
			SAssignNew(Toolbar, SCompositePanelLayerTreeToolbar)
			.CanAddLayer(this, &SCompositePanelLayerTree::CanAddLayer)
			.CanPilotCamera(this, &SCompositePanelLayerTree::CanPilotCamera)
			.IsPilotingCamera(this, &SCompositePanelLayerTree::IsPilotingCamera)
			.OnPilotCameraToggled(this, &SCompositePanelLayerTree::OnPilotCameraToggled)
			.MeshVisualizationMode(this, &SCompositePanelLayerTree::GetMeshVisualizationMode)
			.OnMeshVisualizationModeChanged(this, &SCompositePanelLayerTree::SetMeshVisualizationMode)
			.OnFilterChanged(this, &SCompositePanelLayerTree::OnFilterChanged)
			.OnCompositeActorAdded(this, &SCompositePanelLayerTree::AddCompositeActor)
			.OnLayerAdded(this, &SCompositePanelLayerTree::OnLayerAdded)
		]

		+SVerticalBox::Slot()
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this]{ return CompositeActorTreeItems.IsEmpty() ? 0 : 1; })

			+SWidgetSwitcher::Slot()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.0, 12.0, 0.0, 8.0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddCompositeActorLabel", "Add a Composite Actor"))
					.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.0, 0.0, 12.0, 0.0)
				[
					SNew(SButton)
					.OnClicked_Lambda([this](){ AddCompositeActor(); return FReply::Handled(); })
					[
						SNew(SHorizontalBox)
						
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0, 1))
						[
							SNew(SImage)
							.Image(FSlateIconFinder::FindIconBrushForClass(ACompositeActor::StaticClass()))
						]
						
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(2, 0, 0, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PlaceCompositeActorButtonLabel", "Place Composite Actor"))
						]
					]
				]
			]

			+SWidgetSwitcher::Slot()
			[
				SAssignNew(TreeView, STreeView<FCompositeActorTreeItemPtr>)
				.TreeItemsSource(&FilteredCompositeActorTreeItems)
				.HeaderRow(
					SNew(SHeaderRow)

					+SHeaderRow::Column(CompositePanelLayerTree::LayerTreeColumn_Active)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(24.0f)
					.HAlignHeader(HAlign_Center)
					.VAlignHeader(VAlign_Center)
					.HAlignCell(HAlign_Center)
					.VAlignCell(VAlign_Center)
					[
						SNew(SCheckBox)
							.Style(FAppStyle::Get(), "RadioButton")
							.ToolTipText(LOCTEXT("HeaderActiveTooltip", "Toggle the active composite actor"))
							.IsChecked(this, &SCompositePanelLayerTree::GetGlobalActiveCheckState)
							.OnCheckStateChanged(this, &SCompositePanelLayerTree::OnGlobalActiveCheckStateChanged)
					]
					
					+SHeaderRow::Column(CompositePanelLayerTree::LayerTreeColumn_Enabled)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(24.0f)
					.HAlignHeader(HAlign_Center)
					.VAlignHeader(VAlign_Center)
					.HAlignCell(HAlign_Center)
					.VAlignCell(VAlign_Center)
					[
						SNew(SCheckBox)
						.ToolTipText(LOCTEXT("HeaderEnabledTooltip", "Toggle rendering of all layers"))
						.IsChecked(this, &SCompositePanelLayerTree::GetGlobalEnabledCheckState)
						.OnCheckStateChanged(this, &SCompositePanelLayerTree::OnGlobalEnabledCheckStateChanged)
					]
					
					+SHeaderRow::Column(CompositePanelLayerTree::LayerTreeColumn_Solo)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(24.0f)
					.HAlignHeader(HAlign_Center)
					.VAlignHeader(VAlign_Center)
					.HAlignCell(HAlign_Center)
					.VAlignCell(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("MediaAsset.AssetActions.Solo.Small"))
						.ToolTipText(LOCTEXT("HeaderSoloTooltip", "Soloed layer"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					
					+SHeaderRow::Column(CompositePanelLayerTree::LayerTreeColumn_Name)
					.DefaultLabel(LOCTEXT("LayerTreeColumnLabel", "Layer"))
					.FillWidth(0.55)

					+SHeaderRow::Column(CompositePanelLayerTree::LayerTreeColumn_Type)
					.DefaultLabel(LOCTEXT("TypeTreeColumnLabel", "Type"))
					.FillWidth(0.3)

					+SHeaderRow::Column(CompositePanelLayerTree::LayerTreeColumn_Operation)
					.DefaultLabel(LOCTEXT("OperationTreeColumnLabel", "Operation"))
					.FillWidth(0.15)
				)
				.OnGenerateRow_Lambda([this](FCompositeActorTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& InOwnerTable)
				{
					return SNew(SCompositePanelLayerTreeItemRow, InOwnerTable, InTreeItem)
						.OnCompositeActorActivated(this, &SCompositePanelLayerTree::OnCompositeActorActivated)
						.OnLayerSoloToggled(this, &SCompositePanelLayerTree::OnLayerSoloToggled)
						.OnLayerMoved(this, &SCompositePanelLayerTree::OnLayerMoved);
				})
				.OnGetChildren_Lambda([](FCompositeActorTreeItemPtr InTreeItem, TArray<FCompositeActorTreeItemPtr>& OutChildren)
				{
					if (InTreeItem.IsValid())
					{
						for (FCompositeActorTreeItemPtr& Child : InTreeItem->Children)
						{
							if (Child.IsValid() && !Child->bFilteredOut)
							{
								OutChildren.Add(Child);
							}
						}
					}
				})
				.OnSelectionChanged(this, &SCompositePanelLayerTree::OnLayerSelectionChanged)
				.OnContextMenuOpening(this, &SCompositePanelLayerTree::CreateTreeContextMenu)
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(FooterContainer, SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(14, 9))
			[
				SNew( STextBlock )
				.Text(this, &SCompositePanelLayerTree::GetFilterStatusText)
				.ColorAndOpacity(this, &SCompositePanelLayerTree::GetFilterStatusTextColor)
			]
		]
	];

	FillCompositeActorTree();
}

SCompositePanelLayerTree::~SCompositePanelLayerTree()
{
	// Reset the visualization scalar in the world we last wrote to (each editor world owns its own MPC instance).
	if (UMaterialParameterCollection* MPC = CachedVisualizationMPC.Get())
	{
		if (UWorld* LastWorld = CachedVisualizationWorld.Get())
		{
			UKismetMaterialLibrary::SetScalarParameterValue(LastWorld, MPC, CompositePanelLayerTree::MeshVisualizationParameterName, 0.0f);
		}
	}

	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorListChanged().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SCompositePanelLayerTree::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Update the IsPilotingCamera flag by checking that the current piloted actor is the active composite actor's camera
	bCachedIsPilotingCamera = false;

	AActor* CompositeCameraActor = GetActiveCompositeActorCamera();
	if (IsValid(CompositeCameraActor))
	{
		if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
		{
			bCachedIsPilotingCamera = CompositeCameraActor == LevelEditorSubsystem->GetPilotLevelActor();
		}
	}

	UpdateMeshVisualizationMPC();
}

ECompositeMeshVisualizationMode SCompositePanelLayerTree::GetMeshVisualizationMode() const
{
	return GetDefault<UCompositeEditorPanelSettings>()->MeshVisualizationMode;
}

void SCompositePanelLayerTree::SetMeshVisualizationMode(ECompositeMeshVisualizationMode InMode)
{
	UCompositeEditorPanelSettings* Settings = GetMutableDefault<UCompositeEditorPanelSettings>();
	if (Settings->MeshVisualizationMode == InMode)
	{
		return;
	}

	Settings->MeshVisualizationMode = InMode;
	Settings->SaveConfig();

	UpdateMeshVisualizationMPC();
}

float SCompositePanelLayerTree::ResolveMeshVisualizationValue(ECompositeMeshVisualizationMode InMode, bool bIsPilotingCompositeCamera)
{
	switch (InMode)
	{
	case ECompositeMeshVisualizationMode::Enabled:
		return 1.0f;
	case ECompositeMeshVisualizationMode::EnabledWhenNotPiloting:
		return bIsPilotingCompositeCamera ? 0.0f : 1.0f;
	case ECompositeMeshVisualizationMode::Disabled:
	default:
		return 0.0f;
	}
}

void SCompositePanelLayerTree::UpdateMeshVisualizationMPC()
{
	if (!GEditor)
	{
		return;
	}

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return;
	}

	// Each editor world owns its own MPC instance starting at the asset's defaults, so invalidate the cache on world swap.
	if (CachedVisualizationWorld.Get() != World)
	{
		LastWrittenMeshVisualization.Reset();
		CachedVisualizationWorld = World;
	}

	const ECompositeMeshVisualizationMode Mode = GetMeshVisualizationMode();
	const float NewValue = ResolveMeshVisualizationValue(Mode, bCachedIsPilotingCamera);

	if (LastWrittenMeshVisualization.IsSet() && LastWrittenMeshVisualization.GetValue() == NewValue)
	{
		return;
	}

	if (!CachedVisualizationMPC.IsValid())
	{
		CachedVisualizationMPC = Cast<UMaterialParameterCollection>(CompositePanelLayerTree::MPCCompositePath.TryLoad());
	}

	UMaterialParameterCollection* MPC = CachedVisualizationMPC.Get();
	if (!MPC)
	{
		return;
	}

	UKismetMaterialLibrary::SetScalarParameterValue(World, MPC, CompositePanelLayerTree::MeshVisualizationParameterName, NewValue);

	LastWrittenMeshVisualization = NewValue;
}

FReply SCompositePanelLayerTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (this->CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SCompositePanelLayerTree::PostUndo(bool bSuccess)
{
	constexpr bool bPreserveSelection = true;
	FillCompositeActorTree(bPreserveSelection);

	VerifyActiveCompositeActorPiloting();
}

void SCompositePanelLayerTree::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SCompositePanelLayerTree::SelectCompositeActors(const TArray<TWeakObjectPtr<ACompositeActor>>& InCompositeActors)
{
	if (!TreeView.IsValid())
	{
		return;
	}

	TreeView->ClearSelection();
	
	for (const TWeakObjectPtr<ACompositeActor>& Actor : InCompositeActors)
	{
		FCompositeActorTreeItemPtr* TreeItem = CompositeActorTreeItems.FindByPredicate([&Actor](const FCompositeActorTreeItemPtr& InTreeItem)
		{
			return InTreeItem.IsValid() && InTreeItem->CompositeActor == Actor.Get();
		});

		if (TreeItem)
		{
			TreeView->SetItemSelection(*TreeItem, true);
		}
	}
}

void SCompositePanelLayerTree::SelectObjects(const TArray<UObject*>& InObjects)
{
	if (!TreeView.IsValid())
	{
		return;
	}

	TreeView->ClearSelection();

	bool bSelectionChanged = false;
	
	// Silence the normal invoke of selection changes in the tree. A single, bulk selection change will be invoked after all objects are selected
	TGuardValue<bool> SilenceSelectionChangesGuard(bSilenceSelectionChanges, true);
	for (const UObject* Object : InObjects)
	{
		if (const ACompositeActor* CompositeActor = Cast<ACompositeActor>(Object))
		{
			FCompositeActorTreeItemPtr* TreeItem = CompositeActorTreeItems.FindByPredicate([&CompositeActor](const FCompositeActorTreeItemPtr& InTreeItem)
			{
				return InTreeItem.IsValid() && InTreeItem->CompositeActor == CompositeActor;
			});

			if (TreeItem)
			{
				TreeView->SetItemSelection(*TreeItem, true);
				bSelectionChanged = true;
			}
		}
		else if (const UCompositeLayerBase* CompositeLayer = Cast<UCompositeLayerBase>(Object))
		{
			FCompositeActorTreeItemPtr* TreeItem = nullptr;
			for (const FCompositeActorTreeItemPtr& ActorTreeItem : CompositeActorTreeItems)
			{
				if (!ActorTreeItem.IsValid())
				{
					continue;
				}

				TreeItem = ActorTreeItem->Children.FindByPredicate([&CompositeLayer](const FCompositeActorTreeItemPtr& InTreeItem)
				{
					return InTreeItem.IsValid() && InTreeItem->HasValidCompositeLayer() && InTreeItem->GetCompositeLayer() == CompositeLayer;
				});

				if (TreeItem)
				{
					break;
				}
			}

			if (TreeItem)
			{
				TreeView->SetItemSelection(*TreeItem, true);
				bSelectionChanged = true;
			}
		}
	}

	if (bSelectionChanged)
	{
		InvokeLayerSelectionChanged();
	}
}

TArray<UObject*> SCompositePanelLayerTree::GetSelectedObjects()
{
	TArray<UObject*> SelectedObjects;

	if (TreeView.IsValid())
	{
		TArray<FCompositeActorTreeItemPtr> SelectedTreeItems = TreeView->GetSelectedItems();
		for (const FCompositeActorTreeItemPtr& TreeItem : SelectedTreeItems)
		{
			if (!TreeItem.IsValid())
			{
				continue;
			}

			if (TreeItem->HasValidCompositeLayer())
			{
				SelectedObjects.Add(TreeItem->GetCompositeLayer());
			}
			else if (TreeItem->CompositeActor.IsValid())
			{
				SelectedObjects.Add(TreeItem->CompositeActor.Get());
			}
		}
	}
	
	return SelectedObjects;
}

float SCompositePanelLayerTree::GetMinimumHeight() const
{
	// Calculate the number of rows the tree view should be displaying
	int32 NumRows = 0;
	for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
	{
		NumRows += TreeItem->Children.Num() + 1;
	}
	
	constexpr float HeaderHeight = 24.0f;
	constexpr float RowHeight = 18.0f;

	const float ToolbarHeight = Toolbar->GetCachedGeometry().GetAbsoluteSize().Y + 12.0f;
	const float FooterHeight = FooterContainer->GetCachedGeometry().GetAbsoluteSize().Y;
	const float TreeHeight = NumRows > 0 ? RowHeight * NumRows : 48.0f;

	// Add the toolbar height (plus padding), the tree view header, the tree view rows, and the footer height
	return ToolbarHeight + HeaderHeight + TreeHeight + FooterHeight + 2.0f;
}

void SCompositePanelLayerTree::BindCommands()
{
	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CopySelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanCopySelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CutSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanCutSelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::PasteSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanPasteSelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Duplicate,
	FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::DuplicateSelectedItems),
	FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanDuplicateSelectedItems));
	
	CommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::DeleteSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanDeleteSelectedItems));
		
	CommandList->MapAction(FGenericCommands::Get().Rename,
		FUIAction(FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::RenameSelectedItem),
		FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanRenameSelectedItem)));

	CommandList->MapAction(FCompositeEditorCommands::Get().Enable,
		FExecuteAction::CreateSP(this, &SCompositePanelLayerTree::EnableSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePanelLayerTree::CanEnableSelectedItems));
}

void SCompositePanelLayerTree::FillCompositeActorTree(bool bPreserveSelection)
{
	TArray<TPair<ACompositeActor*, int32>> ItemsToReselect;
	if (bPreserveSelection)
	{
		TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
		for (const FCompositeActorTreeItemPtr& Item : SelectedItems)
		{
			if (Item->CompositeActor.IsValid())
			{
				ItemsToReselect.Add(TPair<ACompositeActor*, int32>(Item->CompositeActor.Get(), Item->LayerIndex));
			}
		}
	}
	
	CompositeActorTreeItems.Empty();
	FilteredCompositeActorTreeItems.Empty();
	
	if (!GEditor)
	{
		return;
	}

	if (const UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		for (TActorIterator<ACompositeActor> It(World); It; ++It)
		{
			ACompositeActor* CompositeActor = *It;
			if (!IsValid(CompositeActor))
			{
				continue;
			}
			
			FCompositeActorTreeItemPtr NewActorTreeItem = MakeShared<FCompositeActorTreeItem>();
			NewActorTreeItem->CompositeActor = CompositeActor;
			NewActorTreeItem->bFilteredOut = Toolbar.IsValid() ? !Toolbar->ItemPassesFilters(NewActorTreeItem) : false;
			
			FillCompositeActorTreeItem(NewActorTreeItem);

			const bool bAnyChildPassesFilter = NewActorTreeItem->Children.ContainsByPredicate([](const FCompositeActorTreeItemPtr& InChildItem)
			{
				return !InChildItem->bFilteredOut;
			});
			
			CompositeActorTreeItems.Add(NewActorTreeItem);
			if (!NewActorTreeItem->bFilteredOut || bAnyChildPassesFilter)
			{
				FilteredCompositeActorTreeItems.Add(NewActorTreeItem);
			}
		}
	}

	RefreshAndExpandTreeView();

	if (TreeView.IsValid())
	{
		// Silence any selection changes while re-selecting, and track if the selection has changed (i.e. the originally selected items no longer exist)
		// If the selection has changed at the end of re-selection, the selection changed delegate will be invoked manually
		bool bSelectionChanged = false;
		{
			TGuardValue<bool> SilenceSelectionChangesGuard(bSilenceSelectionChanges, true);
		
			for (const TPair<ACompositeActor*, int32>& Pair : ItemsToReselect)
			{
				ACompositeActor* CompositeActorToReselect = Pair.Key;
				int32 LayerToReselect = Pair.Value;

				const int32 CompositeActorItemIndex = FilteredCompositeActorTreeItems.IndexOfByPredicate([CompositeActorToReselect](const FCompositeActorTreeItemPtr& InItem)
				{
					return InItem->CompositeActor == CompositeActorToReselect;
				});

				if (CompositeActorItemIndex == INDEX_NONE)
				{
					// Composite actor that was previously selected is no longer in the list, likely from deletion, so
					// we must invoke SelectionChanged to notify any external widgets (e.g. details panel) to clear and refresh
					bSelectionChanged = true;
					continue;
				}

				FCompositeActorTreeItemPtr CompositeActorItem = FilteredCompositeActorTreeItems[CompositeActorItemIndex];
				if (LayerToReselect == CompositeActorItem->LayerIndex)
				{
					TreeView->SetItemSelection(CompositeActorItem, true);
				}
				else if (CompositeActorItem->Children.IsValidIndex(LayerToReselect) && !CompositeActorItem->Children[LayerToReselect]->bFilteredOut)
				{
					TreeView->SetItemSelection(CompositeActorItem->Children[LayerToReselect], true);
				}
				else
				{
					// Index of composite layer that was previously selected is no longer valid, indicating the layer was deleted, so
					// we must invoke SelectionChanged to notify any external widgets (e.g. details panel) to clear and refresh
					bSelectionChanged = true;
				}
			}
		}

		if (bSelectionChanged)
		{
			OnLayerSelectionChanged(nullptr, ESelectInfo::Direct);
		}
	}
}

void SCompositePanelLayerTree::FillCompositeActorTreeItem(FCompositeActorTreeItemPtr& InOutTreeItem)
{
	if (!InOutTreeItem.IsValid())
	{
		return;
	}
	
	InOutTreeItem->Children.Empty();

	if (TStrongObjectPtr<ACompositeActor> CompositeActor = InOutTreeItem->CompositeActor.Pin())
	{
		for (int32 Index = 0; Index < CompositeActor->CompositeLayers.Num(); ++Index)
		{
			FCompositeActorTreeItemPtr NewLayerTreeItem = MakeShared<FCompositeActorTreeItem>();
			NewLayerTreeItem->CompositeActor = CompositeActor.Get();
			NewLayerTreeItem->LayerIndex = Index;
			NewLayerTreeItem->bFilteredOut = Toolbar.IsValid() ? !Toolbar->ItemPassesFilters(NewLayerTreeItem) : false;
			
			InOutTreeItem->Children.Add(NewLayerTreeItem);
		}
	}
}

void SCompositePanelLayerTree::RefreshCompositeActorLayers(ACompositeActor* InCompositeActor, bool bRefreshTree, bool bPreserveSelection)
{
	TArray<int32> LayersToSelect;
	if (bPreserveSelection)
	{
		TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
		for (const FCompositeActorTreeItemPtr& SelectedItem : SelectedItems)
		{
			if (SelectedItem->CompositeActor.Get() == InCompositeActor)
			{
				LayersToSelect.Add(SelectedItem->LayerIndex);
			}
		}
	}
	
	int32 TreeItemIndex = CompositeActorTreeItems.IndexOfByPredicate([InCompositeActor](const FCompositeActorTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->CompositeActor == InCompositeActor;
	});

	if (TreeItemIndex != INDEX_NONE)
	{
		FillCompositeActorTreeItem(CompositeActorTreeItems[TreeItemIndex]);

		if (bRefreshTree)
		{
			RefreshAndExpandTreeView();
		}

		if (!LayersToSelect.IsEmpty())
		{
			for (int32 LayerIndex : LayersToSelect)
			{
				const bool bIsValidIndex = CompositeActorTreeItems[TreeItemIndex]->Children.IsValidIndex(LayerIndex);
				FCompositeActorTreeItemPtr LayerToSelect = bIsValidIndex ? CompositeActorTreeItems[TreeItemIndex]->Children[LayerIndex] : nullptr;
				if (LayerToSelect.IsValid() && TreeView.IsValid())
				{
					TreeView->SetItemSelection(LayerToSelect, true);
				}
			}
		}
	}
}

void SCompositePanelLayerTree::SelectCompositeActorLayers(ACompositeActor* InCompositeActor, const TArray<int32>& InLayersToSelect)
{
	if (!TreeView.IsValid())
	{
		return;
	}
	
	int32 TreeItemIndex = CompositeActorTreeItems.IndexOfByPredicate([InCompositeActor](const FCompositeActorTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->CompositeActor == InCompositeActor;
	});

	if (TreeItemIndex != INDEX_NONE)
	{
		FCompositeActorTreeItemPtr& CompositeActorTreeItem = CompositeActorTreeItems[TreeItemIndex];
		for (const int32 LayerIndex : InLayersToSelect)
		{
			if (CompositeActorTreeItem->Children.IsValidIndex(LayerIndex))
			{
				TreeView->SetItemSelection(CompositeActorTreeItem->Children[LayerIndex], true);	
			}
		}
	}
}

void SCompositePanelLayerTree::RefreshAndExpandTreeView()
{
	if (TreeView.IsValid())
	{
		TreeView->RebuildList();

		for (int32 Index = 0; Index < FilteredCompositeActorTreeItems.Num(); ++Index)
		{
			TreeView->SetItemExpansion(FilteredCompositeActorTreeItems[Index], true);
		}
	}
}

bool SCompositePanelLayerTree::CanAddLayer() const
{
	if (CompositeActorTreeItems.Num() == 1)
	{
		return CompositeActorTreeItems[0]->CompositeActor.IsValid();
	}

	return TreeView.IsValid() && !TreeView->GetSelectedItems().IsEmpty();
}

bool SCompositePanelLayerTree::CanPilotCamera() const
{
	// Always allow the user to eject regardless of selection if a composite actor's camera is being piloted
	if (bCachedIsPilotingCamera)
	{
		return true;
	}

	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (!SelectedItems.IsEmpty())
	{
		// If the active composite actor is not included in the currently selected items, disable the Pilot button
		const bool bSelectionContainsActiveCompositeActor = SelectedItems.ContainsByPredicate([](const FCompositeActorTreeItemPtr& InTreeItem)
		{
			return InTreeItem.IsValid() && InTreeItem->HasValidCompositeActor() && InTreeItem->CompositeActor->GetIsActive();
		});

		if (!bSelectionContainsActiveCompositeActor)
		{
			return false;
		}
	}
	
	AActor* CompositeCameraActor = GetActiveCompositeActorCamera();
	return IsValid(CompositeCameraActor);
}

bool SCompositePanelLayerTree::IsPilotingCamera() const
{
	return bCachedIsPilotingCamera;
}

void SCompositePanelLayerTree::OnPilotCameraToggled(bool bPilotCamera)
{
	if (bPilotCamera)
	{
		TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
		if (!SelectedItems.IsEmpty())
		{
			// If the active composite actor is not included in the currently selected items, don't try and pilot anything
			const bool bSelectionContainsActiveCompositeActor = SelectedItems.ContainsByPredicate([](const FCompositeActorTreeItemPtr& InTreeItem)
			{
				return InTreeItem.IsValid() && InTreeItem->HasValidCompositeActor() && InTreeItem->CompositeActor->GetIsActive();
			});

			if (!bSelectionContainsActiveCompositeActor)
			{
				return;
			}
		}

		AActor* CompositeCameraActor = GetActiveCompositeActorCamera();

		if (!IsValid(CompositeCameraActor))
		{
			return;
		}
		
		PilotCamera(CompositeCameraActor);
	}
	else
	{
		StopPilotingCamera();
	}
}

AActor* SCompositePanelLayerTree::GetActiveCompositeActorCamera() const
{
	TStrongObjectPtr<ACompositeActor> ActiveCompositeActor = nullptr;
	const FCompositeActorTreeItemPtr* ActiveTreeItem = CompositeActorTreeItems.FindByPredicate([](const FCompositeActorTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->HasValidCompositeActor() && InTreeItem->CompositeActor->GetIsActive();
	});
	
	if (ActiveTreeItem)
	{
		ActiveCompositeActor = (*ActiveTreeItem)->CompositeActor.Pin();
	}

	if (ActiveCompositeActor.IsValid())
	{
		return ActiveCompositeActor->GetCameraActor().Get();
	}

	return nullptr;
}

void SCompositePanelLayerTree::PilotCamera(AActor* InCameraActorToPilot)
{
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<SLevelViewport> ActiveLevelViewport = nullptr;
		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		if (LevelEditor.IsValid())
		{
			ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
		}

		// The subsystem currently has no method for capturing the view transform, so we extract it ourselves for now.
		if (ActiveLevelViewport.IsValid())
		{
			FLevelEditorViewportClient& Client = ActiveLevelViewport->GetLevelViewportClient();
			CachedPerspectiveCameraTransform = Client.GetViewTransform();
			CachedViewFOV = Client.ViewFOV;
		}

		LevelEditorSubsystem->PilotLevelActor(InCameraActorToPilot);
		LevelEditorSubsystem->SetExactCameraView(true);
	}
}

void SCompositePanelLayerTree::StopPilotingCamera()
{
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		// Early abort if the level editor is not actively piloting anything
		if (!LevelEditorSubsystem->GetPilotLevelActor())
		{
			return;
		}
		
		LevelEditorSubsystem->EjectPilotLevelActor();

		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<SLevelViewport> ActiveLevelViewport = nullptr;
		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		if (LevelEditor.IsValid())
		{
			ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
		}
		
		if (ActiveLevelViewport.IsValid())
		{
			// Restore the previously cached view transform
			FLevelEditorViewportClient& Client = ActiveLevelViewport->GetLevelViewportClient();
			if (CachedPerspectiveCameraTransform.IsSet())
			{
				Client.SetViewLocation(CachedPerspectiveCameraTransform.GetValue().GetLocation());
				Client.SetViewRotation(CachedPerspectiveCameraTransform.GetValue().GetRotation());
			}

			if(CachedViewFOV.IsSet())
			{
				Client.ViewFOV = CachedViewFOV.GetValue();
			}
		}
	}
}

void SCompositePanelLayerTree::VerifyActiveCompositeActorPiloting()
{
	if (bCachedIsPilotingCamera)
	{
		// Check that the active composite actor's camera is still the one being piloted, and if it isn't, stop piloting
		bool bIsStillPilotingCamera = false;
		AActor* CompositeCameraActor = GetActiveCompositeActorCamera();
		if (IsValid(CompositeCameraActor))
		{
			if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
			{
				bIsStillPilotingCamera = CompositeCameraActor == LevelEditorSubsystem->GetPilotLevelActor();
			}
		}

		if (!bIsStillPilotingCamera)
		{
			StopPilotingCamera();
		}
	}
}

ECheckBoxState SCompositePanelLayerTree::GetGlobalActiveCheckState() const
{
	bool bAnyEnabled = false;
	
	for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
	{
		if (!TreeItem.IsValid() || !TreeItem->HasValidCompositeActor() || TreeItem->HasValidCompositeLayer())
		{
			continue;
		}

		if (TreeItem->CompositeActor->GetIsActive())
		{
			bAnyEnabled = true;
		}
	}

	return bAnyEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCompositePanelLayerTree::OnGlobalActiveCheckStateChanged(ECheckBoxState CheckBoxState)
{
	if (CheckBoxState == ECheckBoxState::Checked)
	{
		bool bActivated = false;
		for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
		{
			if (!TreeItem.IsValid() || !TreeItem->HasValidCompositeActor() || TreeItem->HasValidCompositeLayer())
			{
				continue;
			}

			TStrongObjectPtr<ACompositeActor> CompositeActor = TreeItem->CompositeActor.Pin();
			if (CompositeActor.Get() == LastActiveActor.Get())
			{
				if (CompositeActor->IsInVPRole())
				{
					CompositeActor->SetIsActive(true);
					OnCompositeActorActivated(CompositeActor);
					bActivated = true;
				}
			}
		}

		// If no match was found, we simply activate the first actor
		if (!bActivated)
		{
			for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
			{
				if (!TreeItem.IsValid() || !TreeItem->HasValidCompositeActor() || TreeItem->HasValidCompositeLayer())
				{
					continue;
				}

				TStrongObjectPtr<ACompositeActor> CompositeActor = TreeItem->CompositeActor.Pin();
				if (CompositeActor->IsInVPRole())
				{
					CompositeActor->SetIsActive(true);
					OnCompositeActorActivated(CompositeActor);
					break;
				}
			}
		}
	}
	else
	{
		for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
		{
			if (!TreeItem.IsValid() || !TreeItem->HasValidCompositeActor() || TreeItem->HasValidCompositeLayer())
			{
				continue;
			}

			if (TreeItem->CompositeActor->GetIsActive())
			{
				LastActiveActor = TreeItem->CompositeActor;
			}

			TreeItem->CompositeActor->SetIsActive(false);
			OnCompositeActorActivated(nullptr);
		}
	}
}

ECheckBoxState SCompositePanelLayerTree::GetGlobalEnabledCheckState() const
{
	bool bAnyEnabled = false;
	bool bAnyDisabled = false;
	for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
	{
		if (!TreeItem.IsValid() || !TreeItem->HasValidCompositeActor())
		{
			continue;
		}

		if (TreeItem->CompositeActor->GetIsEnabled())
		{
			bAnyEnabled = true;
		}
		else
		{
			bAnyDisabled = true;
		}

		for (const FCompositeActorTreeItemPtr& ChildTreeItem : TreeItem->Children)
		{
			if (!ChildTreeItem.IsValid() ||
				!ChildTreeItem->HasValidCompositeLayer() ||
				ChildTreeItem->bFilteredOut)
			{
				continue;
			}

			if (UCompositeLayerBase* CompositeLayer = ChildTreeItem->GetCompositeLayer())
			{
				if (CompositeLayer->GetIsEnabled())
				{
					bAnyEnabled = true;
				}
				else
				{
					bAnyDisabled = true;
				}
			}
		}
	}

	if (bAnyEnabled)
	{
		return bAnyDisabled ? ECheckBoxState::Undetermined : ECheckBoxState::Checked;
	}
	else
	{
		return bAnyDisabled ? ECheckBoxState::Unchecked : ECheckBoxState::Undetermined;
	}
}

void SCompositePanelLayerTree::OnGlobalEnabledCheckStateChanged(ECheckBoxState CheckBoxState)
{
	TSharedPtr<FScopedTransaction> GlobalSetEnabledTransaction = nullptr;
	
	for (const FCompositeActorTreeItemPtr& TreeItem : FilteredCompositeActorTreeItems)
	{
		if (!TreeItem.IsValid() || !TreeItem->HasValidCompositeActor())
		{
			continue;
		}

		if (!GlobalSetEnabledTransaction.IsValid())
		{
			GlobalSetEnabledTransaction = MakeShared<FScopedTransaction>(LOCTEXT("GlobalSetEnabledTransaction", "Set Enabled"));
		}

		TreeItem->CompositeActor->Modify();
		TreeItem->CompositeActor->SetIsEnabled(CheckBoxState == ECheckBoxState::Checked);
		
		for (const FCompositeActorTreeItemPtr& ChildTreeItem : TreeItem->Children)
		{
			if (!ChildTreeItem.IsValid() ||
				!ChildTreeItem->HasValidCompositeLayer() ||
				ChildTreeItem->bFilteredOut)
			{
				continue;
			}

			if (UCompositeLayerBase* CompositeLayer = ChildTreeItem->GetCompositeLayer())
			{
				CompositeLayer->Modify();
				CompositeLayer->SetIsEnabled(CheckBoxState == ECheckBoxState::Checked);
			}
		}
	}
}

void SCompositePanelLayerTree::OnLayerSelectionChanged(FCompositeActorTreeItemPtr InTreeItem, ESelectInfo::Type SelectInfo)
{
	if (!bSilenceSelectionChanges)
	{
		InvokeLayerSelectionChanged();
	}
}

void SCompositePanelLayerTree::InvokeLayerSelectionChanged()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	
	TArray<UObject*> SelectedLayers;
	SelectedLayers.Reserve(SelectedItems.Num());

	for (const FCompositeActorTreeItemPtr& Item : SelectedItems)
	{
		if (!Item->HasValidCompositeActor())
		{
			continue;
		}

		if (Item->LayerIndex == INDEX_NONE)
		{
			SelectedLayers.Add(Item->CompositeActor.Get());
		}
		else if (Item->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = Item->GetCompositeLayer())
			{
				SelectedLayers.Add(CompositeLayer);
			}
		}
	}
	
	OnSelectionChanged.ExecuteIfBound(SelectedLayers);
}

void SCompositePanelLayerTree::OnFilterChanged()
{
	FilteredCompositeActorTreeItems.Empty();
	
	for (FCompositeActorTreeItemPtr& TreeItem : CompositeActorTreeItems)
	{
		TreeItem->bFilteredOut = Toolbar.IsValid() ? !Toolbar->ItemPassesFilters(TreeItem) : false;

		bool bChildLayerPassesFilter = false;
		for (FCompositeActorTreeItemPtr& ChildTreeItem : TreeItem->Children)
		{
			ChildTreeItem->bFilteredOut = Toolbar.IsValid() ? !Toolbar->ItemPassesFilters(ChildTreeItem) : false;
			if (!ChildTreeItem->bFilteredOut)
			{
				bChildLayerPassesFilter = true;
			}
		}

		if (!TreeItem->bFilteredOut || bChildLayerPassesFilter)
		{
			FilteredCompositeActorTreeItems.Add(TreeItem);
		}
	}
	
	RefreshAndExpandTreeView();
}

FText SCompositePanelLayerTree::GetFilterStatusText() const
{
	const int32 NumActors = CompositeActorTreeItems.Num();
	const int32 NumFiltered = FilteredCompositeActorTreeItems.Num();

	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	const int32 NumSelected = SelectedItems.Num();
	
	const FText ActorLabel = NumActors > 1 ? LOCTEXT("ActorPlural", "Composite Actors") : LOCTEXT("ActorSingular", "Composite Actor");
	if (NumActors > 0 && NumActors == NumFiltered)
	{
		if (NumSelected > 0)
		{
			return FText::Format(LOCTEXT("NumActorsAndSelectedTextFormat", "{0} {1}, {2} Selected"), FText::AsNumber(NumActors), ActorLabel, FText::AsNumber(NumSelected));
		}
		else
		{
			return FText::Format(LOCTEXT("NumActorsTextFormat", "{0} {1}"), FText::AsNumber(NumActors), ActorLabel);
		}
	}
	else if (NumFiltered > 0)
	{
		if (NumSelected > 0)
		{
			return FText::Format(LOCTEXT("NumActorsWithFilteredAndSelectedTextFormat", "Showing {0} of {1} {2}, {3} Selected"),
				FText::AsNumber(NumFiltered),
				FText::AsNumber(NumActors),
				ActorLabel,
				FText::AsNumber(NumSelected));
		}
		else
		{
			return FText::Format(LOCTEXT("NumActorsWithFilteredTextFormat", "Showing {0} of {1} {2}"),
				FText::AsNumber(NumFiltered),
				FText::AsNumber(NumActors),
				ActorLabel);
		}
	}
	else
	{
		if (NumActors > 0)
		{
			return FText::Format(LOCTEXT("NoMatchingActorsTextFormat", "No matching actors or layers ({0} {1})"), FText::AsNumber(NumActors), ActorLabel);
		}
		else
		{
			return LOCTEXT("NoActorsFoundLabel", "0 Composite Actors");
		}
	}
}

FSlateColor SCompositePanelLayerTree::GetFilterStatusTextColor() const
{
	const int32 NumActors = CompositeActorTreeItems.Num();
	const int32 NumFiltered = FilteredCompositeActorTreeItems.Num();
	
	if (NumActors > 0 && NumFiltered == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	
	return FSlateColor::UseForeground();
}

TSharedPtr<SWidget> SCompositePanelLayerTree::CreateTreeContextMenu()
{
	if (!TreeView.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	const int32 NumItems = TreeView->GetNumItemsSelected();
	if (NumItems >= 1)
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);

		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry(FCompositeEditorCommands::Get().Enable);
		
		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void SCompositePanelLayerTree::AddCompositeActor()
{
	UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
	if (!World)
	{
		return;
	}

	FScopedTransaction AddCompositeActorTransaction(LOCTEXT("AddCompositeActorTransaction", "Add Composite Actor"));
	
	ACompositeActor* AddedActor = nullptr;

	if (GEditor)
	{
		UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(ACompositeActor::StaticClass());
		if (ActorFactory)
		{
			AddedActor = Cast<ACompositeActor>(ActorFactory->CreateActor(ACompositeActor::StaticClass(), World->GetCurrentLevel(), FTransform()));
		}
	}

	// Fallback to world spawn if the factory spawn failed
	if (!AddedActor)
	{
		AddedActor = World->SpawnActor<ACompositeActor>(ACompositeActor::StaticClass());
	}

	if (AddedActor)
	{
		if (GEditor)
		{
			GEditor->SelectNone(true, true);
			GEditor->SelectActor(AddedActor, true, true);
		}

		RefreshAndExpandTreeView();
		
		FCompositeActorTreeItemPtr* TreeItem = CompositeActorTreeItems.FindByPredicate([AddedActor](const FCompositeActorTreeItemPtr& InTreeItem)
		{
			return InTreeItem.IsValid() && InTreeItem->CompositeActor == AddedActor;
		});

		if (TreeItem && TreeView.IsValid())
		{
			TreeView->SetSelection(*TreeItem);
		}
	}
}

void SCompositePanelLayerTree::OnLayerAdded(const UClass* InLayerClass)
{
	TWeakObjectPtr<ACompositeActor> CompositeActorWeak = nullptr;
	int32 NewLayerIndex = 0;

	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (!SelectedItems.IsEmpty() && SelectedItems[0].IsValid())
	{
		CompositeActorWeak = SelectedItems[0]->CompositeActor;
		NewLayerIndex = SelectedItems[0]->LayerIndex + 1;
	}
	else if (CompositeActorTreeItems.Num() == 1)
	{
		CompositeActorWeak = CompositeActorTreeItems[0]->CompositeActor;
	}

	TStrongObjectPtr<ACompositeActor> CompositeActor = CompositeActorWeak.Pin();
	
	if (!CompositeActor.IsValid())
	{
		return;
	}

	FScopedTransaction LayerAddedTransaction(LOCTEXT("LayerAddedTransaction", "Add Layer"));
	CompositeActor->Modify();

	UCompositeLayerBase* NewLayer = NewObject<UCompositeLayerBase>(CompositeActor.Get(), InLayerClass, NAME_None, RF_Transactional);

	FProperty* CompositeLayersProperty = FindFProperty<FProperty>(ACompositeActor::StaticClass(), GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));
	CompositeActor->PreEditChange(CompositeLayersProperty);
	
	CompositeActor->CompositeLayers.Insert(NewLayer, NewLayerIndex);

	// Auto-increment display name if a sibling of the same type already exists
	const FString UniqueName = UCompositePassBase::MakeUniqueDisplayName(NewLayer->GetDisplayName(), MakeArrayView(CompositeActor->CompositeLayers), NewLayer);
	NewLayer->SetDisplayName(UniqueName);

	FEditPropertyChain EditChain;
	EditChain.AddTail(CompositeLayersProperty);
	EditChain.SetActivePropertyNode(CompositeLayersProperty);

	FPropertyChangedEvent ChangeEvent(CompositeLayersProperty, EPropertyChangeType::ArrayAdd);
	FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
	CompositeActor->PostEditChangeChainProperty(ChainEvent);

	Composite::Analytics::RecordLayerAdded(*NewLayer);

	RefreshCompositeActorLayers(CompositeActor.Get());
}

void SCompositePanelLayerTree::OnCompositeActorActivated(const TStrongObjectPtr<ACompositeActor>& CompositeActor)
{
	if (!CompositeActor.IsValid())
	{
		StopPilotingCamera();
		return;
	}

	// Keep last active actor up-to-date
	LastActiveActor = CompositeActor.Get();

	AActor* CompositeCameraActor = CompositeActor->GetCameraActor().Get();

	if (!IsValid(CompositeCameraActor))
	{
		StopPilotingCamera();
		return;
	}
	
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		if (CompositeCameraActor != LevelEditorSubsystem->GetPilotLevelActor())
		{
			StopPilotingCamera();
		}
	}
}

FReply SCompositePanelLayerTree::OnLayerSoloToggled(const FCompositeActorTreeItemPtr& InLayerTreeItem)
{
	if (!InLayerTreeItem.IsValid() ||
		!InLayerTreeItem->HasValidCompositeLayer())
	{
		return FReply::Unhandled();
	}

	TStrongObjectPtr<ACompositeActor> CompositeActor = InLayerTreeItem->CompositeActor.Pin();
	if (!CompositeActor.IsValid())
	{
		return FReply::Unhandled();
	}
	
	UCompositeLayerBase* CompositeLayer = InLayerTreeItem->GetCompositeLayer();

	if (!CompositeLayer)
	{
		return FReply::Unhandled();
	}
	
	FScopedTransaction SoloLayerTransaction(LOCTEXT("SoloLayerTransaction", "Solo Layer"));
	CompositeActor->Modify();

	// Route through the property system so notifications fire; sibling un-solo happens in PostEditChangeProperty.
	FProperty* SoloProperty = FindFProperty<FProperty>(UCompositeLayerBase::StaticClass(), GET_MEMBER_NAME_CHECKED(UCompositeLayerBase, bIsSolo));
	CompositeLayer->PreEditChange(SoloProperty);
	CompositeLayer->bIsSolo = !CompositeLayer->bIsSolo;
	FPropertyChangedEvent SoloChangedEvent(SoloProperty, EPropertyChangeType::ValueSet);
	CompositeLayer->PostEditChangeProperty(SoloChangedEvent);

	// IDetailsView caches CanEditChange at build time; force a refresh so Operation greys out without requiring re-selection.
	OnLayerStateChanged.ExecuteIfBound();

	return FReply::Handled();
}

void SCompositePanelLayerTree::OnLayerMoved(const FCompositeActorTreeItemPtr& InTreeItem, int32 InDestIndex)
{
	if (!InTreeItem.IsValid() ||
		!InTreeItem->HasValidCompositeActor() ||
		!InTreeItem->HasValidCompositeLayer())
	{
		return;
	}
	
	TStrongObjectPtr<ACompositeActor> CompositeActor = InTreeItem->CompositeActor.Pin();
	if (!CompositeActor.IsValid())
	{
		return;
	}
	
	UCompositeLayerBase* CompositeLayer = InTreeItem->GetCompositeLayer();
	
	if (!CompositeLayer || !CompositeActor->CompositeLayers.IsValidIndex(InDestIndex))
	{
		return;
	}
	
	FScopedTransaction MoveLayerTransaction(LOCTEXT("MoveLayerTransaction", "Move Layer"));
	CompositeActor->Modify();

	FProperty* CompositeLayersProperty = FindFProperty<FProperty>(ACompositeActor::StaticClass(), GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));
	CompositeActor->PreEditChange(CompositeLayersProperty);
	
	CompositeActor->CompositeLayers.RemoveAt(InTreeItem->LayerIndex);
	CompositeActor->CompositeLayers.Insert(CompositeLayer, InDestIndex);

	FEditPropertyChain EditChain;
	EditChain.AddTail(CompositeLayersProperty);
	EditChain.SetActivePropertyNode(CompositeLayersProperty);

	FPropertyChangedEvent ChangeEvent(CompositeLayersProperty, EPropertyChangeType::ArrayMove);
	FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
	CompositeActor->PostEditChangeChainProperty(ChainEvent);
	
	RefreshCompositeActorLayers(CompositeActor.Get());
}

void SCompositePanelLayerTree::OnActorAddedToLevel(AActor* Actor)
{
	if (Actor->IsA<ACompositeActor>())
	{
		FillCompositeActorTree();
	}
}

void SCompositePanelLayerTree::OnLevelActorListChanged()
{
	FillCompositeActorTree();
}

void SCompositePanelLayerTree::OnActorRemovedFromLevel(AActor* Actor)
{
	if (Actor->IsA<ACompositeActor>())
	{
		FillCompositeActorTree();
		VerifyActiveCompositeActorPiloting();
	}
}

void SCompositePanelLayerTree::OnMapChange(uint32 MapFlags)
{
	FillCompositeActorTree();
}

void SCompositePanelLayerTree::OnNewCurrentLevel()
{
	FillCompositeActorTree();
}

void SCompositePanelLayerTree::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	FillCompositeActorTree();
}

void SCompositePanelLayerTree::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	FillCompositeActorTree();
}

void SCompositePanelLayerTree::OnObjectReplaced(const TMap<UObject*, UObject*>& Tuples)
{
	bool bContainsCompositeActor = false;
	for (const FCompositeActorTreeItemPtr& TreeItem : CompositeActorTreeItems)
	{
		if (Tuples.Contains(TreeItem->CompositeActor.GetEvenIfUnreachable()))
		{
			bContainsCompositeActor = true;
		}
	}

	if (bContainsCompositeActor)
	{
		FillCompositeActorTree();
		VerifyActiveCompositeActorPiloting();
	}
}

void SCompositePanelLayerTree::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (Object->IsA<ACompositeActor>())
	{
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers))
		{
			constexpr bool bRefreshTree = true;
			constexpr bool bPreserveSelection = true;
			RefreshCompositeActorLayers(Cast<ACompositeActor>(Object), bRefreshTree, bPreserveSelection);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositeActor, CameraActor))
		{
			ACompositeActor* CompositeActor = Cast<ACompositeActor>(Object);
			AActor* CompositeCameraActor = CompositeActor->GetCameraActor().Get();

			if (!IsValid(CompositeCameraActor))
			{
				StopPilotingCamera();
				return;
			}
	
			if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
			{
				if (CompositeCameraActor != LevelEditorSubsystem->GetPilotLevelActor())
				{
					StopPilotingCamera();
				}
			}
		}
	}
}

void SCompositePanelLayerTree::CopySelectedItems()
{
	TArray<UObject*> ObjectsToCopy;

	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	for (const FCompositeActorTreeItemPtr& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsValid() && SelectedItem->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* CompositeLayer = SelectedItem->GetCompositeLayer())
			{
				ObjectsToCopy.Add(CompositeLayer);
			}
		}
	}

	if (ObjectsToCopy.IsEmpty())
	{
		return;
	}
	
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	for (UObject* Object : ObjectsToCopy)
	{
		UExporter::ExportToOutputDevice(&Context, Object, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, nullptr);
	}

	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

bool SCompositePanelLayerTree::CanCopySelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (!SelectedItems.Num())
	{
		return false;
	}

	const bool bContainsLayer = SelectedItems.ContainsByPredicate([](const FCompositeActorTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->HasValidCompositeLayer() && InTreeItem->GetCompositeLayer();
	});

	return bContainsLayer;
}

void SCompositePanelLayerTree::CutSelectedItems()
{
	CopySelectedItems();
	DeleteSelectedItems();
}

bool SCompositePanelLayerTree::CanCutSelectedItems()
{
	return CanCopySelectedItems() && CanDeleteSelectedItems();
}

class FCompositeLayerObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FCompositeLayerObjectTextFactory() : FCustomizableTextObjectFactory(GWarn) { }

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		return ObjectClass->IsChildOf(UCompositeLayerBase::StaticClass());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if (UCompositeLayerBase* CompositeLayer = Cast<UCompositeLayerBase>(NewObject))
		{
			CompositeLayers.Add(CompositeLayer);
		}
	}

public:
	TArray<UCompositeLayerBase*> CompositeLayers;
};

void SCompositePanelLayerTree::PasteSelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	// A list of composite actors to paste the clipboard contents in, and the layer index to paste at
	TMap<ACompositeActor*, int32> CompositeActorsToPasteTo;
	for (const FCompositeActorTreeItemPtr& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsValid() && SelectedItem->HasValidCompositeActor())
		{
			if (CompositeActorsToPasteTo.Contains(SelectedItem->CompositeActor.Get()))
			{
				CompositeActorsToPasteTo[SelectedItem->CompositeActor.Get()] = FMath::Max(SelectedItem->LayerIndex + 1, CompositeActorsToPasteTo[SelectedItem->CompositeActor.Get()]);
			}
			else
			{
				CompositeActorsToPasteTo.Add(SelectedItem->CompositeActor.Get(), SelectedItem->LayerIndex + 1);
			}
		}
	}

	if (CompositeActorsToPasteTo.IsEmpty())
	{
		return;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	
	FCompositeLayerObjectTextFactory Factory;
	Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardContent);

	if (Factory.CompositeLayers.IsEmpty())
	{
		return;
	}

	TMap<ACompositeActor*, TArray<int32>> LayersToSelect;
	FScopedTransaction PasteLayersTransaction(LOCTEXT("PasteLayersTransaction", "Paste Layers"));
	for (const TPair<ACompositeActor*, int32>& Pair : CompositeActorsToPasteTo)
	{
		ACompositeActor* CompositeActor = Pair.Key;
		int32 IndexToPasteAt = Pair.Value;

		LayersToSelect.Add(CompositeActor, TArray<int32>());
		
		CompositeActor->Modify();
		for (UCompositeLayerBase* LayerToPaste : Factory.CompositeLayers)
		{
			UCompositeLayerBase* NewLayer = DuplicateObject(LayerToPaste, CompositeActor);
			
			FProperty* CompositeLayersProperty = FindFProperty<FProperty>(ACompositeActor::StaticClass(), GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));
			CompositeActor->PreEditChange(CompositeLayersProperty);
			
			CompositeActor->CompositeLayers.Insert(NewLayer, IndexToPasteAt);

			FEditPropertyChain EditChain;
			EditChain.AddTail(CompositeLayersProperty);
			EditChain.SetActivePropertyNode(CompositeLayersProperty);

			FPropertyChangedEvent ChangeEvent(CompositeLayersProperty, EPropertyChangeType::ArrayAdd);
			FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
			CompositeActor->PostEditChangeChainProperty(ChainEvent);
			
			LayersToSelect[CompositeActor].Add(IndexToPasteAt);
			++IndexToPasteAt;
		}

		constexpr bool bRefreshTree = false;
		RefreshCompositeActorLayers(CompositeActor, bRefreshTree);
	}

	RefreshAndExpandTreeView();

	for (const TPair<ACompositeActor*, TArray<int32>>& Pair : LayersToSelect)
	{
		SelectCompositeActorLayers(Pair.Key, Pair.Value);
	}
}

bool SCompositePanelLayerTree::CanPasteSelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (SelectedItems.IsEmpty())
	{
		return false;
	}

	const bool bHasValidCompositeActor = SelectedItems.ContainsByPredicate([](const FCompositeActorTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->HasValidCompositeActor();
	});

	if (!bHasValidCompositeActor)
	{
		return false;
	}
	
	// Can't paste unless the clipboard has a string in it
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (!ClipboardContent.IsEmpty())
	{
		FCompositeLayerObjectTextFactory Factory;
		Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional | RF_Transient, ClipboardContent);
		return !Factory.CompositeLayers.IsEmpty();
	}

	return false;
}

void SCompositePanelLayerTree::DuplicateSelectedItems()
{
	CopySelectedItems();
	PasteSelectedItems();
}

bool SCompositePanelLayerTree::CanDuplicateSelectedItems()
{
	return CanCopySelectedItems();
}

void SCompositePanelLayerTree::DeleteSelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();

	TSet<ACompositeActor*> ActorsToDelete;
	for (const FCompositeActorTreeItemPtr& Item : SelectedItems)
	{
		// Skip any invalid items, or any items that represent layers, as we want to find items that represent composite actors
		if (!Item.IsValid() ||
			!Item->HasValidCompositeActor() ||
			Item->HasValidCompositeLayer())
		{
			continue;
		}

		ActorsToDelete.Add(Item->CompositeActor.Get());
	}
	
	TMap<ACompositeActor*, TArray<int32>> LayersToDelete;
	for (const FCompositeActorTreeItemPtr& Item : SelectedItems)
	{
		if (!Item.IsValid() ||
			!Item->HasValidCompositeLayer())
		{
			continue;
		}

		// Skip any layers whose owning actor is already getting deleted
		if (ActorsToDelete.Contains(Item->CompositeActor.Get()))
		{
			continue;
		}

		if (LayersToDelete.Contains(Item->CompositeActor.Get()))
		{
			LayersToDelete[Item->CompositeActor.Get()].Add(Item->LayerIndex);
		}
		else
		{
			LayersToDelete.Add(Item->CompositeActor.Get(), TArray<int32> { Item->LayerIndex });
		}
	}

	if (ActorsToDelete.IsEmpty() && LayersToDelete.IsEmpty())
	{
		return;
	}

	FScopedTransaction DeleteActorTransaction(LOCTEXT("DeleteTransaction", "Delete Actors/Layers"));
	
	if (ActorsToDelete.Num())
	{
		for (ACompositeActor* Actor : ActorsToDelete)
		{
			if (IsValid(Actor))
			{
				Actor->Destroy();
			}
		}
	}
	
	if (LayersToDelete.Num())
	{
		for (TPair<ACompositeActor*, TArray<int32>>& CompositeActorLayerList : LayersToDelete)
		{
			ACompositeActor* CompositeActor = CompositeActorLayerList.Key;
			CompositeActor->Modify();
			
			// Delete the layers in reverse order so that indices don't get messed up as we delete
			TArray<int32>& LayerIndices = CompositeActorLayerList.Value;
			LayerIndices.Sort();

			FProperty* CompositeLayersProperty = FindFProperty<FProperty>(ACompositeActor::StaticClass(), GET_MEMBER_NAME_CHECKED(ACompositeActor, CompositeLayers));
			CompositeActor->PreEditChange(CompositeLayersProperty);
			
			for (int32 Index = LayerIndices.Num() - 1; Index >= 0; --Index)
			{
				const int32 LayerIndex = LayerIndices[Index];

				// Here we replicate details panels instanced property array removal code.
				// The removed instanced layer objects must be moved to the transient package.
				TObjectPtr<UCompositeLayerBase>& LayerToRemove = CompositeActor->CompositeLayers[LayerIndex];
				if (IsValid(LayerToRemove))
				{
					LayerToRemove->Modify();
					LayerToRemove->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}

				CompositeActor->CompositeLayers.RemoveAt(LayerIndex);
			}

			FEditPropertyChain EditChain;
			EditChain.AddTail(CompositeLayersProperty);
			EditChain.SetActivePropertyNode(CompositeLayersProperty);

			FPropertyChangedEvent ChangeEvent(CompositeLayersProperty, EPropertyChangeType::ArrayRemove);
			FPropertyChangedChainEvent ChainEvent(EditChain, ChangeEvent);
			CompositeActor->PostEditChangeChainProperty(ChainEvent);
			
			int32 TreeItemIndex = CompositeActorTreeItems.IndexOfByPredicate([CompositeActor](const FCompositeActorTreeItemPtr& InTreeItem)
			{
				return InTreeItem->CompositeActor.Get() == CompositeActor;
			});

			if (TreeItemIndex != INDEX_NONE && ActorsToDelete.IsEmpty())
			{
				FillCompositeActorTreeItem(CompositeActorTreeItems[TreeItemIndex]);
			}
		}
	}

	if (!ActorsToDelete.IsEmpty())
	{
		// If an actor was deleted, rebuild the whole tree view
		constexpr bool bPreserveSelection = false;
		FillCompositeActorTree(bPreserveSelection);
	}
	else if (!LayersToDelete.IsEmpty())
	{
		// Otherwise, we can just refresh the tree view since the layers were updated with FillCompositeActorTreeItem
		RefreshAndExpandTreeView();
	}
}

bool SCompositePanelLayerTree::CanDeleteSelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	return !SelectedItems.IsEmpty();
}

void SCompositePanelLayerTree::RenameSelectedItem()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (SelectedItems.Num() != 1 || !SelectedItems[0]->HasValidCompositeLayer() || !SelectedItems[0]->GetCompositeLayer())
	{
		return;
	}
	
	TSharedPtr<ITableRow> RowWidget = TreeView->WidgetFromItem(SelectedItems[0]);
	if (TSharedPtr<SCompositePanelLayerTreeItemRow> TreeItemRowWidget = StaticCastSharedPtr<SCompositePanelLayerTreeItemRow>(RowWidget))
	{
		TreeItemRowWidget->RequestRename();
	}
}

bool SCompositePanelLayerTree::CanRenameSelectedItem()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	if (SelectedItems.Num() == 1)
	{
		return SelectedItems[0].IsValid() && SelectedItems[0]->HasValidCompositeLayer() && SelectedItems[0]->GetCompositeLayer();
	}
	
	return false;
}

void SCompositePanelLayerTree::EnableSelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();

	bool bAnyEnabled = false;
	for (const FCompositeActorTreeItemPtr& Item : SelectedItems)
	{
		TStrongObjectPtr<ACompositeActor> PinnedCompositeActor = Item->CompositeActor.Pin();
		if (!PinnedCompositeActor.IsValid())
		{
			continue;
		}
		
		if (Item->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* Layer = Item->GetCompositeLayer())
			{
				if (Layer->GetIsEnabled())
				{
					bAnyEnabled = true;
				}
			}
		}
		else
		{
			if (PinnedCompositeActor->GetIsEnabled())
			{
				bAnyEnabled = true;
			}
		}
	}

	const bool bSetEnabled = !bAnyEnabled;

	FScopedTransaction SetEnabledTransaction(LOCTEXT("SetEnabledTransaction", "Set Enabled"));
	for (const FCompositeActorTreeItemPtr& Item : SelectedItems)
	{
		TStrongObjectPtr<ACompositeActor> PinnedCompositeActor = Item->CompositeActor.Pin();
		if (!PinnedCompositeActor.IsValid())
		{
			continue;
		}
		
		if (Item->HasValidCompositeLayer())
		{
			if (UCompositeLayerBase* Layer = Item->GetCompositeLayer())
			{
				Layer->Modify();
				Layer->SetIsEnabled(bSetEnabled);
			}
		}
		else
		{
			PinnedCompositeActor->Modify();
			PinnedCompositeActor->SetIsEnabled(bSetEnabled);
		}
	}
}

bool SCompositePanelLayerTree::CanEnableSelectedItems()
{
	TArray<FCompositeActorTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FCompositeActorTreeItemPtr>();
	return !SelectedItems.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
