// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "EditorViewportClient.h"
#include "Layers/CompositeLayerBase.h"
#include "Misc/Optional.h"
#include "UI/CompositeEditorPanelSettings.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class SCompositePanelLayerTreeToolbar;
class ACompositeActor;
class UMaterialParameterCollection;
template<typename T> class STreeView;

/**
 * A hierarchical view that displays all compositing actors and their rendering layers in the level
 */
class SCompositePanelLayerTree : public SCompoundWidget, public FEditorUndoClient
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, const TArray<UObject*>&);
	DECLARE_DELEGATE(FOnLayerStateChanged);
	
private:
	/** Item that stores a composite actor and layer index to display in the layer tree view */
	struct FCompositeActorTreeItem
	{
		/** Composite actor whose layer is being displayed */
		TWeakObjectPtr<ACompositeActor> CompositeActor = nullptr;

		/** Index of the layer in the composite actor's CompositeLayers list. If INDEX_NONE, then this tree item is the composite actor itself */
		int32 LayerIndex = INDEX_NONE;

		/** Child tree items of this tree item */
		TArray<TSharedPtr<FCompositeActorTreeItem>> Children;

		/** Indicates that this item is being filtered out by the active filter */
		bool bFilteredOut = false;

		/** Gets whether this tree item references a valid composite actor */
		bool HasValidCompositeActor() const;

		/** Gets whether this tree item references a valid composite layer */
		bool HasValidCompositeLayer() const;

		/** Gets the composite layer this tree item references */
		UCompositeLayerBase* GetCompositeLayer() const;
	};

	using FCompositeActorTreeItemPtr = TSharedPtr<FCompositeActorTreeItem>;
	
public:
	SLATE_BEGIN_ARGS(SCompositePanelLayerTree) { }
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnLayerStateChanged, OnLayerStateChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual ~SCompositePanelLayerTree() override;

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// End of SWidget interface
	
	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** Selects the specified composite actors in the layer tree */
	void SelectCompositeActors(const TArray<TWeakObjectPtr<ACompositeActor>>& InCompositeActors);

	/** Selects all matching composite actors and layers in the tree from the specified list of objects */
	void SelectObjects(const TArray<UObject*>& InObjects);

	/** Gets a list of the currently selected composite actors and layers in the tree */
	TArray<UObject*> GetSelectedObjects();
	
	/** Computes the minimum height necessary to display the layer tree with all layers visible */
	float GetMinimumHeight() const;
	
private:	
	/** Binds commands and callbacks to the command list */
	void BindCommands();
	
	/** Fills the tree view with all composite actors found on the current level */
	void FillCompositeActorTree(bool bPreserveSelection = false);

	/** Fills the children of the composite actor tree item with tree items for each of the composite actor's layers */
	void FillCompositeActorTreeItem(FCompositeActorTreeItemPtr& InOutTreeItem);

	/** Refreshes the child tree elements for the specified composite actor */
	void RefreshCompositeActorLayers(ACompositeActor* InCompositeActor, bool bRefreshTree = true, bool bPreserveSelection = false);

	/** Selects the specified layers of a composite actor in the tree view */
	void SelectCompositeActorLayers(ACompositeActor* InCompositeActor, const TArray<int32>& InLayersToSelect);
	
	/** Refreshes the tree view and expands all tree view items */
	void RefreshAndExpandTreeView();

	/** Gets whether it is possible to add a layer based on active selection or lack thereof. */
	bool CanAddLayer() const;

	/** Gets whether the currently selected composite actor has a camera that can be piloted */
	bool CanPilotCamera() const;

	/** Gets whether camera actor of the currently selected composite actor is being piloted */
	bool IsPilotingCamera() const;

	/** Sets the piloted actor to the currently selected composite actor camera, or clears it if it is already being piloted */
	void OnPilotCameraToggled(bool bPilotCamera);

	/** Gets the currently active composite actor's camera, or nullptr if no active composite actor or no valid camera was found */
	AActor* GetActiveCompositeActorCamera() const;

	/** Starts piloting the specified camera actor */
	void PilotCamera(AActor* InCameraActorToPilot);

	/** Stops piloting the currently piloted camera actor */
	void StopPilotingCamera();

	/** Checks whether the current active composite actor's camera is still the actor being piloted, and if not, stops piloting */
	void VerifyActiveCompositeActorPiloting();

	/** Gets the currently selected composite mesh visualization mode from the editor settings. */
	ECompositeMeshVisualizationMode GetMeshVisualizationMode() const;

	/** Updates the editor settings with the new visualization mode and refreshes the MPC scalar immediately. */
	void SetMeshVisualizationMode(ECompositeMeshVisualizationMode InMode);

	/** Computes the effective visualization scalar (0 or 1) for the given mode and piloting state. */
	static float ResolveMeshVisualizationValue(ECompositeMeshVisualizationMode InMode, bool bIsPilotingCompositeCamera);

	/** Writes the effective visualization scalar to MPC_Composite if it differs from the last-written value. */
	void UpdateMeshVisualizationMPC();

	/** Gets the global active checkbox state. */
	ECheckBoxState GetGlobalActiveCheckState() const;

	/** Sets the Active flag for all currently filtered items in the tree view */
	void OnGlobalActiveCheckStateChanged(ECheckBoxState CheckBoxState);

	/** Gets the check state for the global enabled checkbox */
	ECheckBoxState GetGlobalEnabledCheckState() const;

	/** Sets the Enabled flag for all currently filtered items in the tree view */
	void OnGlobalEnabledCheckStateChanged(ECheckBoxState CheckBoxState);

	/** Callback that is raised when the selected items in the tree view change */
	void OnLayerSelectionChanged(FCompositeActorTreeItemPtr InTreeItem, ESelectInfo::Type SelectInfo);

	/** Invokes the OnSelectionChanged delegate with the appropriate selected layers */
	void InvokeLayerSelectionChanged();
	
	/** Callback that is raised when the tree view filter is changed, applies the new filter to the composite actor tree items */
	void OnFilterChanged();

	/** Gets the status text to display at the bottom of the tree view that shows the number of actors filtered and selected */
	FText GetFilterStatusText() const;

	/** Gets the color of the filter status text */
	FSlateColor GetFilterStatusTextColor() const;

	/** Creates the right click context menu for the tree view */
	TSharedPtr<SWidget> CreateTreeContextMenu();

	/** Adds a new composite actor to the level */
	void AddCompositeActor();
	
	/** Raised when a new layer is being added to the selected composite actor */
	void OnLayerAdded(const UClass* InLayerClass);

	/** Raised when a composite actor is made the composite actor */
	void OnCompositeActorActivated(const TStrongObjectPtr<ACompositeActor>& CompositeActor);
	
	/** Raised when the Solo button has been toggled on the specified layer tree item */
	FReply OnLayerSoloToggled(const FCompositeActorTreeItemPtr& InLayerTreeItem);

	/** Raised when a layer has been moved via a drag drop operation to the specified destination index */
	void OnLayerMoved(const FCompositeActorTreeItemPtr& InTreeItem, int32 InDestIndex);

	/** Callbacks for any editor events that may necessitate updating the composite actors being displayed in the tree view */
	void OnActorAddedToLevel(AActor* Actor);
	void OnLevelActorListChanged();
	void OnActorRemovedFromLevel(AActor* Actor);
	void OnMapChange(uint32 MapFlags);
	void OnNewCurrentLevel();
	void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);
	void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);
	void OnObjectReplaced(const TMap<UObject*, UObject*>& Tuples);
	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	/** Generic command callbacks to handle copy, cut, paste, delete, and rename */
	void CopySelectedItems();
	bool CanCopySelectedItems();
	void CutSelectedItems();
	bool CanCutSelectedItems();
	void PasteSelectedItems();
	bool CanPasteSelectedItems();
	void DuplicateSelectedItems();
	bool CanDuplicateSelectedItems();
	void DeleteSelectedItems();
	bool CanDeleteSelectedItems();
	void RenameSelectedItem();
	bool CanRenameSelectedItem();
	void EnableSelectedItems();
	bool CanEnableSelectedItems();
	
private:
	/** Tree view widget that displays the level's composite actors and their layers */
	TSharedPtr<STreeView<FCompositeActorTreeItemPtr>> TreeView;

	/** A list of all tree items that can be displayed in the tree view */
	TArray<FCompositeActorTreeItemPtr> CompositeActorTreeItems;

	/** The actual list of tree items to display when a filter has been applied */
	TArray<FCompositeActorTreeItemPtr> FilteredCompositeActorTreeItems;
	
	/** Toolbar being displayed above the tree view */
	TSharedPtr<SCompositePanelLayerTreeToolbar> Toolbar;

	/** The root widget that contains all the footer content */
	TSharedPtr<SWidget> FooterContainer;
	
	/** The command list for commands related to the tree view */
	TSharedPtr<FUICommandList> CommandList;

	/** Raised when the selected tree items have changed */
	FOnSelectionChanged OnSelectionChanged;

	/** Raised when a layer's editable state has changed (e.g. Solo toggled) and the details panel should re-evaluate CanEditChange */
	FOnLayerStateChanged OnLayerStateChanged;

	/** Last active actor. */
	TWeakObjectPtr<ACompositeActor> LastActiveActor;

	/**
	 * Used to store last perspective camera transform before piloting (Actor Lock)
	 * Once piloting ends, this transform is re-applied to the camera.
	 */
	TOptional<FViewportCameraTransform> CachedPerspectiveCameraTransform;

	/** Cached field of view to re-apply once piloting ends. */
	TOptional<float> CachedViewFOV;

	/** When true, selection changes in the tree view will not invoke the SelectionChanged delegate */
	bool bSilenceSelectionChanges = false;

	/** Stores whether the active composite actor's camera is currently being piloted */
	bool bCachedIsPilotingCamera = false;

	/** Lazy-loaded reference to MPC_Composite used to publish the visualization scalar. */
	TWeakObjectPtr<UMaterialParameterCollection> CachedVisualizationMPC;

	/** World that LastWrittenMeshVisualization refers to; cache is invalidated when the editor world changes. */
	TWeakObjectPtr<UWorld> CachedVisualizationWorld;

	/** Last value written to the MPC scalar to avoid redundant per-tick updates. */
	TOptional<float> LastWrittenMeshVisualization;

	friend class SCompositePanelLayerTreeItemRow;
	friend class SCompositePanelLayerTreeToolbar;
};
