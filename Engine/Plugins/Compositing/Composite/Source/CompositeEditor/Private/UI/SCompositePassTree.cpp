// Copyright Epic Games, Inc. All Rights Reserved.


#include "SCompositePassTree.h"

#include "CompositeEditorCommands.h"
#include "CompositeEditorStyle.h"
#include "Factories.h"
#include "SActionButton.h"
#include "ScopedTransaction.h"
#include "UnrealExporter.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Exporters/Exporter.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Framework/Commands/GenericCommands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "CompositeAnalytics.h"
#include "Layers/CompositeLayerPlate.h"
#include "Misc/StringOutputDevice.h"
#include "Passes/CompositePassDistortion.h"
#include "Passes/CompositePassMasking.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Package.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SCompositePassTree"

namespace CompositePassTree
{
	static const FName PassTreeColumn_Enabled = "Enabled";
	static const FName PassTreeColumn_Name = "Name";
	static const FName PassTreeColumn_Type = "Type";
}

/**
 * Widget for the filter bar, needs to be a subclass that overrides MakeAddFilterMenu to give the filter bar its own unique menu name,
 * otherwise the editor will get confused with any other basic filter bar used elsewhere.
 */
template<typename TFilterType>
class SCompositePassesFilterBar : public SBasicFilterBar<TFilterType>
{
	using Super = SBasicFilterBar<TFilterType>;
	
public:

	using FOnFilterChanged = typename SBasicFilterBar<TFilterType>::FOnFilterChanged;
	using FCreateTextFilter = typename SBasicFilterBar<TFilterType>::FCreateTextFilter;

	SLATE_BEGIN_ARGS(SCompositePassesFilterBar)
	{}
		SLATE_EVENT(SCompositePassesFilterBar<TFilterType>::FOnFilterChanged, OnFilterChanged)
		SLATE_ARGUMENT(TArray<TSharedRef<FFilterBase<TFilterType>>>, CustomFilters)	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		typename SBasicFilterBar<TFilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._UseSectionsForCategories = true;
		Args._bPinAllFrontendFilters = true;
		
		SBasicFilterBar<TFilterType>::Construct(Args.FilterPillStyle(EFilterPillStyle::Basic));
	}

private:
	virtual TSharedRef<SWidget> MakeAddFilterMenu() override
	{
		const FName FilterMenuName = "CompositePlatePassesFilterBar.FilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UFilterBarContext* Context = InMenu->FindContext<UFilterBarContext>())
				{
					Context->PopulateFilterMenu.ExecuteIfBound(InMenu);
					Context->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
				}
			}));
		}

		UFilterBarContext* FilterBarContext = NewObject<UFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateLambda([this](UToolMenu* Menu)
		{
			Super::PopulateCommonFilterSections(Menu);
			Super::PopulateCustomFilters(Menu);
		});
		FToolMenuContext ToolMenuContext(FilterBarContext);
		
		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}
};

/** Custom filter for the filter bar so that events can be received for when the filter is activated or deactivated */
class FCompositePassTreeFilter : public FGenericFilter<SCompositePassTree::FPassTreeItemPtr>
{
public:
	DECLARE_DELEGATE_OneParam(FOnActiveStateChanged, bool)
	
public:
	FCompositePassTreeFilter(const TSharedPtr<FFilterCategory>& InCategory,
		const FString& InName,
		const FText& DisplayName,
		const FOnItemFiltered& InFilterDelegate)
		: FGenericFilter<TSharedPtr<SCompositePassTree::FPassTreeItem>>(InCategory, InName, DisplayName, InFilterDelegate)
	{ }

	/** Activates the filter directly, avoiding invoking OnActiveStateChanged */
	void ActivateFilter()
	{
		TGuardValue<bool> Guard(bStateChangeGuard, true);
		SetActive(true);
	}

	/** Deactivates the filter directly, avoiding invoking OnActiveStateChanged */
	void DeactivateFilter()
	{
		TGuardValue<bool> Guard(bStateChangeGuard, true);
		SetActive(false);
	}

	/** Raised when the state of the filter is chanaged */
	virtual void ActiveStateChanged(bool bActive) override
	{
		if (bStateChangeGuard)
		{
			return;
		}

		OnActiveStateChanged.ExecuteIfBound(bActive);
	}

public:
	FOnActiveStateChanged OnActiveStateChanged;
	bool bStateChangeGuard = false;
};

/** Toolbar widget that contains filtering and an add button */
class SCompositePassTreeToolbar : public SCompoundWidget
{
private:
	using FFilterType = SCompositePassTree::FPassTreeItemPtr;
	using FTreeFilter =  TTextFilter<FFilterType>;
	
public:
	DECLARE_DELEGATE(FOnFilterChanged)
	DECLARE_DELEGATE_OneParam(FOnPassAdded, const UClass* /* InPassClass */)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilterNewPassType, const UClass*);
	
	SLATE_BEGIN_ARGS(SCompositePassTreeToolbar) {}
		SLATE_EVENT(FOnFilterChanged, OnFilterChanged)
		SLATE_EVENT(FOnPassAdded, OnPassAdded)
		SLATE_EVENT(FOnFilterNewPassType, OnFilterNewPassType)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<ICompositePassListOwner>& InPassListOwner)
	{
		PassListOwner = InPassListOwner;
		OnFilterChanged = InArgs._OnFilterChanged;
		OnPassAdded = InArgs._OnPassAdded;
		OnFilterNewPassType = InArgs._OnFilterNewPassType;

		TArray<TSharedRef<FFilterBase<FFilterType>>> AllFilters;
		InitializeFilters(AllFilters);
		
		FilterBar = SNew(SCompositePassesFilterBar<FFilterType>)
			.CustomFilters(AllFilters)
			.OnFilterChanged(this, &SCompositePassTreeToolbar::FilterChanged);

		if (AllPassesFilter.IsValid())
		{
			AllPassesFilter->SetActive(true);
		}
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SActionButton)
					.ActionButtonType(EActionButtonType::Simple)
					.Icon(FAppStyle::GetBrush("Icons.PlusCircle"))
					.ToolTipText(LOCTEXT("AddPassButtonToolTip", "Add a new pass to the selected list of passes"))
					.OnGetMenuContent(this, &SCompositePassTreeToolbar::GetAddPassMenuContent)
				]
				
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(TextFilterSearchBox, SFilterSearchBox)
					.HintText(LOCTEXT("FilterSearch", "Search..."))
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search for specific passes"))
					.OnTextChanged_Lambda([this](const FText& InText)
					{
						TreeTextFilter->SetRawFilterText(InText);
					})
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				FilterBar.ToSharedRef()
			]
		];
	}

	bool ItemPassesFilters(const FFilterType& InItem) const
	{
		// Performs an OR check on any of the filters that are enabled in the filter bar, mirroring the scene outliner filter bar behavior
		auto PassesAnyFilterBarFilter = [this](const FFilterType& InItem)
		{
			TSharedPtr<TFilterCollection<FFilterType>> FilterCollection = FilterBar->GetAllActiveFilters();
			const int32 NumFilters = FilterCollection->Num();
			if (NumFilters == 0)
			{
				return true;
			}
		
			for (const TSharedPtr<IFilter<FFilterType>>& Filter : *FilterCollection.Get())
			{
				if (Filter->PassesFilter(InItem))
				{
					return true;
				}
			}

			return false;
		};
		
		return TreeTextFilter->PassesFilter(InItem) && PassesAnyFilterBarFilter(InItem);
	}

	/** Returns the group index of the active group filter pill, or INDEX_NONE if "All Passes" (or no filter) is active */
	int32 GetActiveGroupFilterIndex() const
	{
		for (int32 Index = 0; Index < GroupFilters.Num(); ++Index)
		{
			if (GroupFilters[Index].IsValid() && GroupFilters[Index]->IsActive())
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}

	void ClearFilters()
	{
		// Prevent filter changed callback from firing while changing filter status, will invoke once all filters are reset
		TGuardValue<bool> FilterChangedGuard(bFilterChangedGuard, true);
		for (const TSharedPtr<FCompositePassTreeFilter>& Filter : GroupFilters)
		{
			Filter->DeactivateFilter();
		}

		if (AllPassesFilter.IsValid())
		{
			AllPassesFilter->ActivateFilter();
		}

		TextFilterSearchBox->SetText(FText::GetEmpty());

		OnFilterChanged.ExecuteIfBound();
	}
	
private:
	/** Creates all the filter bar filters and initializes the text filter */
	void InitializeFilters(TArray<TSharedRef<FFilterBase<FFilterType>>>& OutAllFilters)
	{
		TSharedPtr<FFilterCategory> BasicFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("BasicFiltersCategory", "Basic"), FText::GetEmpty());

		const int32 NumGroups = PassListOwner.IsValid() ? PassListOwner->GetNumGroups() : 0;
		for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
		{
			ICompositePassListOwner::FGroupFilterConfig FilterConfig = PassListOwner->GetGroupFilterConfig(GroupIndex);
			TSharedPtr<FCompositePassTreeFilter> PassTypeFilter = MakeShared<FCompositePassTreeFilter>(
				BasicFiltersCategory,
				FilterConfig.FilterName,
				FilterConfig.DisplayName,
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([GroupIndex](FFilterType InItem)
				{
					if (!InItem.IsValid())
					{
						return false;
					}

					return InItem->GroupIndex == GroupIndex;
				}));
	
			PassTypeFilter->SetToolTipText(FilterConfig.ToolTip);
			PassTypeFilter->OnActiveStateChanged.BindSP(this, &SCompositePassTreeToolbar::OnGroupFilterStateChanged, GroupIndex);

			GroupFilters.Add(PassTypeFilter);
			OutAllFilters.Add(PassTypeFilter.ToSharedRef());
		}

		if (NumGroups > 0)
		{
			AllPassesFilter = MakeShared<FCompositePassTreeFilter>(
					BasicFiltersCategory,
					TEXT("AllPassesFilter"),
					LOCTEXT("AllPassesFilterName", "All Passes"),
					FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
					{
						return true;
					}));
	
			AllPassesFilter->SetToolTipText(LOCTEXT("AllPassesFilterToolTip", "Show all passes"));
			AllPassesFilter->OnActiveStateChanged.BindSP(this, &SCompositePassTreeToolbar::OnAllPassesFilterStateChanged);
		
			OutAllFilters.Add(AllPassesFilter.ToSharedRef());
		}
		
		TreeTextFilter = MakeShared<FTreeFilter>(FTreeFilter::FItemToStringArray::CreateLambda([](const FFilterType& InItem, TArray<FString>& OutStrings)
		{
			if (InItem.IsValid())
			{
				if (InItem->HasValidPassIndex())
				{
					if (UCompositePassBase* Pass = InItem->GetPass())
					{
						OutStrings.Add(Pass->GetClass()->GetDisplayNameText().ToString());
					}
					else
					{
						OutStrings.Add(TEXT("None"));
					}
				}

				const FString GroupFilterString = InItem->GetGroupFilterString();
				if (!GroupFilterString.IsEmpty())
				{
					OutStrings.Add(GroupFilterString);
				}
			}
		}));
		TreeTextFilter->OnChanged().AddSP(this, &SCompositePassTreeToolbar::FilterChanged);
	}
	
	TSharedRef<SWidget> GetAddPassMenuContent()
	{
		constexpr bool bCloseMenuAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

		TArray<UClass*> BasePassTypes;
		const bool bRecursive = false;
		GetDerivedClasses(UCompositePassBase::StaticClass(), BasePassTypes, bRecursive);

		auto ShouldSkipClass = [](const UClass* InClass)
			{
				constexpr EClassFlags InvalidClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract;
				return InClass->HasAnyClassFlags(InvalidClassFlags) ||
					InClass->IsChildOf(UCompositeLayerBase::StaticClass()) ||
					InClass->GetName().StartsWith(TEXT("SKEL_")) ||
					InClass->GetName().StartsWith(TEXT("REINST_"));
			};
		
		// Sort name overrides to allow some control over how passes are sorted
		static TMap<UClass*, FText> SortNameOverrides =
		{
			// Ultimatte masking pass should display just below regular masking pass in sort order, so override its sort name to "Masking|Ultimatte Masking"
			{
				UCompositePassUltimatteMasking::StaticClass(),
				FText::Format(FText::FromString(TEXT("{0}|{1}")), UCompositePassMasking::StaticClass()->GetDisplayNameText(), UCompositePassUltimatteMasking::StaticClass()->GetDisplayNameText())
			}
		};
		
		// Sort the types alphabetically based on their display name
		BasePassTypes.Sort([](const UClass& InClassA, const UClass& InClassB)
		{
			FText ADisplayName = SortNameOverrides.Contains(&InClassA) ? SortNameOverrides[&InClassA] : InClassA.GetDisplayNameText();
			FText BDisplayName = SortNameOverrides.Contains(&InClassB) ? SortNameOverrides[&InClassB] : InClassB.GetDisplayNameText();

			return ADisplayName.CompareToCaseIgnored(BDisplayName) < 0;
		});

		// Add the native child classes of UCompositePassBase to the menu first
		for (const UClass* BasePassType : BasePassTypes)
		{
			if (ShouldSkipClass(BasePassType))
			{
				continue;
			}

			if (OnFilterNewPassType.IsBound())
			{
				if (OnFilterNewPassType.Execute(BasePassType))
				{
					continue;
				}
			}
			
			MenuBuilder.AddMenuEntry(
				BasePassType->GetDisplayNameText(),
				BasePassType->GetToolTipText(),
				FSlateIconFinder::FindIconForClass(BasePassType),
				FUIAction(FExecuteAction::CreateSP(this, &SCompositePassTreeToolbar::AddPass, BasePassType)));
		}
				
		// Add any blueprint derived classes in a new section of the menu
		bool bHasAddedSeparator = false;
		for (const UClass* BaseLayerType : BasePassTypes)
		{
			TArray<UClass*> ChildLayerTypes;
			GetDerivedClasses(BaseLayerType, ChildLayerTypes);

			for (const UClass* ChildLayerType : ChildLayerTypes)
			{
				if (ShouldSkipClass(ChildLayerType))
				{
					continue;
				}

				if (OnFilterNewPassType.IsBound())
				{
					if (OnFilterNewPassType.Execute(ChildLayerType))
					{
						continue;
					}
				}
				
				if (!bHasAddedSeparator)
				{
					MenuBuilder.AddSeparator();
					bHasAddedSeparator = true;
				}

				MenuBuilder.AddMenuEntry(
					ChildLayerType->GetDisplayNameText(),
					ChildLayerType->GetToolTipText(),
					FSlateIconFinder::FindIconForClass(ChildLayerType),
					FUIAction(FExecuteAction::CreateSP(this, &SCompositePassTreeToolbar::AddPass, ChildLayerType)));
			}
		}
		
		return MenuBuilder.MakeWidget();
	}

	void AddPass(const UClass* InPassClass)
	{
		OnPassAdded.ExecuteIfBound(InPassClass);	
	}

	void OnGroupFilterStateChanged(bool bActive, int32 GroupIndex)
	{
		if (bActive)
		{
			for (int32 Index = 0; Index < GroupFilters.Num(); ++Index)
			{
				if (Index != GroupIndex && GroupFilters[Index]->IsActive())
				{
					GroupFilters[Index]->SetActive(false);
				}
			}
			
			if (AllPassesFilter.IsValid() && AllPassesFilter->IsActive())
			{
				AllPassesFilter->DeactivateFilter();
			}
		}
		else
		{
			if (AllPassesFilter.IsValid() && !AllPassesFilter->IsActive())
			{
				AllPassesFilter->ActivateFilter();
			}
		}
	}

	void OnAllPassesFilterStateChanged(bool bActive)
	{
		if (bActive)
		{
			for (TSharedPtr<FCompositePassTreeFilter>& Filter : GroupFilters)
			{
				if (Filter->IsActive())
				{
					Filter->DeactivateFilter();
				}
			}
		}
		else
		{
			if (!GroupFilters.IsEmpty())
			{
				if (!GroupFilters[0]->IsActive())
				{
					GroupFilters[0]->ActivateFilter();
				}
			}
		}
	}
	
	void FilterChanged()
	{
		if (bFilterChangedGuard)
		{
			return;
		}

		OnFilterChanged.ExecuteIfBound();
	}
	
private:
	TSharedPtr<ICompositePassListOwner> PassListOwner;
	
	TSharedPtr<FCompositePassTreeFilter> AllPassesFilter;
	TArray<TSharedPtr<FCompositePassTreeFilter>> GroupFilters;
	TSharedPtr<FTreeFilter> TreeTextFilter;
	
	TSharedPtr<SFilterSearchBox> TextFilterSearchBox;
	TSharedPtr<SCompositePassesFilterBar<FFilterType>> FilterBar;

	bool bFilterChangedGuard = false;
	
	FOnFilterChanged OnFilterChanged;
	FOnPassAdded OnPassAdded;
	FOnFilterNewPassType OnFilterNewPassType;
};

/** Table row that displays the pass tree item content in the tree view */
class SCompositePassTreeItemRow : public SMultiColumnTableRow<SCompositePassTree::FPassTreeItemPtr>
{
	using FTreeItem = SCompositePassTree::FPassTreeItem;
	using FTreeItemPtr = SCompositePassTree::FPassTreeItemPtr;
	using FTreeItemWeakPtr = TWeakPtr<FTreeItem>;
	
public:
	DECLARE_DELEGATE_ThreeParams(FOnPassMoved, const FTreeItemPtr& /* InItemToMove */, int32 /* InDestGroupIndex */, int32 /* InDestIndex */)

private:
	/** Drag/drop operation for changing the order of passes in the tree view */
	class FPassDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FCompositeLayerDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FPassDragDropOp> New(const FTreeItemPtr& InDraggedItem)
		{
			TSharedRef<FPassDragDropOp> Operation = MakeShared<FPassDragDropOp>();
			Operation->DraggedItem = InDraggedItem;
			Operation->Construct();
			return Operation;
		}

		virtual void Construct() override
		{
			if (FTreeItemPtr PinnedItem = DraggedItem.Pin())
			{
				if (UCompositePassBase* Pass = PinnedItem->GetPass())
				{
					const FText PassName = Pass->GetClass()->GetDisplayNameText();
					const FSlateBrush* PassIcon = FSlateIconFinder::FindIconForClass(Pass->GetClass()).GetIcon();
					SetToolTip(PassName, PassIcon);
				}
			}
			
			FDecoratedDragDropOp::Construct();
		}

		FTreeItemWeakPtr DraggedItem;
	};
	
public:
	SLATE_BEGIN_ARGS(SCompositePassTreeItemRow) { }
		SLATE_EVENT(FOnPassMoved, OnPassMoved)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, FTreeItemPtr InTreeItem)
	{
		TreeItem = InTreeItem;
		OnPassMoved = InArgs._OnPassMoved;

		STableRow<FTreeItemPtr>::FArguments Args = FSuperRowType::FArguments()
           .Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
           .OnDragDetected(this, &SCompositePassTreeItemRow::HandleDragDetected)
           .OnCanAcceptDrop(this, &SCompositePassTreeItemRow::HandleCanAcceptDrop)
           .OnAcceptDrop(this, &SCompositePassTreeItemRow::HandleAcceptDrop);
		
		SMultiColumnTableRow::Construct(Args, InOwnerTable);
	}

	/** Sets the name text block to be in edit mode */
	void RequestRename()
	{
		if (NameTextBlock.IsValid())
		{
			NameTextBlock->EnterEditingMode();
		}
	}
	
private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == CompositePassTree::PassTreeColumn_Enabled)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.HAlign(HAlign_Center)
				.Padding(3.5f, 0.0f)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SCompositePassTreeItemRow::IsItemEnabled)
					.OnCheckStateChanged(this, &SCompositePassTreeItemRow::OnIsItemEnabledChanged)
					.IsEnabled(this, &SCompositePassTreeItemRow::CanToggleEnabled)
				];
		}
		if (InColumnName == CompositePassTree::PassTreeColumn_Name)
		{
			return SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(6.f, 0.f, 0.f, 0.f)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
				]
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 1.0f, 6.0f, 1.0f)
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					.HeightOverride(16.0f)
					[
						SNew(SImage).Image(this, &SCompositePassTreeItemRow::GetItemIcon)
					]
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(NameTextBlock, SInlineEditableTextBlock)
					.Text(this, &SCompositePassTreeItemRow::GetItemName)
					.ToolTipText(this, &SCompositePassTreeItemRow::GetItemToolTip)
					.OnTextCommitted(this, &SCompositePassTreeItemRow::SetItemName)
					.IsReadOnly(this, &SCompositePassTreeItemRow::IsItemNameReadOnly)
					.IsSelected(FIsSelected::CreateSP(this, &SCompositePassTreeItemRow::IsSelectedExclusively))
				];
		}
		if (InColumnName == CompositePassTree::PassTreeColumn_Type)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(4.5f, 0.0f)
				[
					SNew(STextBlock)
					.Text(this, &SCompositePassTreeItemRow::GetItemType)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
		}
		
		return SNullWidget::NullWidget;
	}

	/** Gets the checkbox state for the row item's Enabled flag */
	ECheckBoxState IsItemEnabled() const
	{
		if (!TreeItem->HasValidOwner())
		{
			return ECheckBoxState::Unchecked;
		}
		
		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				return Pass->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			// In the case where there is a valid pass index, but the pass itself is null
			return ECheckBoxState::Unchecked;
		}
		
		// Tree item is a pass type group item, so set its checkbox state as the aggregate of all its children

		bool bAnyEnabled = false;
		bool bAnyDisabled = false;
		for (const FTreeItemPtr& ChildTreeItem : TreeItem->Children)
		{
			if (!ChildTreeItem.IsValid() || !ChildTreeItem->HasValidOwner() || !ChildTreeItem->HasValidPassIndex())
			{
				continue;
			}

			if (UCompositePassBase* Pass = ChildTreeItem->GetPass())
			{
				if (Pass->GetIsEnabled())
				{
					bAnyEnabled = true;
				}
				else
				{
					bAnyDisabled = true;
				}
			}
		}

		if (bAnyEnabled)
		{
			return bAnyDisabled ? ECheckBoxState::Undetermined : ECheckBoxState::Checked;
		}

		return ECheckBoxState::Unchecked;
	}

	/** Gets whether the row item's Enabled checkbox can be toggled */
	bool CanToggleEnabled() const
	{
		if (!TreeItem->HasValidOwner())
		{
			return false;
		}

		// Individual pass items can always be toggled
		if (TreeItem->HasValidPassIndex())
		{
			return true;
		}

		// Group items can only be toggled when they have children
		return !TreeItem->Children.IsEmpty();
	}

	/** Raised when the row item's Enabled checkbox has changed state */
	void OnIsItemEnabledChanged(ECheckBoxState InCheckBoxState)
	{
		if (!TreeItem->HasValidOwner())
		{
			return;
		}
		
		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				FScopedTransaction SetEnabledTransaction(LOCTEXT("SetEnabledTransaction", "Set Enabled"));
				Pass->Modify();

				Pass->SetIsEnabled(InCheckBoxState == ECheckBoxState::Checked);
				return;
			}

			// In the case where there is a valid pass index, but the pass itself is null
			return;
		}
		
		// Tree item is a group item, so set enable state of all child passes
		FScopedTransaction SetAllEnabledTransaction(LOCTEXT("SetAllEnabledTransaction", "Set All Enabled"));
		for (const FTreeItemPtr& ChildTreeItem : TreeItem->Children)
		{
			if (!ChildTreeItem.IsValid() || !ChildTreeItem->HasValidOwner() || !ChildTreeItem->HasValidPassIndex())
			{
				continue;
			}

			if (UCompositePassBase* Pass = ChildTreeItem->GetPass())
			{
				Pass->Modify();
				Pass->SetIsEnabled(InCheckBoxState == ECheckBoxState::Checked);
			}
		}
	}
	
	/** Gets the row item's icon to display next to its name */
	const FSlateBrush* GetItemIcon() const
	{
		if (!TreeItem->HasValidOwner())
		{
			return nullptr;
		}
		
		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				return FSlateIconFinder::FindIconForClass(Pass->GetClass()).GetIcon();
			}
			
			// In the case where there is a valid pass index, but the pass itself is null
			return nullptr;
		}

		// Tree item is a group item, so return icon for the group
		return TreeItem->GetGroupIcon();
	}

	/** Gets the row item's display name */
	FText GetItemName() const
	{
		if (!TreeItem->HasValidOwner())
		{
			return FText::GetEmpty();
		}
		
		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				return FText::FromString(Pass->GetDisplayName());
			}
			
			// In the case where there is a valid pass index, but the pass itself is null
			return LOCTEXT("PassNotSetLabel", "Pass not set");
		}

		// Tree item is a group item, so return the label for the group
		return TreeItem->GetGroupDisplayName();
	}

	/** Gets the row item's tooltip, drawn from the pass class's documentation. */
	FText GetItemToolTip() const
	{
		if (TreeItem.IsValid() && TreeItem->HasValidOwner() && TreeItem->HasValidPassIndex())
		{
			if (const UCompositePassBase* Pass = TreeItem->GetPass())
			{
				return Pass->GetClass()->GetToolTipText();
			}
		}

		return FText::GetEmpty();
	}

	/** Sets this row item's display name */
	void SetItemName(const FText& InNewText, ETextCommit::Type InCommitType)
	{
		if (!TreeItem->HasValidOwner())
		{
			return;
		}
		
		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				FScopedTransaction SetPassNameTransaction(LOCTEXT("SetPassNameTransaction", "Set Name"));
				Pass->Modify();

				Pass->SetDisplayName(InNewText.ToString());
			}
		}
	}

	/** Gets whether the row item's display name can be changed */
	bool IsItemNameReadOnly() const
	{
		if (!TreeItem->HasValidOwner())
		{
			return true;
		}

		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				return false;
			}
		}

		return true;
	}
	
	/** Gets the row item's display name */
	FText GetItemType() const
	{
		if (!TreeItem->HasValidOwner())
		{
			return FText::GetEmpty();
		}
		
		if (TreeItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = TreeItem->GetPass())
			{
				return Pass->GetClass()->GetDisplayNameText();
			}
			
			// In the case where there is a valid pass index, but the pass itself is null
            return LOCTEXT("NoneTypeLabel", "None");
		}
		
		return FText::GetEmpty();
	}

	/** Raised when a drag start has been detected on this row item */
	FReply HandleDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		if (!TreeItem->HasValidOwner())
		{
			return FReply::Unhandled();
		}

		if (!TreeItem->HasValidPassIndex())
		{
			return FReply::Unhandled();
		}
		
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TSharedPtr<FDragDropOperation> DragDropOp = FPassDragDropOp::New(TreeItem);
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}

		return FReply::Unhandled();
	}

	/** Raised when the user is attempting to drop something onto this item */
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, FTreeItemPtr InTreeItem)
	{
		TSharedPtr<FPassDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FPassDragDropOp>();

		if (!DragDropOp.IsValid() || !DragDropOp->DraggedItem.IsValid())
		{
			return TOptional<EItemDropZone>();
		}

		FTreeItemPtr DraggedItem = DragDropOp->DraggedItem.Pin();
		if (DraggedItem->HasValidOwner() && InTreeItem->HasValidOwner())
		{
			if (UCompositePassBase* Pass = DraggedItem->GetPass())
			{
				const int32 DestIndex = InItemDropZone == EItemDropZone::AboveItem ? InTreeItem->PassIndex: InTreeItem->PassIndex + 1;
				if (!InTreeItem->Owner->CanAddPass(Pass->GetClass(), InTreeItem->GroupIndex, DestIndex))
				{
					return TOptional<EItemDropZone>();
				}
			}
			
			if (!InTreeItem->HasValidPassIndex())
			{
				return EItemDropZone::BelowItem;
			}

			return InItemDropZone == EItemDropZone::OntoItem ? EItemDropZone::AboveItem : InItemDropZone;
		}
		
		return TOptional<EItemDropZone>();
	}

	/** Raised when the user has dropped something onto this item */
	FReply HandleAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, FTreeItemPtr InTreeItem)
	{
		TSharedPtr<FPassDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FPassDragDropOp>();

		if (!DragDropOp.IsValid() || !DragDropOp->DraggedItem.IsValid())
		{
			return FReply::Unhandled();
		}

		if (OnPassMoved.IsBound())
		{
			int32 DestIndex = InItemDropZone == EItemDropZone::AboveItem ? InTreeItem->PassIndex: InTreeItem->PassIndex + 1;

			// If the destination is further down than the item being moved, we must subtract one from the destination index
			// to account for the removal of the original item
			if (InTreeItem->PassIndex > DragDropOp->DraggedItem.Pin()->PassIndex && InTreeItem->GroupIndex == DragDropOp->DraggedItem.Pin()->GroupIndex)
			{
				--DestIndex;
			}
			
			OnPassMoved.Execute(DragDropOp->DraggedItem.Pin(), InTreeItem->GroupIndex, DestIndex);
			return FReply::Handled();
		}
		
		return FReply::Unhandled();
	}
	
private:
	/** The tree item this widget is outputting */
	FTreeItemPtr TreeItem;

	/** The editable text block displaying the item's name */
	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;
	
	/** Callback that is raised when a layer is moved via a drag and drop operation */
	FOnPassMoved OnPassMoved;
};

bool SCompositePassTree::FPassTreeItem::HasValidOwner() const
{
	return Owner.IsValid() && Owner->IsObjectValid();
}

bool SCompositePassTree::FPassTreeItem::HasValidPassIndex() const
{
	if (!Owner.IsValid())
	{
		return false;
	}

	return Owner->IsValidPassIndex(GroupIndex, PassIndex);
}

UCompositePassBase* SCompositePassTree::FPassTreeItem::GetPass() const
{
	if (!HasValidPassIndex())
	{
		return nullptr;
	}

	return Owner->GetPass(GroupIndex, PassIndex);
}

FString SCompositePassTree::FPassTreeItem::GetGroupFilterString() const
{
	if (!Owner.IsValid())
	{
		return TEXT("");
	}

	return Owner->GetGroupFilterString(GroupIndex);
}

const FSlateBrush* SCompositePassTree::FPassTreeItem::GetGroupIcon() const
{
	if (!Owner.IsValid())
	{
		return nullptr;
	}

	return Owner->GetGroupIcon(GroupIndex);
}

FText SCompositePassTree::FPassTreeItem::GetGroupDisplayName() const
{
	if (!Owner.IsValid())
	{
		return FText::GetEmpty();
	}

	return Owner->GetGroupDisplayName(GroupIndex);
}

void SCompositePassTree::Construct(const FArguments& InArgs, const TSharedPtr<ICompositePassListOwner>& InPassListOwner)
{
	PassListOwner = InPassListOwner;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	OnLayoutChanged = InArgs._OnLayoutChanged;

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SCompositePassTree::OnObjectPropertyChanged);
	
	CommandList = MakeShared<FUICommandList>();
	BindCommands();
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f)
		[
			SAssignNew(Toolbar, SCompositePassTreeToolbar, PassListOwner)
			.OnFilterChanged(this, &SCompositePassTree::OnFilterChanged)
			.OnPassAdded(this, &SCompositePassTree::OnPassAdded)
			.OnFilterNewPassType(this, &SCompositePassTree::OnFilterNewPassType)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(TreeView, STreeView<FPassTreeItemPtr>)
			.TreeItemsSource(&FilteredPassTreeItems)
			.HeaderRow(
				SNew(SHeaderRow)

				+SHeaderRow::Column(CompositePassTree::PassTreeColumn_Enabled)
				.DefaultLabel(FText::GetEmpty())
				.FixedWidth(24.0f)
				.HAlignHeader(HAlign_Center)
				.VAlignHeader(VAlign_Center)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SCompositePassTree::GetGlobalEnabledState)
					.OnCheckStateChanged(this, &SCompositePassTree::OnGlobalEnabledStateChanged)
					.IsEnabled(this, &SCompositePassTree::HasAnyPasses)
				]

				+SHeaderRow::Column(CompositePassTree::PassTreeColumn_Name)
				.FillWidth(0.55)
				.DefaultLabel(LOCTEXT("PassNameColumnLabel", "Pass Name"))

				+SHeaderRow::Column(CompositePassTree::PassTreeColumn_Type)
				.FillWidth(0.45)
				.DefaultLabel(LOCTEXT("PassTypeColumnLabel", "Pass Type"))
			)
			.OnGenerateRow_Lambda([this](FPassTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& InOwnerTable)
			{
				return SNew(SCompositePassTreeItemRow, InOwnerTable, InTreeItem)
					.OnPassMoved(this, &SCompositePassTree::OnPassMoved);
			})
			.OnGetChildren_Lambda([](FPassTreeItemPtr InTreeItem, TArray<FPassTreeItemPtr>& OutChildren)
			{
				if (InTreeItem.IsValid())
				{
					for (FPassTreeItemPtr& Child : InTreeItem->Children)
					{
						if (Child.IsValid() && !Child->bFilteredOut)
						{
							OutChildren.Add(Child);
						}
					}
				}
			})
			.OnSelectionChanged(this, &SCompositePassTree::OnPassSelectionChanged)
			.OnContextMenuOpening(this, &SCompositePassTree::CreateTreeContextMenu)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.MinHeight(24.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SCompositePassTree::GetFilterStatusText)
			.ColorAndOpacity(this, &SCompositePassTree::GetFilterStatusColor)
		]
	];

	FillPassTreeItems();
}

SCompositePassTree::~SCompositePassTree()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

FReply SCompositePassTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SCompositePassTree::PostUndo(bool bSuccess)
{
	constexpr bool bPreserveSelection = true;
	FillPassTreeItems(bPreserveSelection);
}

void SCompositePassTree::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SCompositePassTree::SelectPasses(const TArray<UCompositePassBase*>& InPasses)
{
	if (!TreeView.IsValid())
	{
		return;
	}
	
	TreeView->ClearSelection();

	for (UCompositePassBase* Pass : InPasses)
	{
		for (const FPassTreeItemPtr& TreeItem : FilteredPassTreeItems)
		{
			if (!TreeItem.IsValid())
			{
				continue;
			}
			
			FPassTreeItemPtr* TreeItemToSelect = TreeItem->Children.FindByPredicate([Pass](const FPassTreeItemPtr& ChildItem)
			{
				return ChildItem.IsValid() && ChildItem->HasValidPassIndex() && ChildItem->GetPass() == Pass;
			});

			if (TreeItemToSelect)
			{
				TreeView->SetItemSelection(*TreeItemToSelect, true);
			}
		}
	}
}

TArray<UCompositePassBase*> SCompositePassTree::GetSelectedPasses() const
{
	TArray<UCompositePassBase*> SelectedPasses;
	
	if (TreeView.IsValid())
	{
		TArray<FPassTreeItemPtr> SelectedItems = TreeView->GetSelectedItems();
		for (const FPassTreeItemPtr& SelectedItem : SelectedItems)
		{
			if (!SelectedItem.IsValid())
			{
				continue;
			}

			if (UCompositePassBase* Pass = SelectedItem->GetPass())
			{
				SelectedPasses.Add(Pass);
			}
		}
	}

	return SelectedPasses;
}

void SCompositePassTree::BindCommands()
{
	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SCompositePassTree::CopySelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanCopySelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SCompositePassTree::CutSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanCutSelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SCompositePassTree::PasteSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanPasteSelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SCompositePassTree::DuplicateSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanDuplicateSelectedItems));
	
	CommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SCompositePassTree::DeleteSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanDeleteSelectedItems));

	CommandList->MapAction(FGenericCommands::Get().Rename,
			FUIAction(FExecuteAction::CreateSP(this, &SCompositePassTree::RenameSelectedItem),
			FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanRenameSelectedItem)));
	
	CommandList->MapAction(FCompositeEditorCommands::Get().Enable,
		FExecuteAction::CreateSP(this, &SCompositePassTree::EnableSelectedItems),
		FCanExecuteAction::CreateSP(this, &SCompositePassTree::CanEnableSelectedItems));
}

void SCompositePassTree::FillPassTreeItems(bool bPreserveSelection)
{
	// A list of passes to reselect after the list of pass tree items has been regenerated
	TArray<UCompositePassBase*> PassesToReselect;
	if (bPreserveSelection)
	{
		TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
		for (const FPassTreeItemPtr& Item : SelectedItems)
		{
			if (Item.IsValid() && Item->HasValidPassIndex())
			{
				if (UCompositePassBase* Pass = Item->GetPass())
				{
					PassesToReselect.Add(Pass);
				}
			}
		}
	}
	
	PassTreeItems.Empty();

	if (PassListOwner.IsValid() && PassListOwner->IsObjectValid())
	{
		const int32 NumGroups = PassListOwner->GetNumGroups();
		if (NumGroups <= 0)
		{
			TArray<TObjectPtr<UCompositePassBase>>& Passes = PassListOwner->GetPassesForGroup(INDEX_NONE);
			for (int32 Index = 0; Index < Passes.Num(); ++Index)
			{
				FPassTreeItemPtr NewPassTreeItem = MakeShared<FPassTreeItem>();
				NewPassTreeItem->Owner = PassListOwner;
				NewPassTreeItem->GroupIndex = INDEX_NONE;
				NewPassTreeItem->PassIndex = Index;

				PassTreeItems.Add(NewPassTreeItem);
			}
		}
		else
		{
			for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
			{
				FPassTreeItemPtr NewPassTypeTreeItem = MakeShared<FPassTreeItem>();
				NewPassTypeTreeItem->Owner = PassListOwner;
				NewPassTypeTreeItem->GroupIndex = GroupIndex;
			
				PassTreeItems.Add(NewPassTypeTreeItem);
			
				TArray<TObjectPtr<UCompositePassBase>>& Passes = PassListOwner->GetPassesForGroup(GroupIndex);
				for (int32 Index = 0; Index < Passes.Num(); ++Index)
				{
					FPassTreeItemPtr NewPassTreeItem = MakeShared<FPassTreeItem>();
					NewPassTreeItem->Owner = PassListOwner;
					NewPassTreeItem->GroupIndex = GroupIndex;
					NewPassTreeItem->PassIndex = Index;

					NewPassTypeTreeItem->Children.Add(NewPassTreeItem);
				}
			}
		}
	}

	FilterPassTreeItems();
	RefreshAndExpandTreeView();

	if (TreeView.IsValid())
	{
		// Silence any selection changes while re-selecting, and track if the selection has changed (i.e. the originally selected items no longer exist)
		// If the selection has changed at the end of re-selection, the selection changed delegate will be invoked manually
		bool bSelectionChanged = false;
		{
			TGuardValue<bool> SilenceSelectionChangesGuard(bSilenceSelectionChanges, true);

			auto FindTreeItemForPass = [this](const UCompositePassBase* Pass)->FPassTreeItemPtr
			{
				for (const FPassTreeItemPtr& TreeItem : FilteredPassTreeItems)
				{
					if (!TreeItem.IsValid() || !TreeItem->HasValidOwner())
					{
						continue;
					}

					if (TreeItem->HasValidPassIndex())
					{
						if (TreeItem->GetPass() == Pass)
						{
							return TreeItem;
						}

						continue;
					}
					
					for (const FPassTreeItemPtr& ChildItem : TreeItem->Children)
					{
						if (ChildItem->HasValidPassIndex())
						{
							if (ChildItem->GetPass() == Pass)
							{
								return ChildItem;
							}
						}
					}
				}

				return nullptr;
			};
			
			for (const UCompositePassBase* PassToReselect : PassesToReselect)
			{
				FPassTreeItemPtr TreeItemToReselect = FindTreeItemForPass(PassToReselect);

				if (!TreeItemToReselect.IsValid())
				{
					// Pass no longer exists in the displayed list of pass items. Invoke selection changed to clear out selection from other UI elements
					bSelectionChanged = true;
					continue;
				}

				TreeView->SetItemSelection(TreeItemToReselect, true);
			}
		}

		if (bSelectionChanged)
		{
			OnPassSelectionChanged(nullptr, ESelectInfo::Direct);
		}
	}
}

void SCompositePassTree::FilterPassTreeItems()
{
	FilteredPassTreeItems.Empty();

	if (!Toolbar.IsValid())
	{
		return;
	}
	
	for (FPassTreeItemPtr& TreeItem : PassTreeItems)
	{
		TreeItem->bFilteredOut = !Toolbar->ItemPassesFilters(TreeItem);

		bool bHasUnfilteredChildren = false;
		for (FPassTreeItemPtr& ChildTreeItem : TreeItem->Children)
		{
			ChildTreeItem->bFilteredOut = !Toolbar->ItemPassesFilters(ChildTreeItem);
			if (!ChildTreeItem->bFilteredOut)
			{
				bHasUnfilteredChildren = true;
			}
		}

		if (!TreeItem->bFilteredOut || bHasUnfilteredChildren)
		{
			FilteredPassTreeItems.Add(TreeItem);
		}
	}
}

void SCompositePassTree::RefreshGroup(int32 InGroupIndex, bool bRefreshTreeView)
{
	if (InGroupIndex == INDEX_NONE)
	{
		// Remove any pass tree items that aren't children of a group
		PassTreeItems.RemoveAll([](const FPassTreeItemPtr& TreeItem)
		{
			return !TreeItem.IsValid() || TreeItem->GroupIndex == INDEX_NONE;
		});

		TArray<TObjectPtr<UCompositePassBase>>& Passes = PassListOwner->GetPassesForGroup(INDEX_NONE);
		for (int32 Index = 0; Index < Passes.Num(); ++Index)
		{
			FPassTreeItemPtr NewPassTreeItem = MakeShared<FPassTreeItem>();
			NewPassTreeItem->Owner = PassListOwner;
			NewPassTreeItem->GroupIndex = INDEX_NONE;
			NewPassTreeItem->PassIndex = Index;

			PassTreeItems.Add(NewPassTreeItem);
		}
	}
	else
	{
		FPassTreeItemPtr PassTypeTreeItem = PassTreeItems[InGroupIndex];
		PassTypeTreeItem->Children.Empty();

		if (!PassListOwner.IsValid() || !PassListOwner->IsObjectValid())
		{
			return;
		}
	
		TArray<TObjectPtr<UCompositePassBase>>& Passes = PassListOwner->GetPassesForGroup(InGroupIndex);
		for (int32 Index = 0; Index < Passes.Num(); ++Index)
		{
			FPassTreeItemPtr NewPassTreeItem = MakeShared<FPassTreeItem>();
			NewPassTreeItem->Owner = PassListOwner;
			NewPassTreeItem->GroupIndex = InGroupIndex;
			NewPassTreeItem->PassIndex = Index;

			PassTypeTreeItem->Children.Add(NewPassTreeItem);
		}
	}
	
	FilterPassTreeItems();

	if (bRefreshTreeView)
	{
		RefreshAndExpandTreeView();
	}
}

void SCompositePassTree::RefreshAndExpandTreeView()
{
	if (TreeView.IsValid())
	{
		TreeView->RebuildList();

		for (int32 Index = 0; Index < FilteredPassTreeItems.Num(); ++Index)
		{
			TreeView->SetItemExpansion(FilteredPassTreeItems[Index], true);
		}

		OnLayoutChanged.ExecuteIfBound();
	}
}

TSharedPtr<SWidget> SCompositePassTree::CreateTreeContextMenu()
{
	if (!TreeView.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	const int32 NumItems = TreeView->GetNumItemsSelected();
	if (NumItems >= 1)
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);

		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry(FCompositeEditorCommands::Get().Enable);
		
		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

ECheckBoxState SCompositePassTree::GetGlobalEnabledState() const
{
	if (!PassListOwner.IsValid() || !PassListOwner->IsObjectValid())
	{
		return ECheckBoxState::Checked;
	}

	// Determine if all passes are disabled, enabled, or if there is a mix
	bool bAnyPassesEnabled = false;
	bool bAnyPassesDisabled = false;

	auto CheckPassListEnabledStatus = [this, &bAnyPassesEnabled, &bAnyPassesDisabled](int32 InGroupIndex)
	{
		const TArray<TObjectPtr<UCompositePassBase>>& PassList = PassListOwner->GetPassesForGroup(InGroupIndex);

		for (const TObjectPtr<UCompositePassBase>& Pass : PassList)
		{
			if (!Pass)
			{
				continue;
			}
			
			if (Pass->GetIsEnabled())
			{
				bAnyPassesEnabled = true;
			}
			else
			{
				bAnyPassesDisabled = true;
			}
		}
	};
	
	const int32 NumGroups = PassListOwner->GetNumGroups();
	if (NumGroups <= 0)
	{
		CheckPassListEnabledStatus(INDEX_NONE);
	}
	else
	{
		for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
		{
			CheckPassListEnabledStatus(GroupIndex);
		}
	}

	if (bAnyPassesEnabled)
	{
		return bAnyPassesDisabled ? ECheckBoxState::Undetermined : ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

bool SCompositePassTree::HasAnyPasses() const
{
	if (!PassListOwner.IsValid() || !PassListOwner->IsObjectValid())
	{
		return false;
	}

	const int32 NumGroups = PassListOwner->GetNumGroups();
	if (NumGroups <= 0)
	{
		return !PassListOwner->GetPassesForGroup(INDEX_NONE).IsEmpty();
	}

	for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
	{
		if (!PassListOwner->GetPassesForGroup(GroupIndex).IsEmpty())
		{
			return true;
		}
	}

	return false;
}

void SCompositePassTree::OnGlobalEnabledStateChanged(ECheckBoxState CheckBoxState)
{
	if (!PassListOwner.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> OwnerObject = PassListOwner->GetObject();
	if (!OwnerObject.IsValid())
	{
		return;
	}
	
	TSharedPtr<FScopedTransaction> AllEnabledTransaction;

	auto SetPassListEnabledStatus = [this, &AllEnabledTransaction, &OwnerObject, &CheckBoxState](int32 InGroupIndex)
	{
		const TArray<TObjectPtr<UCompositePassBase>>& PassList = PassListOwner->GetPassesForGroup(InGroupIndex);

		for (const TObjectPtr<UCompositePassBase>& Pass : PassList)
		{
			if (!Pass)
			{
				continue;
			}

			if (!AllEnabledTransaction.IsValid())
			{
				AllEnabledTransaction = MakeShared<FScopedTransaction>(LOCTEXT("AllEnabledTransaction", "Set All Enabled"));
				OwnerObject->Modify();
			}

			Pass->Modify();
			Pass->SetIsEnabled(CheckBoxState == ECheckBoxState::Checked);
		}
	};
	
	const int32 NumGroups = PassListOwner->GetNumGroups();
	if (NumGroups <= 0)
	{
		SetPassListEnabledStatus(INDEX_NONE);
	}
	else
	{
		for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
		{
			SetPassListEnabledStatus(GroupIndex);
		}
	}
}

void SCompositePassTree::OnPassMoved(const TSharedPtr<FPassTreeItem>& InTreeItem, int32 InDestGroupIndex, int InDestIndex)
{
	if (!InTreeItem.IsValid() ||
		!InTreeItem->HasValidOwner() ||
		!InTreeItem->HasValidPassIndex())
	{
		return;
	}

	FScopedTransaction MovePassTransaction(LOCTEXT("MovePassTransaction", "Move Pass"));
	PassListOwner->MovePass(InTreeItem->GroupIndex, InTreeItem->PassIndex, InDestGroupIndex, InDestIndex);
	
	RefreshGroup(InTreeItem->GroupIndex, /* bRefreshTreeView */ false);
	RefreshGroup(InDestGroupIndex);
}

void SCompositePassTree::OnPassSelectionChanged(TSharedPtr<FPassTreeItem> InTreeItem, ESelectInfo::Type SelectInfo)
{
	if (bSilenceSelectionChanges)
	{
		return;
	}
	
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	
	TArray<UObject*> SelectedLayers;
	SelectedLayers.Reserve(SelectedItems.Num());

	for (const FPassTreeItemPtr& Item : SelectedItems)
	{
		if (!Item->HasValidOwner())
		{
			continue;
		}

		if (Item->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = Item->GetPass())
			{
				SelectedLayers.Add(Pass);
			}
		}
	}
	
	OnSelectionChanged.ExecuteIfBound(SelectedLayers);
}

void SCompositePassTree::OnFilterChanged()
{
	FilterPassTreeItems();
	RefreshAndExpandTreeView();
}

void SCompositePassTree::OnPassAdded(const UClass* InPassClass)
{
	if (!PassListOwner.IsValid() || !PassListOwner->IsObjectValid())
	{
		return;
	}
	
	if (!IsValid(InPassClass) || InPassClass->IsChildOf<UCompositeLayerBase>())
	{
		return;
	}

	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (SelectedItems.IsEmpty())
	{
		// If a group filter pill is active, honour it; otherwise fall back to the class-based default group.
		const int32 ActiveGroupIndex = Toolbar.IsValid() ? Toolbar->GetActiveGroupFilterIndex() : INDEX_NONE;
		const int32 DefaultGroupIndex = (ActiveGroupIndex != INDEX_NONE) ? ActiveGroupIndex : PassListOwner->GetDefaultGroupForNewPass(InPassClass);
		if (DefaultGroupIndex != INDEX_NONE)
		{
			if (PassTreeItems.IsValidIndex(DefaultGroupIndex))
			{
				SelectedItems.Add(PassTreeItems[DefaultGroupIndex]);
			}
			else
			{
				SelectedItems.Add(PassTreeItems[0]);
			}
		}
		else
		{
			if (!PassTreeItems.IsEmpty())
			{
				SelectedItems.Add(PassTreeItems.Last());
			}
		}
	}

	int32 GroupIndex = INDEX_NONE;
	int32 PassIndex = PassTreeItems.Num();
	
	// Item after which the new pass will get added
	FPassTreeItemPtr AnchorItem = !SelectedItems.IsEmpty() ? SelectedItems.Last() : nullptr;
	if (AnchorItem.IsValid())
	{
		GroupIndex = AnchorItem->GroupIndex;
		PassIndex = AnchorItem->PassIndex + 1;
	}

	if (!PassListOwner->CanAddPass(InPassClass, GroupIndex, PassIndex))
	{
		return;
	}
	
	FScopedTransaction AddPassTransaction(LOCTEXT("AddPassTransaction", "Add Pass"));
	const int32 NewPassIndex = PassListOwner->AddPass(InPassClass, GroupIndex, PassIndex);

	// Auto-increment display name if a sibling of the same type already exists
	if (UCompositePassBase* NewPass = PassListOwner->GetPass(GroupIndex, NewPassIndex))
	{
		TArray<TObjectPtr<UCompositePassBase>>& Passes = PassListOwner->GetPassesForGroup(GroupIndex);
		const FString UniqueName = UCompositePassBase::MakeUniqueDisplayName(NewPass->GetDisplayName(), MakeArrayView(Passes), NewPass);

		NewPass->SetDisplayName(UniqueName);

		Composite::Analytics::RecordPassAdded(*NewPass);
	}

	// Clear filters so the new pass is visible — but only when no group filter is active.
	// If a group filter pill is active, the pass was just added to that group and is already visible.
	if (Toolbar.IsValid() && Toolbar->GetActiveGroupFilterIndex() == INDEX_NONE)
	{
		Toolbar->ClearFilters();
	}
	
	RefreshGroup(GroupIndex);

	FPassTreeItemPtr PassToSelect = nullptr;
	if (GroupIndex != INDEX_NONE)
	{
		if (PassTreeItems.IsValidIndex(GroupIndex) && PassTreeItems[GroupIndex]->Children.IsValidIndex(NewPassIndex))
		{
			PassToSelect = PassTreeItems[GroupIndex]->Children[NewPassIndex];
		}
	}
	else if (PassTreeItems.IsValidIndex(NewPassIndex) && PassTreeItems[NewPassIndex]->HasValidPassIndex())
	{
		PassToSelect = PassTreeItems[NewPassIndex];
	}
	
	if (TreeView.IsValid())
	{
		TreeView->ClearSelection();
		if (PassToSelect.IsValid())
		{
			TreeView->SetItemSelection(PassToSelect, true);
		}
	}
}

bool SCompositePassTree::OnFilterNewPassType(const UClass* InPassType) const
{
	if (!PassListOwner.IsValid() || !PassListOwner->IsObjectValid())
	{
		return true;
	}
	
	if (!IsValid(InPassType) || InPassType->IsChildOf<UCompositeLayerBase>())
	{
		return true;
	}

	// Filter out Distortion passes as those are implicitly added when there is a lens file on the composite actor camera
	if (InPassType->IsChildOf<UCompositePassDistortion>())
	{
		return true;
	}

	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (SelectedItems.IsEmpty())
	{
		// No row selected — if a group filter pill is active, validate against that group.
		const int32 ActiveGroupIndex = Toolbar.IsValid() ? Toolbar->GetActiveGroupFilterIndex() : INDEX_NONE;
		if (ActiveGroupIndex != INDEX_NONE)
		{
			return !PassListOwner->CanAddPass(InPassType, ActiveGroupIndex, INDEX_NONE);
		}

		return false;
	}

	for (const FPassTreeItemPtr& Item : SelectedItems)
	{
		if (!PassListOwner->CanAddPass(InPassType, Item->GroupIndex, Item->PassIndex))
		{
			// If the pass type can't be added at even one of the selected items, filter that type out
			return true;
		}
	}

	return false;
}

FText SCompositePassTree::GetFilterStatusText() const
{
	int32 NumPasses = 0;
	for (const FPassTreeItemPtr& PassTypeItem : PassTreeItems)
	{
		NumPasses += PassTypeItem->Children.Num();
	}

	int32 NumFilteredPasses = 0;
	for (const FPassTreeItemPtr& PassTypeItem : FilteredPassTreeItems)
	{
		for (const FPassTreeItemPtr& PassItem : PassTypeItem->Children)
		{
			if (!PassItem->bFilteredOut)
			{
				++NumFilteredPasses;
			}
		}
	}
	
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	int32 NumSelectedPasses = 0;
	for (const FPassTreeItemPtr& PassItem : SelectedItems)
	{
		if (PassItem->HasValidPassIndex())
		{
			++NumSelectedPasses;
		}
	}
	
	const FText PassLabel = NumPasses > 1 ? LOCTEXT("PassPlural", "Passes") : LOCTEXT("PassSingular", "Pass");
	if (NumPasses > 0 && NumPasses == NumFilteredPasses)
	{
		if (NumSelectedPasses > 0)
		{
			return FText::Format(LOCTEXT("NumPassesAndSelectedTextFormat", "{0} {1}, {2} Selected"), FText::AsNumber(NumPasses), PassLabel, FText::AsNumber(NumSelectedPasses));
		}
		else
		{
			return FText::Format(LOCTEXT("NumPassesTextFormat", "{0} {1}"), FText::AsNumber(NumPasses), PassLabel);
		}
	}
	else if (NumFilteredPasses > 0)
	{
		if (NumSelectedPasses > 0)
		{
			return FText::Format(LOCTEXT("NumPassesWithFilteredAndSelectedTextFormat", "Showing {0} of {1} {2}, {3} Selected"),
				FText::AsNumber(NumSelectedPasses),
				FText::AsNumber(NumPasses),
				PassLabel,
				FText::AsNumber(NumSelectedPasses));
		}
		else
		{
			return FText::Format(LOCTEXT("NumPassesWithFilteredTextFormat", "Showing {0} of {1} {2}"),
				FText::AsNumber(NumFilteredPasses),
				FText::AsNumber(NumPasses),
				PassLabel);
		}
	}
	else
	{
		if (NumPasses > 0)
		{
			return FText::Format(LOCTEXT("NoMatchingPassesTextFormat", "No matching passes ({0} {1})"), FText::AsNumber(NumPasses), PassLabel);
		}
		else
		{
			return LOCTEXT("NoPassesLabel", "0 Passes");
		}
	}
}

FSlateColor SCompositePassTree::GetFilterStatusColor() const
{
	int32 NumPasses = 0;
	for (const FPassTreeItemPtr& PassTypeItem : PassTreeItems)
	{
		NumPasses += PassTypeItem->Children.Num();
	}

	int32 NumFilteredPasses = 0;
	for (const FPassTreeItemPtr& PassTypeItem : FilteredPassTreeItems)
	{
		for (const FPassTreeItemPtr& PassItem : PassTypeItem->Children)
		{
			if (!PassItem->bFilteredOut)
			{
				++NumFilteredPasses;
			}
		}
	}

	if (NumPasses > 0 && NumFilteredPasses == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	
	return FSlateColor::UseForeground();
}

void SCompositePassTree::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!PassListOwner.IsValid())
	{
		return;
	}

	if (InObject != PassListOwner->GetObject().Get())
	{
		return;
	}

	const FName PropName = InPropertyChangedEvent.GetPropertyName();
	if (PassListOwner->IsPassListPropertyName(PropName))
	{
		FillPassTreeItems();
	}
}

void SCompositePassTree::CopySelectedItems()
{
	TArray<UObject*> ObjectsToCopy;

	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	for (const FPassTreeItemPtr& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsValid() && SelectedItem->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = SelectedItem->GetPass())
			{
				ObjectsToCopy.Add(Pass);
			}
		}
	}

	if (ObjectsToCopy.IsEmpty())
	{
		return;
	}
	
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	for (UObject* Object : ObjectsToCopy)
	{
		UExporter::ExportToOutputDevice(&Context, Object, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, nullptr);
	}

	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

bool SCompositePassTree::CanCopySelectedItems()
{
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (!SelectedItems.Num())
	{
		return false;
	}

	const bool bContainsPass = SelectedItems.ContainsByPredicate([](const FPassTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->HasValidPassIndex() && InTreeItem->GetPass();
	});

	return bContainsPass;
}

void SCompositePassTree::CutSelectedItems()
{
	CopySelectedItems();
	DeleteSelectedItems();
}

bool SCompositePassTree::CanCutSelectedItems()
{
	return CanCopySelectedItems() && CanDeleteSelectedItems();
}

class FCompositePassObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FCompositePassObjectTextFactory() : FCustomizableTextObjectFactory(GWarn) { }

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		return ObjectClass->IsChildOf(UCompositePassBase::StaticClass()) && !ObjectClass->IsChildOf(UCompositeLayerBase::StaticClass());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if (UCompositePassBase* Pass = Cast<UCompositePassBase>(NewObject))
		{
			if (!Pass->IsA<UCompositeLayerBase>())
			{
				CompositePasses.Add(Pass);
			}
		}
	}

public:
	TArray<UCompositePassBase*> CompositePasses;
};

void SCompositePassTree::PasteSelectedItems()
{
	if (!PassListOwner.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> OwnerObject = PassListOwner->GetObject();
	if (!OwnerObject.IsValid())
	{
		return;
	}
	
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	// A list of groups to paste the clipboard contents in, and the index to paste at
	TMap<int32, int32> GroupsToPasteTo;
	for (const FPassTreeItemPtr& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsValid())
		{
			if (GroupsToPasteTo.Contains(SelectedItem->GroupIndex))
			{
				GroupsToPasteTo[SelectedItem->GroupIndex] = FMath::Max(SelectedItem->PassIndex + 1, GroupsToPasteTo[SelectedItem->GroupIndex]);
			}
			else
			{
				GroupsToPasteTo.Add(SelectedItem->GroupIndex, SelectedItem->PassIndex + 1);
			}
		}
	}

	if (GroupsToPasteTo.IsEmpty())
	{
		return;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	
	FCompositePassObjectTextFactory Factory;
	Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardContent);

	if (Factory.CompositePasses.IsEmpty())
	{
		return;
	}

	TMap<int32, TArray<int32>> PassesToSelect;
	FScopedTransaction PastePassesTransaction(LOCTEXT("PastePassesTransaction", "Paste Passes"));
	OwnerObject->Modify();
	for (const TPair<int32, int32>& Pair : GroupsToPasteTo)
	{
		const int32 GroupToPasteAt = Pair.Key;
		int32 IndexToPasteAt = Pair.Value;

		PassesToSelect.Add(GroupToPasteAt, PassListOwner->CopyPasses(Factory.CompositePasses, GroupToPasteAt, IndexToPasteAt));
		
		constexpr bool bRefreshTree = false;
		RefreshGroup(GroupToPasteAt, bRefreshTree);
	}

	FilterPassTreeItems();
	RefreshAndExpandTreeView();

	if (TreeView.IsValid())
	{
		for (const TPair<int32, TArray<int32>>& Pair : PassesToSelect)
		{
			const int32 GroupIndex = Pair.Key;

			if (GroupIndex != INDEX_NONE)
			{
				if (PassTreeItems.IsValidIndex(GroupIndex))
				{
					for (int32 IndexToSelect : Pair.Value)
					{
						if (PassTreeItems[GroupIndex]->Children.IsValidIndex(IndexToSelect))
						{
							TreeView->SetItemSelection(PassTreeItems[GroupIndex]->Children[IndexToSelect], true);
						}
					}
				}
			}
			else
			{
				for (int32 IndexToSelect : Pair.Value)
				{
					if (PassTreeItems.IsValidIndex(IndexToSelect) && PassTreeItems[IndexToSelect]->HasValidPassIndex())
					{
						TreeView->SetItemSelection(PassTreeItems[IndexToSelect], true);
					}
				}
			}
		}
	}
}

bool SCompositePassTree::CanPasteSelectedItems()
{
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (SelectedItems.IsEmpty())
	{
		return false;
	}
	
	// Can't paste unless the clipboard has a string in it
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (!ClipboardContent.IsEmpty())
	{
		FCompositePassObjectTextFactory Factory;
		Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional | RF_Transient, ClipboardContent);
		return !Factory.CompositePasses.IsEmpty();
	}

	return false;
}

void SCompositePassTree::DuplicateSelectedItems()
{
	CopySelectedItems();
	PasteSelectedItems();
}

bool SCompositePassTree::CanDuplicateSelectedItems()
{
	return CanCopySelectedItems();
}

void SCompositePassTree::DeleteSelectedItems()
{
	if (!PassListOwner.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> OwnerObject = PassListOwner->GetObject();
	if (!OwnerObject.IsValid())
	{
		return;
	}
	
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();

	TMap<int32, TArray<int32>> PassesToDelete;
	for (const FPassTreeItemPtr& Item : SelectedItems)
	{
		if (!Item.IsValid() || !Item->HasValidPassIndex())
		{
			continue;
		}

		if (PassesToDelete.Contains(Item->GroupIndex))
		{
			PassesToDelete[Item->GroupIndex].Add(Item->PassIndex);
		}
		else
		{
			PassesToDelete.Add(Item->GroupIndex, TArray<int32> { Item->PassIndex });
		}
	}

	if (PassesToDelete.Num())
	{
		FScopedTransaction DeletePassTransaction(LOCTEXT("DeletePassTransaction", "Delete Pass"));
		OwnerObject->Modify();
		
		for (TPair<int32, TArray<int32>>& PassesToDeleteByType : PassesToDelete)
		{
			const int32 GroupIndex = PassesToDeleteByType.Key;
			TArray<int32>& PassIndicesToDelete = PassesToDeleteByType.Value;

			PassListOwner->RemovePasses(GroupIndex, PassIndicesToDelete);
			
			constexpr bool bRefreshTreeView = false;
			RefreshGroup(GroupIndex, bRefreshTreeView);
		}

		RefreshAndExpandTreeView();
	}
}

bool SCompositePassTree::CanDeleteSelectedItems()
{
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	const bool bContainsPassInSelection = SelectedItems.ContainsByPredicate([](const FPassTreeItemPtr& InTreeItem)
	{
		return InTreeItem.IsValid() && InTreeItem->HasValidPassIndex();
	});
	
	return bContainsPassInSelection;
}

void SCompositePassTree::RenameSelectedItem()
{
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (SelectedItems.Num() != 1 || !SelectedItems[0]->HasValidPassIndex() || !SelectedItems[0]->GetPass())
	{
		return;
	}
	
	TSharedPtr<ITableRow> RowWidget = TreeView->WidgetFromItem(SelectedItems[0]);
	if (TSharedPtr<SCompositePassTreeItemRow> TreeItemRowWidget = StaticCastSharedPtr<SCompositePassTreeItemRow>(RowWidget))
	{
		TreeItemRowWidget->RequestRename();
	}
}

bool SCompositePassTree::CanRenameSelectedItem()
{
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	if (SelectedItems.Num() == 1)
	{
		return SelectedItems[0].IsValid() && SelectedItems[0]->HasValidPassIndex() && SelectedItems[0]->GetPass();
	}
	
	return false;
}

void SCompositePassTree::EnableSelectedItems()
{
	if (!PassListOwner.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> OwnerObject = PassListOwner->GetObject();
	if (!OwnerObject.IsValid())
	{
		return;
	}
	
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();

	bool bAnyEnabled = false;
	for (const FPassTreeItemPtr& Item : SelectedItems)
	{
		if (Item->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = Item->GetPass())
			{
				if (Pass->GetIsEnabled())
				{
					bAnyEnabled = true;
				}
			}
		}
		else
		{
			for (const FPassTreeItemPtr& ChildItem : Item->Children)
			{
				if (ChildItem->HasValidPassIndex())
				{
					if (UCompositePassBase* Pass = ChildItem->GetPass())
					{
						if (Pass->GetIsEnabled())
						{
							bAnyEnabled = true;
						}
					}
				}
			}
		}
	}

	const bool bSetEnabled = !bAnyEnabled;

	FScopedTransaction SetEnabledTransaction(LOCTEXT("SetEnabledTransaction", "Set Enabled"));
	OwnerObject->Modify();

	for (const FPassTreeItemPtr& Item : SelectedItems)
	{
		if (Item->HasValidPassIndex())
		{
			if (UCompositePassBase* Pass = Item->GetPass())
			{
				Pass->Modify();
				Pass->SetIsEnabled(bSetEnabled);
			}
		}
		else
		{
			for (const FPassTreeItemPtr& ChildItem : Item->Children)
			{
				if (ChildItem->HasValidPassIndex())
				{
					if (UCompositePassBase* Pass = ChildItem->GetPass())
					{
						Pass->Modify();
						Pass->SetIsEnabled(bSetEnabled);
					}
				}
			}
		}
	}
}

bool SCompositePassTree::CanEnableSelectedItems()
{
	TArray<FPassTreeItemPtr> SelectedItems = TreeView.IsValid() ? TreeView->GetSelectedItems() : TArray<FPassTreeItemPtr>();
	return !SelectedItems.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
