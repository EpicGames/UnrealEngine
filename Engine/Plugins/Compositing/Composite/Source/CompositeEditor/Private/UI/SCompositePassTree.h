// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"

struct FSlateBrush;
class FUICommandList;
class SCompositePassTreeToolbar;
class UCompositePassBase;
class UCompositeLayerPlate;
template<typename T> class STreeView;

/** Wrapper around a UObject that owns one or more lists of composite passes to display in the pass tree */
class ICompositePassListOwner
{
public:
	/** Gets whether the backing UObject that owns the pass list(s) is valid */
	virtual bool IsObjectValid() = 0;

	/** Gets a strong pointer to the backing UObject that owns the pass list(s) */
	virtual TStrongObjectPtr<UObject> GetObject() = 0;

	/** Gets whether the specified property name matches any of the properties related to the pass list(s) being provided */
	virtual bool IsPassListPropertyName(const FName& InPropertyName) = 0;

	/** Gets whether the specified group and pass indices reference a valid pass in the owner's pass list(s) */
	virtual bool IsValidPassIndex(int32 InGroupIndex, int32 InPassIndex) = 0;

	/** Gets the pass for the specified group and pass indices, or nullptr if no pass is found */
	virtual UCompositePassBase* GetPass(int32 InGroupIndex, int32 InPassIndex) = 0;

	/** Gets the number of different groups all the passes managed by the owner should be stored and displayed in */
	virtual int32 GetNumGroups() const = 0;

	/** Gets all the passes for the specified group index; use INDEX_NONE if there are no distinct groups */
	virtual TArray<TObjectPtr<UCompositePassBase>>& GetPassesForGroup(int32 InGroupIndex) = 0;

	/** Gets the string used for filtering in the tree view for a specified pass group */
	virtual FString GetGroupFilterString(int32 InGroupIndex) = 0;

	/** Gets the icon used for displaying the group in the tree view for a specified pass group */
	virtual const FSlateBrush* GetGroupIcon(int32 InGroupIndex) = 0;

	/** Gets the display name used for displaying the group in the tree view for a specified pass group */
	virtual FText GetGroupDisplayName(int32 InGroupIndex) = 0;

	struct FGroupFilterConfig { FString FilterName; FText DisplayName; FText ToolTip; };
	/** Gets the filter config used to display filtering pills in the tree view for a specified pass group */
	virtual FGroupFilterConfig GetGroupFilterConfig(int32 InGroupIndex) = 0;

	/** Gets the default group a new pass should be added in when no group is selected, or INDEX_NONE if the pass should not be added to a group */
	virtual int32 GetDefaultGroupForNewPass(const UClass* InPassClass) const = 0;

	/** Gets whether a pass of the specified type can be added to the specified group at the specified location */
	virtual bool CanAddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) const = 0;

	/** Adds a pass of the specified type to the specified group at the specified location */
	virtual int32 AddPass(const UClass* InPassClass, int32 InGroupIndex, int32 InPassIndex) = 0;

	/** Copies all the specified passes into the specified group at the specified location */
	virtual TArray<int32> CopyPasses(const TArray<UCompositePassBase*>& InPassesToCopy, int32 InGroupIndex, int32 InPassIndex) = 0;

	/** Moves a pass from the specified group and location to the specified destination group and location */
	virtual void MovePass(int32 InSourceGroupIndex, int32 InSourcePassIndex, int32 InDestGroupIndex, int32 InDestPassIndex) = 0;

	/** Removes all passes at the specified group */
	virtual void RemovePasses(int32 InGroupIndex, const TArray<int32>& InPassIndices) = 0;
};

/**
 * Widget that displays a filterable tree view for the passes within a composite plate layer
 */
class SCompositePassTree : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, const TArray<UObject*>&);
	
private:
	/** Stores data needed for a single row item in the tree view */
	struct FPassTreeItem
	{
		/** Owner object whose passes are being displayed */
		TSharedPtr<ICompositePassListOwner> Owner = nullptr;

		/** If this tree item is for a specific pass, this is the pass's index in the plate's corresponding list of passes */
		int32 PassIndex = INDEX_NONE;

		/** The group this tree item belongs to */
		int32 GroupIndex = INDEX_NONE;
		
		/** Indicates if this tree item is being filtered out by the current filter */
		bool bFilteredOut = false;

		/** Child tree items of this tree item */
		TArray<TSharedPtr<FPassTreeItem>> Children;
		
		/** Gets whether the pass list owner pointer is valid */
		bool HasValidOwner() const;
		
		/** Gets whether the pass index is valid within the plate's list of passes */
		bool HasValidPassIndex() const;

		/** Gets the pass object referenced by this tree item */
		UCompositePassBase* GetPass() const;

		/** If this item is a group item, gets the filter string to use for the item */
		FString GetGroupFilterString() const;

		/** If this item is a group item, gets the icon to use for the item */
		const FSlateBrush* GetGroupIcon() const;

		/** If this item is a group item, gets the display name to use for the item */
		FText GetGroupDisplayName() const;
	};
	using FPassTreeItemPtr = TSharedPtr<FPassTreeItem>;
	
public:
	SLATE_BEGIN_ARGS(SCompositePassTree) { }
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FSimpleDelegate, OnLayoutChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<ICompositePassListOwner>& InPassListOwner);

	virtual ~SCompositePassTree() override;
	
	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// End of SWidget interface

	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface
	
	/** Selects the specified passes in the pass tree */
	void SelectPasses(const TArray<UCompositePassBase*>& InPasses);

	/** Gets a list of currently selected passes in the pass tree */
	TArray<UCompositePassBase*> GetSelectedPasses() const;
	
private:
	/** Binds commands in the command list to callbacks */
	void BindCommands();
	
	/** Creates any tree items needed for all passes in the plate */
	void FillPassTreeItems(bool bPreserveSelection = false);

	/** Applies the current filter to the pass tree items to build a filtered list of tree items */
	void FilterPassTreeItems();

	/** Refreshes the tree items for the passes in the specified group */
	void RefreshGroup(int32 InGroupIndex, bool bRefreshTreeView = true);
	
	/** Refreshes the tree view's displayed list and automatically expands all tree items */
	void RefreshAndExpandTreeView();

	/** Creates the tree view right click context menu */
	TSharedPtr<SWidget> CreateTreeContextMenu();

	/** Gets the global enabled state for all passes on the plate */
	ECheckBoxState GetGlobalEnabledState() const;

	/** Sets the enabled state for all passes on the plate */
	void OnGlobalEnabledStateChanged(ECheckBoxState CheckBoxState);

	/** Gets whether any passes exist across all groups */
	bool HasAnyPasses() const;
	
	/** Raised when a pass has been moved via a drag drop operation */
	void OnPassMoved(const TSharedPtr<FPassTreeItem>& InTreeItem, int32 InDestGroupIndex, int InDestIndex);

	/** Raised when the selected items in the tree view have bene changed */
	void OnPassSelectionChanged(TSharedPtr<FPassTreeItem> InTreeItem, ESelectInfo::Type SelectInfo);
	
	/** Raised when the active filter has been changed */
	void OnFilterChanged();

	/** Raised when a new pass is being added */
	void OnPassAdded(const UClass* InPassClass);

	/** Raised when the toolbar is adding pass types to the Add Pass menu, allowing types to be filtered out */
	bool OnFilterNewPassType(const UClass* InPassType) const;
	
	/** Gets status text based on selection and active filter */
	FText GetFilterStatusText() const;

	/** Gets status text color based on selection and active filter */
	FSlateColor GetFilterStatusColor() const;

	/** Raised when a property on the object has changed */
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	
	/** Generic command callbacks to handle copy, cut, paste, delete, and enable */
	void CopySelectedItems();
	bool CanCopySelectedItems();
	void CutSelectedItems();
	bool CanCutSelectedItems();
	void PasteSelectedItems();
	bool CanPasteSelectedItems();
	void DuplicateSelectedItems();
	bool CanDuplicateSelectedItems();
	void DeleteSelectedItems();
	bool CanDeleteSelectedItems();
	void RenameSelectedItem();
	bool CanRenameSelectedItem();
	void EnableSelectedItems();
	bool CanEnableSelectedItems();
	
private:
	/** The owner whose passes are being displayed */
	TSharedPtr<ICompositePassListOwner> PassListOwner;

	/** List of all pass tree items for the plate */
	TArray<FPassTreeItemPtr> PassTreeItems;

	/** List of all passes that match the current filter */
	TArray<FPassTreeItemPtr> FilteredPassTreeItems;
	
	/** Tree view that displays a hierarchical list of passes on the plate */
	TSharedPtr<STreeView<FPassTreeItemPtr>> TreeView;

	/** Toolbar widget displayed above the tree view */
	TSharedPtr<SCompositePassTreeToolbar> Toolbar;

	/** The command list for commands related to the tree view */
	TSharedPtr<FUICommandList> CommandList;

	/** When true, selection changes in the tree view will not invoke the SelectionChanged delegate */
	bool bSilenceSelectionChanges;
	
	/** Delegate that gets raised when the tree view selection changes */
	FOnSelectionChanged OnSelectionChanged;

	/** Delegate raised when there is a potential layout change to the tree view, to give containers the opportunity to correctly resize their layouts */
	FSimpleDelegate OnLayoutChanged;
	
	friend class SCompositePassTreeItemRow;
	friend class SCompositePassTreeToolbar;
	friend class FCompositePassTreeFilter;
};
