// Copyright Epic Games, Inc. All Rights Reserved.


#include "SCompositePassTreePanel.h"

#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "PropertyPath.h"
#include "SCompositeEditorPanel.h"
#include "SCompositePassTree.h"
#include "Customizations/CompositePassBaseCustomization.h"
#include "Layers/CompositeLayerPlate.h"
#include "Modules/ModuleManager.h"
#include "Sequencer/CompositeDetailKeyframeHandler.h"

void SCompositePassTreePanel::Construct(const FArguments& InArgs, const TSharedPtr<ICompositePassListOwner>& InPassListOwner)
{
	OnLayoutSizeChanged = InArgs._OnLayoutSizeChanged;
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowScrollBar = false;
	DetailsViewArgs.bAllowSearch = false;
	
	DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	DetailsView->SetKeyframeHandler(MakeShared<FCompositeDetailKeyframeHandler>());

	DetailsView->RegisterInstancedCustomPropertyLayout(
		UCompositePassBase::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCompositePassBaseCustomization::MakeInstance));

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(TreeView, SCompositePassTree, InPassListOwner)
			.OnSelectionChanged(this, &SCompositePassTreePanel::OnTreeViewSelectionChanged)
			.OnLayoutChanged_Lambda([this]
			{
				OnLayoutSizeChanged.ExecuteIfBound();
			})
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			DetailsView.ToSharedRef()
		]
	];
}

void SCompositePassTreePanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// This panel often changes its size dynamically due to the details panel (showing new objects, expanding properties, etc.)
	// When this panel is embedded into a parent details panel (or any list/tree view), the dynamic size is an issue when it comes to
	// determining scroll bar visibility and size. As such, this panel invokes a OnLayoutSizeChange event that allows parent containers
	// to request a layout refresh, where the correct scroll bar size can be computed. To detect if the details view may have changed size,
	// we can use GetPropertyRowNumbers, which returns a list of properties that are actively being displayed as widgets (thus will exclude collapsed
	// properties). Caching this count on tick allows this panel to detect if the object properties being displayed have changed or been expanded.
	// The actual layout size changed event must be invoked one tick after a row count difference has been detected to give the details view
	// time to actually change its rendered geometry
	
	if (bLayoutSizeChanged)
	{
		OnLayoutSizeChanged.ExecuteIfBound();
		bLayoutSizeChanged = false;
	}
	
	const int32 RowCount = DetailsView->GetPropertyRowNumbers().Num();	
 	if (CachedDetailsViewRowCount != RowCount)
	{
		CachedDetailsViewRowCount = RowCount;
 		bLayoutSizeChanged = true;
	}
}

void SCompositePassTreePanel::SelectPasses(TArray<UCompositePassBase*>& InPasses)
{
	if (TreeView.IsValid())
	{
		TreeView->SelectPasses(InPasses);
	}
}

TArray<UCompositePassBase*> SCompositePassTreePanel::GetSelectedPasses() const
{
	if (TreeView)
	{
		return TreeView->GetSelectedPasses();
	}

	return { };
}

void SCompositePassTreePanel::OnTreeViewSelectionChanged(const TArray<UObject*>& Objects)
{
	if (!DetailsView.IsValid())
	{
		return;
	}

	DetailsView->SetObjects(Objects);

	SCompositeEditorPanel::GetOnSelectionChanged().Broadcast();
}

