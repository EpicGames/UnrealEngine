// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class ICompositePassListOwner;
class SCompositePassTree;
class UCompositePassBase;
class UCompositeLayerPlate;

/**
 * A panel that displays a tree view and a details panel for passes within the plate composite layer
 */
class SCompositePassTreePanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCompositePassTreePanel) { }
		SLATE_EVENT(FSimpleDelegate, OnLayoutSizeChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<ICompositePassListOwner>& InPassListOwner);

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

	/** Selects the specified passes in the pass tree */
	void SelectPasses(TArray<UCompositePassBase*>& InPasses);

	/** Gets a list of currently selected passes in the pass tree */
	TArray<UCompositePassBase*> GetSelectedPasses() const;
	
private:
	/** Raised when the selected items in the tree view have changed */
	void OnTreeViewSelectionChanged(const TArray<UObject*>& Objects);
	
private:
	/** Tree view that displays a hierarchical list of passes in the plate */
	TSharedPtr<SCompositePassTree> TreeView;

	/** The details view to display selected pass properties */
	TSharedPtr<IDetailsView> DetailsView;

	/** To identify when the details view has changed size by showing a new object or expanding a property group, keep track of its reported row count every tick */
	int32 CachedDetailsViewRowCount = 0;

	/** Flag used to indicate if OnLayoutSizeChanged needs to be called on the next tick */
	bool bLayoutSizeChanged = false;
	
	/** Delegate that is raised when a potential size change has been detected in the panel (e.g. the details panel is displaying a new object) */
	FSimpleDelegate OnLayoutSizeChanged;
};
