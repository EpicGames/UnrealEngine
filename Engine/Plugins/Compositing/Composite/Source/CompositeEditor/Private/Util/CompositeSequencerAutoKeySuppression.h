// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ISequencer.h"
#include "LevelEditorSequencerIntegration.h"
#include "SequencerSettings.h"
#include "Templates/SharedPointer.h"

/**
 * RAII guard that suppresses Sequencer auto-key for the duration of its scope by flipping every
 * open Sequencer's AllowEditsMode to AllowLevelEditsOnly. Restores the previous mode on destruction.
 * Used by the customization helpers (sender-side, around SetValue) and the CompositeEditor module
 * (peer-side, across a Concert apply window).
 */
struct FScopedSequencerAutoKeySuppression
{
	FScopedSequencerAutoKeySuppression()
	{
		for (const TWeakPtr<ISequencer>& Weak : FLevelEditorSequencerIntegration::Get().GetSequencers())
		{
			if (TSharedPtr<ISequencer> Pinned = Weak.Pin())
			{
				CachedModes.Emplace(Weak, Pinned->GetAllowEditsMode());
				Pinned->SetAllowEditsMode(EAllowEditsMode::AllowLevelEditsOnly);
			}
		}
	}

	~FScopedSequencerAutoKeySuppression()
	{
		for (TPair<TWeakPtr<ISequencer>, EAllowEditsMode>& Entry : CachedModes)
		{
			if (TSharedPtr<ISequencer> Pinned = Entry.Key.Pin())
			{
				Pinned->SetAllowEditsMode(Entry.Value);
			}
		}
	}

	FScopedSequencerAutoKeySuppression(const FScopedSequencerAutoKeySuppression&) = delete;
	FScopedSequencerAutoKeySuppression& operator=(const FScopedSequencerAutoKeySuppression&) = delete;

private:
	TArray<TPair<TWeakPtr<ISequencer>, EAllowEditsMode>> CachedModes;
};
