// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelEditor.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

enum class ETransactionNotification;
struct FScopedSequencerAutoKeySuppression;
namespace UE::Sequencer { class FCompositeActorObjectSchema; }

/** Log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogCompositeEditor, Log, All);

class FCompositeEditorModule : public IModuleInterface
{
public:

	FCompositeEditorModule();
	~FCompositeEditorModule();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** Called after the engine has initialized. */
	void OnPostEngineInit();

	/** Called when an Actor has been added to a level. */
	void OnLevelActorAdded(AActor* InActor);

	/** Callback to trigger warnings when an holdout composite component is created with a composite actor. */
	void TriggerHoldoutCompositeWarning(const class UHoldoutCompositeComponent* InComponent);

	/** Register tab spawners. */
	void RegisterTabSpawners();

	/** Unregister tab spawners. */
	void UnregisterTabSpawners();

	void RegisterCustomizations();
	void UnregisterCustomizations();

	/** Subscribe to Concert apply notifications to suppress Sequencer auto-key on the receiving editor. */
	void RegisterConcertApplyListener();

	/** Unsubscribe and release any held suppression. */
	void UnregisterConcertApplyListener();

	/** Holds the Sequencer auto-key suppression for the duration of a Concert apply (Begin -> construct, End -> reset). */
	void OnConcertApplyTransaction(ETransactionNotification InNotification, const bool bIsSnapshot);

private:

	/** Tab manager change delegate. */
	FDelegateHandle OnTabManagerChangedDelegateHandle;

	/** Sequencer object schema for exposing layers and passes as possessable sub-objects. */
	TSharedPtr<UE::Sequencer::FCompositeActorObjectSchema> CompositeActorObjectSchema;

	/** Sequencer auto-key suppression held for the active Concert apply window. */
	TUniquePtr<FScopedSequencerAutoKeySuppression> ConcertApplyAutoKeySuppression;
};
